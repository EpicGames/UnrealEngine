[TakeRecorderSequenceEditor SequencerSettings]
; By default, Sequencer is configured to capture the frame the user has scrubbed to (see FSequencerThumbnailCaptureSettings::CaptureFrameLocationRule).
; In Take Recorder, the user uses no Sequencer UI - so we'll override the defaults.
ThumbnailCaptureSettings=(CaptureFrameLocationRule=MiddleFrame)