// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CacheTrackRecorder : ModuleRules
{
	public CacheTrackRecorder(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new[] {
				"Core",
				"CoreUObject",
				"Engine",
				"LevelEditor",
				"LevelSequence",
				"MovieScene",
				"TakesCore",
				"TakeMovieScene",
				"Settings",
				"Slate",
				"SlateCore",
				"UnrealEd",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new[] {
				"TakeTrackRecorders",
				"Sequencer",
			}
		);
	}
}
