// Copyright Epic Games, Inc. All Rights Reserved.

#include "CacheTrackRecorderModule.h"

#include "Editor.h"
#include "ToolMenus.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "Recorder/CacheTrackRecorder.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "CacheTrackRecorderModule"

IMPLEMENT_MODULE(FCacheTrackRecorderModule, CacheTrackRecorder);

FCacheTrackRecorderModule::FCacheTrackRecorderModule()
{
}

void FCacheTrackRecorderModule::StartupModule()
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->OnEditorClose().AddRaw(this, &FCacheTrackRecorderModule::OnEditorClose);
	}
#endif
}

void FCacheTrackRecorderModule::OnEditorClose()
{
	if (UCacheTrackRecorder* ActiveRecorder = UCacheTrackRecorder::GetActiveRecorder())
	{
		ActiveRecorder->Stop();
	}
}

void FCacheTrackRecorderModule::ShutdownModule()
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->OnEditorClose().RemoveAll(this);
	}

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
#endif
}


#undef LOCTEXT_NAMESPACE
