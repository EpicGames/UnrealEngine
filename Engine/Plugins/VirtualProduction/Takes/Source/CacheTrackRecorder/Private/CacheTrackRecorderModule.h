// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Delegates/IDelegateInstance.h"
#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Modules/ModuleInterface.h"
#include "Widgets/Input/SCheckBox.h"

class FCacheTrackRecorderModule : public IModuleInterface
{
public:
	FCacheTrackRecorderModule();
	
private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	void OnEditorClose();
};
