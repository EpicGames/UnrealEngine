// Copyright Epic Games, Inc. All Rights Reserved.

#include "Recorder/CacheTrackRecorder.h"

#include "AnimatedRange.h"
#include "CoreGlobals.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "ISequencer.h"
#include "LevelSequence.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "TakeMetaData.h"
#include "TakesUtils.h"
#include "Tickable.h"
#include "Stats/Stats.h"
#include "TrackRecorders/IMovieSceneTrackRecorderFactory.h"
#include "TrackRecorders/MovieSceneTrackRecorder.h"
#include "Tracks/MovieSceneCachedTrack.h"
#include "Features/IModularFeatures.h"

// Engine includes
#include "GameFramework/WorldSettings.h"

// UnrealEd includes
#include "Editor.h"

#include "UObject/GCObjectScopeGuard.h"

// Slate includes
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Notifications/INotificationWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Application/ThrottleManager.h"

// LevelEditor includes
#include "IAssetViewport.h"
#include "LevelEditor.h"
#include "SequencerSettings.h"
#include "Editor/EditorPerformanceSettings.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CacheTrackRecorder)

#define LOCTEXT_NAMESPACE "CacheTrackRecorder"

FCacheRecorderUserParameters::FCacheRecorderUserParameters()
	: bMaximizeViewport(false)
	, EngineTimeDilation(1.f)
	, bResetPlayhead(true)
	, bStopAtPlaybackEnd(true)
{
}

FCacheRecorderProjectParameters::FCacheRecorderProjectParameters()
	: RecordingClockSource(EUpdateClockSource::RelativeTimecode)
	, bStartAtCurrentTimecode(false)
	, bRecordTimecode(false)
	, bShowNotifications(true)
{}

class SCacheTrackRecorderNotification : public SCompoundWidget, public INotificationWidget
{
public:
	SLATE_BEGIN_ARGS(SCacheTrackRecorderNotification){}
	SLATE_END_ARGS()

	void SetOwner(TSharedPtr<SNotificationItem> InOwningNotification)
	{
		WeakOwningNotification = InOwningNotification;
	}

	void Construct(const FArguments&, UCacheTrackRecorder* InCacheTrackRecorder)
	{
		WeakRecorder = InCacheTrackRecorder;
		CacheTrackRecorderState = InCacheTrackRecorder->GetState();

		ChildSlot
		[
			SNew(SBox)
			.Padding(FMargin(15.0f))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.Padding(FMargin(0,0,0,5.0f))
				.HAlign(HAlign_Right)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FCoreStyle::Get().GetFontStyle(TEXT("NotificationList.FontBold")))
						.Text(LOCTEXT("RecordingTitleFormat", "Recording sequencer cache"))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(15.f,0,0,0))
					[
						SAssignNew(Throbber, SThrobber)
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0,0,0,5.0f))
				.HAlign(HAlign_Right)
				[
					SAssignNew(TextBlock, STextBlock)
					.Font(FCoreStyle::Get().GetFontStyle(TEXT("NotificationList.FontLight")))
					.Text(GetDetailText())
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(5.0f,0,0,0))
					.VAlign(VAlign_Center)
					[
						SAssignNew(Button, SButton)
						.Text(LOCTEXT("StopButton", "Stop"))
						.OnClicked(this, &SCacheTrackRecorderNotification::ButtonClicked)
					]
				]
			]
		];
	}

private:

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		bool bCloseNotification = false;
		bool bCloseImmediately = false;
		if (WeakRecorder.IsStale())
		{
			// Reset so we don't continually close the notification
			bCloseImmediately = true;
		}
		else if (UCacheTrackRecorder* Recorder = WeakRecorder.Get())
		{
			ECacheTrackRecorderState NewCacheTrackRecorderState = Recorder->GetState();

			if (NewCacheTrackRecorderState == ECacheTrackRecorderState::Starting || NewCacheTrackRecorderState == ECacheTrackRecorderState::Started)
			{
				// When counting down the text may change on tick
				TextBlock->SetText(GetDetailText());
			}

			if (NewCacheTrackRecorderState != CacheTrackRecorderState)
			{
				TextBlock->SetText(GetDetailText());

				if (NewCacheTrackRecorderState == ECacheTrackRecorderState::Stopped || NewCacheTrackRecorderState == ECacheTrackRecorderState::Cancelled)
				{
					Throbber->SetVisibility(EVisibility::Collapsed);
					Button->SetVisibility(EVisibility::Collapsed);

					bCloseNotification = true;
				}
			}

			CacheTrackRecorderState = NewCacheTrackRecorderState;
		}

		TSharedPtr<SNotificationItem> Owner = WeakOwningNotification.Pin();
		if ((bCloseNotification || bCloseImmediately) && Owner.IsValid())
		{
			if (bCloseImmediately)
			{
				Owner->SetFadeOutDuration(0.f);
				Owner->SetExpireDuration(0.f);
			}

			Owner->ExpireAndFadeout();

			// Remove our reference to the owner now that it's fading out
			Owner = nullptr;
		}
	}

	FText GetDetailText() const
	{
		if (UCacheTrackRecorder* Recorder = WeakRecorder.Get())
		{
			if (Recorder->GetState() == ECacheTrackRecorderState::Starting)
			{
				return LOCTEXT("CountdownText", "Starting to record...");
			}
			if (Recorder->GetState() == ECacheTrackRecorderState::Stopped)
			{
				return LOCTEXT("CompleteText", "Recording Complete");
			}
			if (Recorder->GetState() == ECacheTrackRecorderState::Cancelled)
			{
				return LOCTEXT("CancelledText", "Recording Cancelled");
			}

			static const FNumberFormattingOptions SecondFormattingOptions = FNumberFormattingOptions()
			.SetMinimumFractionalDigits(2)
			.SetMaximumFractionalDigits(2);
			FQualifiedFrameTime RecordTime = Recorder->GetRecordTime();
			return FText::Format(LOCTEXT("RecordingTimecodeText", "Recording... {0}s"), FText::AsNumber(RecordTime.AsSeconds(), &SecondFormattingOptions));
		}

		return LOCTEXT("RecordingText", "Recording...");
	}

	virtual TSharedRef<SWidget> AsWidget() override
	{
		return AsShared();
	}

	// Unused
	virtual void OnSetCompletionState(SNotificationItem::ECompletionState InState) override
	{
	}

	FReply ButtonClicked()
	{
		if (UCacheTrackRecorder* Recorder = WeakRecorder.Get())
		{
			Recorder->Stop();
		}
		return FReply::Handled();
	}

	TSharedPtr<SWidget> Button, Throbber, Hyperlink;
	TSharedPtr<STextBlock> TextBlock;

	ECacheTrackRecorderState CacheTrackRecorderState = ECacheTrackRecorderState::Starting;
	TWeakPtr<SNotificationItem> WeakOwningNotification;
	TWeakObjectPtr<UCacheTrackRecorder> WeakRecorder;
};


class FTickableCacheTrackRecorder : public FTickableGameObject
{
public:

	TWeakObjectPtr<UCacheTrackRecorder> WeakRecorder;

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FTickableCacheTrackRecorder, STATGROUP_Tickables);
	}

	//Make sure it always ticks, otherwise we can miss recording, in particularly when time code is always increasing throughout the system.
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }

	virtual bool IsTickableInEditor() const override
	{
		return true;
	}

	virtual UWorld* GetTickableGameObjectWorld() const override
	{
			UCacheTrackRecorder* Recorder = WeakRecorder.Get();
		return Recorder ? Recorder->GetWorld() : nullptr;
	}

	virtual void Tick(float DeltaTime) override
	{
		if (UCacheTrackRecorder* Recorder = WeakRecorder.Get())
		{
			Recorder->Tick(DeltaTime);
		}
	}
};

FTickableCacheTrackRecorder TickableCacheTrackRecorder;

// Static members of UCacheTrackRecorder
static TStrongObjectPtr<UCacheTrackRecorder>& GetCurrentRecorder()
{
	static TStrongObjectPtr<UCacheTrackRecorder> CurrentRecorder;
	return CurrentRecorder;
}

// Static functions for UCacheTrackRecorder
UCacheTrackRecorder* UCacheTrackRecorder::GetActiveRecorder()
{
	return GetCurrentRecorder().Get();
}

void UCacheTrackRecorder::RecordCacheTrack(IMovieSceneCachedTrack* Track, TSharedPtr<ISequencer> Sequencer, FCacheRecorderParameters Parameters)
{	
	if (Track)
	{
		TArray<IMovieSceneCachedTrack*> CacheTracks;
		CacheTracks.Add(Track);
		RecordCacheTracks(CacheTracks, Sequencer, Parameters);
	}
}

void UCacheTrackRecorder::RecordCacheTracks(const TArray<IMovieSceneCachedTrack*>& CacheTracks, TSharedPtr<ISequencer> Sequencer, FCacheRecorderParameters Parameters)
{
	ULevelSequence* RootSequence = Sequencer ? Cast<ULevelSequence>(Sequencer->GetRootMovieSceneSequence()) : nullptr;	
	ULevelSequence* FocusedSequence = Sequencer ? Cast<ULevelSequence>(Sequencer->GetFocusedMovieSceneSequence()) : nullptr;	
	if (RootSequence && FocusedSequence && CacheTracks.Num() > 0)
	{
		Parameters.StartFrame = FocusedSequence->GetMovieScene()->GetPlaybackRange().GetLowerBoundValue();
		Parameters.Project.bStartAtCurrentTimecode = false;

		// If not resetting the playhead, store the current time as the start frame for recording. 
		// This will ultimately be the start of the playback range and the recording will begin from that time.
		if (!Parameters.User.bResetPlayhead)
		{
			Parameters.StartFrame = Sequencer->GetLocalTime().Time.FrameNumber;
		}

		UCacheTrackRecorder* NewRecorder = NewObject<UCacheTrackRecorder>(GetTransientPackage(), NAME_None, RF_Transient);
		UTakeMetaData* TakeMetaData = FocusedSequence->FindOrAddMetaData<UTakeMetaData>();
		if (TakeMetaData->GetSlate().IsEmpty())
		{
			TakeMetaData->SetSlate(FocusedSequence->GetName());
		}

		FText ErrorText = LOCTEXT("UnknownError", "An unknown error occurred when trying to start recording");
		if (!NewRecorder->Initialize(RootSequence, CacheTracks, TakeMetaData->GetSlate(), Parameters, &ErrorText))
		{
			if (ensure(!ErrorText.IsEmpty()))
			{
				FNotificationInfo Info(ErrorText);
				Info.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
			}
		}
	}
}

void UCacheTrackRecorder::RecordSelectedTracks(TSharedPtr<ISequencer> Sequencer, FCacheRecorderParameters Parameters)
{
	TArray<UMovieSceneTrack*> SelectedTracks;
	Sequencer->GetSelectedTracks(SelectedTracks);

	// gather all selected cache tracks to record
	TArray<IMovieSceneCachedTrack*> CacheTracks;
	for (UMovieSceneTrack* SelectedTrack : SelectedTracks)
	{
		if (IMovieSceneCachedTrack* CacheTrack = Cast<IMovieSceneCachedTrack>(SelectedTrack))
		{
			if (CacheTrack->IsCacheRecordingAllowed())
			{
				CacheTracks.Add(CacheTrack);
			}
		}
	}

	RecordCacheTracks(CacheTracks, Sequencer, Parameters);
}

bool UCacheTrackRecorder::SetActiveRecorder(UCacheTrackRecorder* NewActiveRecorder)
{
	if (GetCurrentRecorder().IsValid())
	{
		return false;
	}

	GetCurrentRecorder().Reset(NewActiveRecorder);
	TickableCacheTrackRecorder.WeakRecorder = GetCurrentRecorder().Get();
	return true;
}

// Non-static api for UCacheTrackRecorder

bool UCacheTrackRecorder::Initialize(ULevelSequence* RootLevelSequence, const TArray<IMovieSceneCachedTrack*>& InCacheTracks, const FString& Slate, const FCacheRecorderParameters& InParameters, FText* OutError)
{
	FGCObjectScopeGuard GCGuard(this);

	if (GetActiveRecorder())
	{
		if (OutError)
		{
			*OutError = LOCTEXT("RecordingInProgressError", "A recording is currently in progress.");
		}
		return false;
	}

	if (Slate.IsEmpty())
	{
		if (OutError)
		{
			*OutError = LOCTEXT("NoSlateSpecifiedError", "No slate specified.");
		}
		return false;
	}

	if (InCacheTracks.Num() == 0)
	{
		*OutError = LOCTEXT("RecordingModeError", "No valid tracks selected for cache recording.");
		return false;
	}
	CacheTracks.Empty();
	for (IMovieSceneCachedTrack* Track : InCacheTracks)
	{
		CacheTracks.AddDefaulted_GetRef().Track = Track;
	}

	FCacheRecorderParameters FinalParameters = InParameters;
	WeakSequencer = TakesUtils::OpenSequencer(RootLevelSequence, OutError);
	if (!WeakSequencer.IsValid())
	{
		return false;
	}

	// The SequenceAsset is either LevelSequenceBase or the currently focused sequence
	SequenceAsset = Cast<ULevelSequence>(WeakSequencer.Pin()->GetFocusedMovieSceneSequence());
	SequenceAsset->MarkPackageDirty();

	// -----------------------------------------------------------
	// Anything after this point assumes successful initialization
	// -----------------------------------------------------------

	AddToRoot();

	Parameters = FinalParameters;
	State      = ECacheTrackRecorderState::Starting;

	// Perform any other parameter-configurable initialization. Must have a valid world at this point.
	InitializeFromParameters();

	// Figure out which world we're recording from
	DiscoverSourceWorld();

	// Open a recording notification
	if (ShouldShowNotifications())
	{
		TSharedRef<SCacheTrackRecorderNotification> Content = SNew(SCacheTrackRecorderNotification, this);

		FNotificationInfo Info(Content);
		Info.bFireAndForget = false;
		Info.ExpireDuration = 5.f;

		TSharedPtr<SNotificationItem> PendingNotification = FSlateNotificationManager::Get().AddNotification(Info);
		Content->SetOwner(PendingNotification);
	}

	ensure(SetActiveRecorder(this));

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		// If a start frame was specified, adjust the playback range before rewinding to the beginning of the playback range
		UMovieScene* MovieScene = SequenceAsset->GetMovieScene();
		CachedViewRange = MovieScene->GetEditorData().GetViewRange();
		MovieScene->SetPlaybackRange(TRange<FFrameNumber>(Parameters.StartFrame, MovieScene->GetPlaybackRange().GetUpperBoundValue()));

		// Center the view range around the current time about to be captured
		FAnimatedRange Range = Sequencer->GetViewRange();
		FTimecode CurrentTime = FApp::GetTimecode();
		FFrameRate FrameRate = MovieScene->GetDisplayRate();
		FFrameNumber ViewRangeStart = CurrentTime.ToFrameNumber(FrameRate);
		double ViewRangeStartSeconds = Parameters.Project.bStartAtCurrentTimecode ? FrameRate.AsSeconds(ViewRangeStart) : MovieScene->GetPlaybackRange().GetLowerBoundValue() / MovieScene->GetTickResolution();
		TRange<double> NewRange(ViewRangeStartSeconds - 0.5f, ViewRangeStartSeconds + (Range.GetUpperBoundValue() - Range.GetLowerBoundValue()) + 0.5f);
		Sequencer->SetViewRange(NewRange, EViewRangeInterpolation::Immediate);
		Sequencer->SetClampRange(TRange(Sequencer->GetViewRange()));
		if (Sequencer->GetPlaybackSpeed() <= 0)
		{
			Sequencer->SetPlaybackSpeed(1);
		}
		ESequencerLoopMode LoopMode = Sequencer->GetSequencerSettings()->GetLoopMode();
		if (LoopMode != SLM_NoLoop)
		{
			Sequencer->GetSequencerSettings()->SetLoopMode(SLM_NoLoop);
			OnStopCleanup.Add([LoopMode, this] { WeakSequencer.Pin()->GetSequencerSettings()->SetLoopMode(LoopMode); });
		}
	}

	return true;
}

void UCacheTrackRecorder::DiscoverSourceWorld()
{
	WeakWorld = TakesUtils::DiscoverSourceWorld();

	bool bPlayInGame = WeakWorld->WorldType == EWorldType::PIE || WeakWorld->WorldType == EWorldType::Game;
	// If recording via PIE, be sure to stop recording cleanly when PIE ends
	if ( bPlayInGame )
	{
		FEditorDelegates::EndPIE.AddUObject(this, &UCacheTrackRecorder::HandlePIE);
	}
	// If not recording via PIE, be sure to stop recording if PIE Starts
	if ( !bPlayInGame )
	{
		FEditorDelegates::BeginPIE.AddUObject(this, &UCacheTrackRecorder::HandlePIE);//reuse same function
	}
}

void UCacheTrackRecorder::InitializeFromParameters()
{
	// Set the end recording frame if enabled
	StopRecordingFrame = Parameters.User.bStopAtPlaybackEnd ? SequenceAsset->GetMovieScene()->GetPlaybackRange().GetUpperBoundValue() : TOptional<FFrameNumber>();

	// Apply immersive mode if the parameters demand it
	if (Parameters.User.bMaximizeViewport)
	{
		TSharedPtr<IAssetViewport> ActiveLevelViewport = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor").GetFirstActiveViewport();

		// If it's already immersive we just leave it alone
		if (ActiveLevelViewport.IsValid() && !ActiveLevelViewport->IsImmersive())
		{
			ActiveLevelViewport->MakeImmersive(true/*bWantImmersive*/, false/*bAllowAnimation*/);

			// Restore it when we're done
			auto RestoreImmersiveMode = [WeakViewport = TWeakPtr<IAssetViewport>(ActiveLevelViewport)]
			{
				if (TSharedPtr<IAssetViewport> CleaupViewport = WeakViewport.Pin())
				{
					CleaupViewport->MakeImmersive(false/*bWantImmersive*/, false/*bAllowAnimation*/);
				}
			};
			OnStopCleanup.Add(RestoreImmersiveMode);
		}
	}

	int32 MinScalability = -1;
	for (FCachedTrackSource& CacheTrack : CacheTracks)
	{
		MinScalability = FMath::Max(MinScalability, CacheTrack.Track->GetMinimumEngineScalabilitySetting());
	}
	if (MinScalability >= 0)
	{
		Scalability::FQualityLevels CurrentQualityLevels = Scalability::GetQualityLevels();
		Scalability::FQualityLevels NewQualityLevels = CurrentQualityLevels;
		NewQualityLevels.SetFromSingleQualityLevel(MinScalability);
		SetQualityLevels(NewQualityLevels, true);
		OnStopCleanup.Add([CurrentQualityLevels] { SetQualityLevels(CurrentQualityLevels, true); });
	}
}

bool UCacheTrackRecorder::ShouldShowNotifications()
{
	// -CacheTrackRecorderISHEADLESS in the command line can force headless behavior and disable the notifications.
	static const bool bCmdLineCacheTrackRecorderIsHeadless = FParse::Param(FCommandLine::Get(), TEXT("CacheTrackRecorderISHEADLESS"));

	return Parameters.Project.bShowNotifications
		&& !bCmdLineCacheTrackRecorderIsHeadless
		&& !FApp::IsUnattended()
		&& !GIsRunningUnattendedScript;
}

void UCacheTrackRecorder::BackupEditorTickState()
{
	SavedState.bBackedUp = true;
	SavedState.bUseFixedTimeStep = FApp::UseFixedTimeStep();
	SavedState.FixedDeltaTime = FApp::GetFixedDeltaTime();
}

void UCacheTrackRecorder::ModifyEditorTickState()
{
	ensure(SavedState.bBackedUp == false);
	if (SequenceAsset)
	{
		if (UMovieScene* MovieScene = SequenceAsset->GetMovieScene())
		{
			BackupEditorTickState();
			
			FFrameRate FrameRate = MovieScene->GetDisplayRate();
			double TickDelta = FrameRate.IsValid() ? FrameRate.AsInterval() : (1 / 30.0);
			
			// Force the engine into fixed timestep mode. There may be a global delay on the job that passes a fixed
			// number of frames, so we want those frames to always pass the same amount of time for determinism. 
			FApp::SetUseFixedTimeStep(true);
			FApp::SetFixedDeltaTime(TickDelta);
		}
	}
}

void UCacheTrackRecorder::RestoreEditorTickState()
{
	if (SavedState.bBackedUp)
	{
		SavedState.bBackedUp = false;
		FApp::SetUseFixedTimeStep(SavedState.bUseFixedTimeStep);
		FApp::SetFixedDeltaTime(SavedState.FixedDeltaTime);
	}
}

UWorld* UCacheTrackRecorder::GetWorld() const
{
	return WeakWorld.Get();
}

void UCacheTrackRecorder::Tick(float DeltaTime)
{
	if (State == ECacheTrackRecorderState::Starting)
	{
		NumberOfTicksAfterPre = 0;
		PreRecord();
	}
	else if (State == ECacheTrackRecorderState::PreRecord)
	{
		if (++NumberOfTicksAfterPre == 2) //seems we need 2 ticks to make sure things are settled
		{
			State = ECacheTrackRecorderState::TickingAfterPre;
		}
	}
	else if (State == ECacheTrackRecorderState::TickingAfterPre)
	{
		NumberOfTicksAfterPre = 0;
		Start();
		InternalTick(DeltaTime);
	}
	else if (State == ECacheTrackRecorderState::Started)
	{
		InternalTick(DeltaTime);
	}
}

FQualifiedFrameTime UCacheTrackRecorder::GetRecordTime() const
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		return Sequencer->GetLocalTime();
	}
	return TakesUtils::GetRecordTime(Sequencer, SequenceAsset, TimecodeAtStart, Parameters.Project.bStartAtCurrentTimecode);
}

void UCacheTrackRecorder::InternalTick(float DeltaTime)
{
	UE::MovieScene::FScopedSignedObjectModifyDefer FlushOnTick(true);

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	FQualifiedFrameTime RecordTime = GetRecordTime();

	CurrentFrameTime = RecordTime.Time;
	bool ShouldContinue = false;
	for (FCachedTrackSource& CacheSource : CacheTracks)
	{
		if (CacheSource.Recorder && CacheSource.Recorder->ShouldContinueRecording(RecordTime) && DeltaTime > 0.0f)
		{
			ShouldContinue = true;
			CacheSource.Recorder->RecordSample(RecordTime);
		}
	}
	if (ShouldContinue == false && DeltaTime > 0.0f)
	{
		StopRecordingFrame = CurrentFrameTime.FrameNumber;
	}
	
	if (Sequencer.IsValid())
	{
		FAnimatedRange Range = Sequencer->GetViewRange();
		if (UMovieScene* MovieScene = SequenceAsset->GetMovieScene())
		{
			FFrameRate FrameRate = MovieScene->GetTickResolution();
			double CurrentTimeSeconds = FrameRate.AsSeconds(CurrentFrameTime) + 0.5f;
			CurrentTimeSeconds = CurrentTimeSeconds > Range.GetUpperBoundValue() ? CurrentTimeSeconds : Range.GetUpperBoundValue();
			TRange<double> NewRange(Range.GetLowerBoundValue(), CurrentTimeSeconds);
			Sequencer->SetViewRange(NewRange, EViewRangeInterpolation::Immediate);
			Sequencer->SetClampRange(TRange(Sequencer->GetViewRange()));
		}
	}

	if ((StopRecordingFrame.IsSet() && CurrentFrameTime.FrameNumber >= StopRecordingFrame.GetValue()) || Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Stopped)
	{
		Stop();
	}
}

void UCacheTrackRecorder::PreRecord()
{
	State = ECacheTrackRecorderState::PreRecord;

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	for (FCachedTrackSource& CacheSource : CacheTracks)
	{
		static const FName MovieSceneSectionRecorderFactoryName("MovieSceneTrackRecorderFactory");
		TArray<IMovieSceneTrackRecorderFactory*> ModularFactories = IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneTrackRecorderFactory>(MovieSceneSectionRecorderFactoryName);
		for (IMovieSceneTrackRecorderFactory* Factory : ModularFactories)
		{
			if (UMovieSceneTrackRecorder* TrackRecorder = Factory->CreateTrackRecorderForCacheTrack(CacheSource.Track, SequenceAsset, Sequencer))
			{
				CacheSource.Recorder = TrackRecorder;
				break;
			}
		}
	}
	
	// Refresh sequencer in case the movie scene data has mutated (ie. existing object bindings removed because they will be recorded again)
	if (Sequencer.IsValid())
	{
		Sequencer->RefreshTree();
	}

	// Apply engine Time Dilation after the countdown, otherwise the countdown will be dilated as well!
	UWorld* RecordingWorld = GetWorld();
	check(RecordingWorld);
	if (AWorldSettings* WorldSettings = RecordingWorld->GetWorldSettings())
	{
		const float ExistingCinematicTimeDilation = WorldSettings->CinematicTimeDilation;
		const bool bInvalidTimeDilation = Parameters.User.EngineTimeDilation == 0.f;

		if (bInvalidTimeDilation)
		{
			UE_LOGF(LogTakesCore, Warning, "Time dilation cannot be 0. Ignoring time dilation for this recording.");
		}

		if (Parameters.User.EngineTimeDilation != ExistingCinematicTimeDilation && !bInvalidTimeDilation)
		{
			WorldSettings->CinematicTimeDilation = Parameters.User.EngineTimeDilation;

			// Restore it when we're done
			auto RestoreTimeDilation = [ExistingCinematicTimeDilation, WeakWorldSettings = MakeWeakObjectPtr(WorldSettings)]
			{
				if (AWorldSettings* CleaupWorldSettings = WeakWorldSettings.Get())
				{
					CleaupWorldSettings->CinematicTimeDilation = ExistingCinematicTimeDilation;
				}
			};
			OnStopCleanup.Add(RestoreTimeDilation);
		}
	}

	// we set this global variable to prevent things like auto-save to trigger during a recording
	GIsSlowTask = true;
	OnStopCleanup.Add([]() { GIsSlowTask = false; });

	// prevent dropped frames from slate throttling
	FSlateThrottleManager::Get().DisableThrottle(true);
	OnStopCleanup.Add([]() { FSlateThrottleManager::Get().DisableThrottle(false); });

	UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();
	if (EditorPerformanceSettings && EditorPerformanceSettings->bThrottleCPUWhenNotForeground)
	{
		EditorPerformanceSettings->bThrottleCPUWhenNotForeground = false;
		EditorPerformanceSettings->PostEditChange();
		OnStopCleanup.Add([]()
		{
			UEditorPerformanceSettings* PerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();
			PerformanceSettings->bThrottleCPUWhenNotForeground = true;
			PerformanceSettings->PostEditChange();
		});
	}
	

	if (Parameters.Project.bCacheTrackRecorderControlsClockTime)
	{
		ModifyEditorTickState();

		auto RestoreEditorTick = [this]()
		{
			RestoreEditorTickState();
		};
		OnStopCleanup.Add(RestoreEditorTick);
	}

	if (UMovieScene* MovieScene = SequenceAsset->GetMovieScene())
	{
		CachedPlaybackRange = MovieScene->GetPlaybackRange();
		CachedClockSource = MovieScene->GetClockSource();
		MovieScene->SetClockSource(Parameters.Project.bCacheTrackRecorderControlsClockTime ? EUpdateClockSource::Tick : Parameters.Project.RecordingClockSource);
		if (Sequencer.IsValid())
		{
			Sequencer->ResetTimeController();
		}

		FFrameRate FrameRate = MovieScene->GetDisplayRate();
		FFrameRate TickResolution = MovieScene->GetTickResolution();
		FFrameNumber PlaybackStartFrame = Parameters.Project.bStartAtCurrentTimecode ? FFrameRate::TransformTime(FFrameTime(FApp::GetTimecode().ToFrameNumber(FrameRate)), FrameRate, TickResolution).FloorToFrame() : MovieScene->GetPlaybackRange().GetLowerBoundValue();

		// Transform all the sections to start the playback start frame
		FFrameNumber DeltaFrame = PlaybackStartFrame - MovieScene->GetPlaybackRange().GetLowerBoundValue();
		if (DeltaFrame != 0)
		{
			for (UMovieSceneSection* Section : MovieScene->GetAllSections())
			{
				Section->MoveSection(DeltaFrame);
			}
		}

		if (StopRecordingFrame.IsSet())
		{
			StopRecordingFrame = StopRecordingFrame.GetValue() + DeltaFrame;
		}

		// Set infinite playback range when starting recording. Playback range will be clamped to the bounds of the sections at the completion of the recording
		MovieScene->SetPlaybackRange(TRange<FFrameNumber>(PlaybackStartFrame, TNumericLimits<int32>::Max() - 1), false);
		if (Sequencer.IsValid())
		{
			Sequencer->SetLocalTimeDirectly(PlaybackStartFrame);
			Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Paused);
		}
	}
}

void UCacheTrackRecorder::Start()
{
	FTimecode Timecode = FApp::GetTimecode();

	State = ECacheTrackRecorderState::Started;

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	CurrentFrameTime = FFrameTime(0);
	TimecodeAtStart = Timecode;

	// Discard any entity tokens we have so that restore state does not take effect when we delete any sections that recording will be replacing.
	if (Sequencer.IsValid())
	{
		Sequencer->PreAnimatedState.DiscardEntityTokens();
	}

	Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	FQualifiedFrameTime RecordTime = GetRecordTime();
	FFrameNumber RecordStartFrame = RecordTime.Time.FloorToFrame();
	FFrameRate TargetLevelSequenceTickResolution = SequenceAsset->MovieScene->GetTickResolution();
	FFrameRate TargetLevelSequenceDisplayRate = SequenceAsset->MovieScene->GetDisplayRate();
	FTimecode CurrentTimecode = FTimecode::FromFrameNumber(FFrameRate::TransformTime(RecordTime.Time, TargetLevelSequenceTickResolution, TargetLevelSequenceDisplayRate).FloorToFrame(), TargetLevelSequenceDisplayRate);
	for (FCachedTrackSource& CacheSource : CacheTracks)
	{
		if (CacheSource.Recorder)
		{
			CacheSource.Recorder->SetSectionStartTimecode(CurrentTimecode, RecordTime.Time.FloorToFrame());

			if (UMovieSceneTrack* SceneTrack = Cast<UMovieSceneTrack>(CacheSource.Track))
			{
				for (UMovieSceneSection* SubSection : SceneTrack->GetAllSections())
				{
					SubSection->TimecodeSource = FMovieSceneTimecodeSource(Timecode);

					// Ensure we're expanded to at least the next frame so that we don't set the start past the end
					// when we set the first frame.
					SubSection->ExpandToFrame(RecordStartFrame + FFrameNumber(1));
					SubSection->SetStartFrame(TRangeBound<FFrameNumber>::Inclusive(RecordStartFrame));
				}
			}
		}
	}
	
	if (!ShouldShowNotifications())
	{
		// Log in lieu of the notification widget
		UE_LOGF(LogTakesCore, Log, "Started recording");
	}
}

void UCacheTrackRecorder::Stop()
{
	constexpr bool bCancelled = false;
	StopInternal(bCancelled);
}

void UCacheTrackRecorder::Cancel()
{
	constexpr bool bCancelled = true;
	StopInternal(bCancelled);
}

void UCacheTrackRecorder::StopInternal(const bool bCancelled)
{
	static bool bStoppedRecording = false;

	if (bStoppedRecording)
	{
		return;
	}

	double StartTime = FPlatformTime::Seconds();

	TGuardValue<bool> ReentrantGuard(bStoppedRecording, true);

	FEditorDelegates::EndPIE.RemoveAll(this);
	FEditorDelegates::BeginPIE.RemoveAll(this);
	
	const bool bDidEverStartRecording = State == ECacheTrackRecorderState::Started;
	const bool bRecordingFinished = !bCancelled && bDidEverStartRecording;
	State = bRecordingFinished ? ECacheTrackRecorderState::Stopped : ECacheTrackRecorderState::Cancelled;

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Stopped);
	}

	UMovieScene* MovieScene = SequenceAsset->GetMovieScene();
	if (MovieScene)
	{
		MovieScene->SetClockSource(CachedClockSource);

		if (Sequencer.IsValid())
		{
			Sequencer->ResetTimeController();
		}
	}

	if (bDidEverStartRecording)
	{
		if (!ShouldShowNotifications())
		{
			// Log in lieu of the notification widget
			if (bRecordingFinished)
			{
				UE_LOGF(LogTakesCore, Log, "Stopped recording");
			}
			else
			{
				UE_LOGF(LogTakesCore, Log, "Recording cancelled");
			}
		}

		if (MovieScene)
		{
			TRange<FFrameNumber> Range = MovieScene->GetPlaybackRange();

			//Set Range to what we recorded instead of that large number, this let's us reliably set camera cut times.
			FFrameNumber PlaybackEndFrame = CurrentFrameTime.FrameNumber;

			if (StopRecordingFrame.IsSet())
			{
				PlaybackEndFrame = StopRecordingFrame.GetValue();
			}
			
			MovieScene->SetPlaybackRange(TRange<FFrameNumber>(Range.GetLowerBoundValue(), PlaybackEndFrame));

			if (Sequencer)
			{
				Sequencer->ResetTimeController();
			}
		}

		// stop all recordings
		for (FCachedTrackSource& CacheSource : CacheTracks)
		{
			if (CacheSource.Recorder)
			{
				CacheSource.Recorder->StopRecording();
			}
		}

		// finalize tracks
		for (FCachedTrackSource& CacheSource : CacheTracks)
		{
			if (CacheSource.Recorder)
			{
				CacheSource.Recorder->FinalizeTrack();
			}
		}

		// Restore the playback/view range to what it was before recording.
		if (MovieScene)
		{
			MovieScene->SetPlaybackRange(CachedPlaybackRange);
			Sequencer->SetViewRange(CachedViewRange, EViewRangeInterpolation::Immediate);
		}

		if (bRecordingFinished)
		{
			UTakeMetaData* AssetMetaData = SequenceAsset->FindMetaData<UTakeMetaData>();

			if (GEditor && GEditor->GetEditorWorldContext().World() && ensure(AssetMetaData))
			{
				AssetMetaData->SetLevelOrigin(GEditor->GetEditorWorldContext().World()->PersistentLevel);
			}

			// Rebuild sequencer because subsequences could have been added or bindings removed
			if (Sequencer)
			{
				Sequencer->RefreshTree();
			}
		}
	}

	// Perform any other cleanup that has been defined for this recording
	for (const TFunction<void()>& Cleanup : OnStopCleanup)
	{
		Cleanup();
	}
	OnStopCleanup.Reset();

	// reset the current recorder and stop us from being ticked
	if (GetCurrentRecorder().Get() == this)
	{
		GetCurrentRecorder().Reset();
		TickableCacheTrackRecorder.WeakRecorder = nullptr;
	}

	if (SequenceAsset)
	{
		UPackage* const Package = SequenceAsset->GetOutermost();
		FString const PackageName = Package->GetName();

		double ElapsedTime = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogTakesCore, Log, "Finished processing %ls in %0.2f seconds", *PackageName, ElapsedTime);
	}

	RemoveFromRoot();
}

void UCacheTrackRecorder::HandlePIE(bool bIsSimulating)
{
	ULevelSequence* FinishedAsset = SequenceAsset;

	if (ShouldShowNotifications())
	{
		TSharedRef<SCacheTrackRecorderNotification> Content = SNew(SCacheTrackRecorderNotification, this);

		FNotificationInfo Info(Content);
		Info.ExpireDuration = 5.f;

		TSharedPtr<SNotificationItem> PendingNotification = FSlateNotificationManager::Get().AddNotification(Info);
		PendingNotification->SetCompletionState(SNotificationItem::CS_Success);
	}
	
	Stop();

	if (FinishedAsset)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(FinishedAsset);
	}
}

#undef LOCTEXT_NAMESPACE

