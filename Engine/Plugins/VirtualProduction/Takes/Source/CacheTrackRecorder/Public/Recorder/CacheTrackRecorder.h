// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "ISequencer.h"
#include "CacheTrackRecorder.generated.h"

#define UE_API CACHETRACKRECORDER_API

class ULevelSequence;
class UMovieSceneTrackRecorder;
class IMovieSceneCachedTrack;

USTRUCT()
struct FCacheRecorderUserParameters
{
	GENERATED_BODY()

	UE_API FCacheRecorderUserParameters();

	/** Whether to maximize the viewport (enter Immersive Mode) when recording */
	UPROPERTY(config, EditAnywhere, Category="User Settings")
	bool bMaximizeViewport;

	/** The engine time dilation to apply during the recording */
	UPROPERTY(config, EditAnywhere, Category="User Settings", meta=(Units=Multiplier, ClampMin="0.00001", UIMin="0.00001"))
	float EngineTimeDilation;

	/** Reset playhead to beginning of the playback range when starting recording */
	UPROPERTY(config, EditAnywhere, Category = "User Settings")
	bool bResetPlayhead;

	/** Automatically stop recording when reaching the end of the playback range */
	UPROPERTY(config, EditAnywhere, Category = "User Settings")
	bool bStopAtPlaybackEnd;
};

USTRUCT()
struct FCacheRecorderProjectParameters
{
	GENERATED_BODY()

	UE_API FCacheRecorderProjectParameters();
	
	/**
	 * The default name to use for the Slate information
	 */
	UPROPERTY(config, EditAnywhere, Category = "Cache Recorder")
	FString DefaultSlate;

	/**
	 * If true then take recorder will control the sequencer timing when recording with a fixed editor time step. The delta time is derived by the sequence's target frame rate. 
	 * This is useful when recording cache data where frame accuracy is important (e.g. Niagara systems), but should be set to false when dealing with data from external sources (e.g. LiveLink).
	 */
	UPROPERTY(config, EditAnywhere, Category = "Cache Recorder")
	bool bCacheTrackRecorderControlsClockTime = true;

	/**
	 * The clock source to use when recording
	 */
	UPROPERTY(config, EditAnywhere, Category = "Cache Recorder", meta=(EditCondition="!bCacheTrackRecorderControlsClockTime"))
	EUpdateClockSource RecordingClockSource;

	/**
	 * If enabled, track sections will start at the current timecode. Otherwise, 0.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Cache Recorder")
	bool bStartAtCurrentTimecode;

	/**
	 * If enabled, timecode will be recorded into each actor track
	 */
	UPROPERTY(config, EditAnywhere, Category = "Cache Recorder")
	bool bRecordTimecode;

	/** Whether to show notification windows or not when recording */
	UPROPERTY(config, EditAnywhere, Category = "Cache Recorder")
	bool bShowNotifications;
};

/**
 * Structure housing all configurable parameters for a take recorder instance
 */
USTRUCT()
struct FCacheRecorderParameters
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Cache Recorder")
	FCacheRecorderUserParameters User;

	UPROPERTY(EditAnywhere, Category="Cache Recorder")
	FCacheRecorderProjectParameters Project;

	UPROPERTY(EditAnywhere, Category = "Cache Recorder")
	FFrameNumber StartFrame;
};

UENUM(BlueprintType)
enum class ECacheTrackRecorderState : uint8
{
	Starting,
	PreRecord,
	TickingAfterPre,
	Started,
	Stopped,
	Cancelled,
};

USTRUCT()
struct FCachedTrackSource
{
	GENERATED_USTRUCT_BODY()
	
	IMovieSceneCachedTrack* Track = nullptr;

	UPROPERTY()
	TObjectPtr<UMovieSceneTrackRecorder> Recorder;
};

UCLASS(MinimalAPI, BlueprintType)
class UCacheTrackRecorder : public UObject
{
public:

	GENERATED_BODY()

	/**
	 * Retrieve the currently active take recorder instance
	 */
	static UE_API UCacheTrackRecorder* GetActiveRecorder();

	/**
	 * Utility method to record a single cache track
	 */
	static UE_API void RecordCacheTrack(IMovieSceneCachedTrack* Track, TSharedPtr<ISequencer> Sequencer, FCacheRecorderParameters Parameters = FCacheRecorderParameters());
	
	/**
	 * Utility method to record several cache tracks at once
	 */
	static UE_API void RecordCacheTracks(const TArray<IMovieSceneCachedTrack*>& CacheTracks, TSharedPtr<ISequencer> Sequencer, FCacheRecorderParameters Parameters = FCacheRecorderParameters());

	/**
	 * Utility method to record all the cache tracks selected in sequencer
	 */
	static UE_API void RecordSelectedTracks(TSharedPtr<ISequencer> Sequencer, FCacheRecorderParameters Parameters = FCacheRecorderParameters());

	/**
	 * Access the sequence asset that this recorder is recording into
	 */
	UFUNCTION(BlueprintCallable, Category="Cache Recorder")
	ULevelSequence* GetSequence() const
	{
		return SequenceAsset;
	}

	/**
	 * Get the current state of this recorder
	 */
	UFUNCTION(BlueprintCallable, Category="Cache Recorder")
	ECacheTrackRecorderState GetState() const
	{
		return State;
	}

	/**
	 * Initialize a new recording with the specified parameters. Fails if another recording is currently in progress.
	 *
	 */
	UE_API bool Initialize(ULevelSequence* RootLevelSequence, const TArray<IMovieSceneCachedTrack*>& CacheTracks, const FString& Slate, const FCacheRecorderParameters& InParameters, FText* OutError = nullptr);

	/**
	 * Called to stop the recording
	 */
	UE_API void Stop();

	/**
	 * Called to cancel the recording
	 */
	UE_API void Cancel();

	/* The time at which to record. Taken from the Sequencer global time, otherwise based on timecode */
	UE_API FQualifiedFrameTime GetRecordTime() const;

private:

	/**
	 * Called after the countdown to PreRecord
	 */
	UE_API void PreRecord();

	/**
	 * Called after PreRecord To Start
	 */
	UE_API void Start();

	/*
	 * Stop or cancel
	 */
	UE_API void StopInternal(const bool bCancelled);

	/**
	 * Ticked by a tickable game object to performe any necessary time-sliced logic
	 */
	UE_API void Tick(float DeltaTime);

	/**
	 * Called if we're currently recording a PIE world that has been shut down or if we start PIE in a non-PIE world. Bound in Initialize, and unbound in Stop.
	 */
	UE_API void HandlePIE(bool bIsSimulating);

	/**
	 * Discovers the source world to record from, and initializes it for recording
	 */
	UE_API void DiscoverSourceWorld();

	/**
	 * Called to perform any initialization based on the user-provided parameters
	 */
	UE_API void InitializeFromParameters();

	/**
     * Returns true if notification widgets should be shown when recording.
	 * It takes into account CacheTrackRecorder project settings, the command line, and global unattended settings.
     */
	UE_API bool ShouldShowNotifications();

	/* Called before setting the editor into a fixed time step mode */
	UE_API void BackupEditorTickState();

	/* Sets the editor up to tick in a fixed time step */
	UE_API void ModifyEditorTickState();

	/* Restores the editor tick state if necessary */
	UE_API void RestoreEditorTickState();

	/** Called by Tick and Start to make sure we record at start */
	UE_API void InternalTick(float DeltaTime);

	UE_API virtual UWorld* GetWorld() const override;

	/** The state of this recorder instance */
	ECacheTrackRecorderState State;

	/** FFrameTime in MovieScene Resolution we are at*/
	FFrameTime CurrentFrameTime;

	/** Timecode at the start of recording */
	FTimecode TimecodeAtStart;

	/** Optional frame to stop recording at*/
	TOptional<FFrameNumber> StopRecordingFrame;

	/** The asset that we should output recorded data into */
	UPROPERTY(transient)
	TObjectPtr<ULevelSequence> SequenceAsset;

	/** The world that we are recording within */
	UPROPERTY(transient)
	TWeakObjectPtr<UWorld> WeakWorld;

	/** Parameters for the recorder - marked up as a uproperty to support reference collection */
	UPROPERTY()
	FCacheRecorderParameters Parameters;

	UPROPERTY()
	TArray<FCachedTrackSource> CacheTracks;

	/** Anonymous array of cleanup functions to perform when a recording has finished */
	TArray<TFunction<void()>> OnStopCleanup;

	/** Sequencer ptr that controls playback of the desination asset during the recording */
	TWeakPtr<ISequencer> WeakSequencer;

	/** Due a few ticks after the pre so we are set up with asset creation */
	int32 NumberOfTicksAfterPre;
	
	TRange<double> CachedViewRange;

	friend class FTickableCacheTrackRecorder;

	/**
	 * Set the currently active take recorder instance
	 */
	static UE_API bool SetActiveRecorder(UCacheTrackRecorder* NewActiveRecorder);

	EUpdateClockSource CachedClockSource;
	TRange<FFrameNumber> CachedPlaybackRange;

	struct FSavedState
	{
		bool bBackedUp = false;

		bool bUseFixedTimeStep = false;
		double FixedDeltaTime = 1.0;
	};

	FSavedState SavedState;
};

#undef UE_API
