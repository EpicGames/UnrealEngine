// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hitching/FrameHitchSceneDecoration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FrameHitchSceneDecoration)

TOptional<UE::TakeMovieScene::FFrameHitchData> UFrameHitchSceneDecoration::Evaluate(const FFrameTime& InFrameTime) const
{
	const TOptional<FTimecode> Target = TargetTimecode.Evaluate(InFrameTime);
	const TOptional<FTimecode> Actual = ActualTimecode.Evaluate(InFrameTime);
	return Target && Actual ? UE::TakeMovieScene::FFrameHitchData{ *Target, *Actual } : TOptional<UE::TakeMovieScene::FFrameHitchData>();
}
