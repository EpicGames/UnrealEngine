// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hitching/TimecodeHitchChannels.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TimecodeHitchChannels)

void FTakeMovieSceneHitchTimecodeCurves::Set(
	TArray<FFrameNumber> Times, TArray<int32> Hours, TArray<int32> Minutes, TArray<int32> Seconds, TArray<int32> Frames
	)
{
	const int32 Num = Times.Num();
	if (ensure(Hours.Num() == Num && Minutes.Num() == Num && Seconds.Num() == Num && Frames.Num() == Num))
	{
		HoursCurve.Set(Times, MoveTemp(Hours));
		MinutesCurve.Set(Times, MoveTemp(Minutes));
		SecondsCurve.Set(Times, MoveTemp(Seconds));
		FramesCurve.Set(MoveTemp(Times), MoveTemp(Frames));
	}
}

TOptional<FTimecode> FTakeMovieSceneHitchTimecodeCurves::Evaluate(const FFrameTime& InFrameTime) const
{
	int32 Hours = 0;
	int32 Minutes = 0;
	int32 Seconds = 0;
	int32 Frames = 0;

	if (!HoursCurve.Evaluate(InFrameTime, Hours)
		|| !MinutesCurve.Evaluate(InFrameTime, Minutes)
		|| !SecondsCurve.Evaluate(InFrameTime, Seconds)
		|| !FramesCurve.Evaluate(InFrameTime, Frames))
	{
		return {};
	}

	return FTimecode(Hours, Minutes, Seconds, Frames, false, false);
}

TConstArrayView<FFrameNumber> FTakeMovieSceneHitchTimecodeCurves::GetFrameTimes() const
{
	return HoursCurve.GetTimes();
}
