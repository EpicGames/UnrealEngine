// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTakeSection.h"
#include "MovieSceneTakeTrack.h"
#include "MovieSceneTakeSettings.h"
#include "Channels/MovieSceneChannelProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTakeSection)

#define LOCTEXT_NAMESPACE "MovieSceneTakeSection"

#if WITH_EDITOR

struct FTakeSectionEditorData
{
	FTakeSectionEditorData()
	{
		MetaData[0].SortOrder = 0;
		MetaData[0].bCanCollapseToTrack = false;

		MetaData[1].SortOrder = 1;
		MetaData[1].bCanCollapseToTrack = false;

		MetaData[2].SortOrder = 2;
		MetaData[2].bCanCollapseToTrack = false;

		MetaData[3].SortOrder = 3;
		MetaData[3].bCanCollapseToTrack = false;

		MetaData[4].SortOrder = 4;
		MetaData[4].bCanCollapseToTrack = false;

		MetaData[5].SortOrder = 5;
		MetaData[5].bCanCollapseToTrack = false;

		MetaData[6].SortOrder = 6;
		MetaData[6].bCanCollapseToTrack = false;
	}

	FMovieSceneChannelMetaData      MetaData[7];
	TMovieSceneExternalValue<int32> ExternalValues[4];
	TMovieSceneExternalValue<float> ExternalFloatValues[2];
	TMovieSceneExternalValue<FString> ExternalStringValues[1];
};

#endif	// WITH_EDITOR

UMovieSceneTakeSection::UMovieSceneTakeSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	bSupportsInfiniteRange = true;

	ReconstructChannelProxy();
}

#if WITH_EDITORONLY_DATA

void UMovieSceneTakeSection::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		ReconstructChannelProxy();
	}
}

#endif

void UMovieSceneTakeSection::PostEditImport()
{
	Super::PostEditImport();

	ReconstructChannelProxy();
}

TOptional<UMovieSceneTakeSection::FSectionData> UMovieSceneTakeSection::Evaluate(FFrameTime InTime) const
{
	int32 Hours = 0;
	int32 Minutes = 0;
	int32 Seconds = 0;
	int32 Frames = 0;
	float Subframe = 0;;

	FSectionData SectionData;

	if (!HoursCurve.Evaluate(InTime, Hours)
		|| !MinutesCurve.Evaluate(InTime, Minutes)
		|| !SecondsCurve.Evaluate(InTime, Seconds)
		|| !FramesCurve.Evaluate(InTime, Frames)
		|| !SubFramesCurve.Evaluate(InTime, Subframe)
		|| !RateCurve.Evaluate(InTime, SectionData.Rate))
	{
		return {};
	}

	const FString* SlateString = Slate.Evaluate(InTime);
	bool bIsDropFrame = FTimecode::IsDropFormatTimecodeSupported(SectionData.Rate) && FTimecode::UseDropFormatTimecodeByDefaultWhenSupported();

	SectionData.Timecode = FTimecode(Hours, Minutes, Seconds, Frames, Subframe, bIsDropFrame);
	SectionData.Slate = SlateString ? *SlateString : TEXT("");

	return SectionData;
}

void UMovieSceneTakeSection::ReconstructChannelProxy()
{
	LLM_SCOPE_BYNAME(TEXT("Takes/MovieSceneTakeSection"))

	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR

	static FTakeSectionEditorData EditorData;

	FString HoursName = GetDefault<UMovieSceneTakeSettings>()->HoursName;
	FString MinutesName = GetDefault<UMovieSceneTakeSettings>()->MinutesName;
	FString SecondsName = GetDefault<UMovieSceneTakeSettings>()->SecondsName;
	FString FramesName = GetDefault<UMovieSceneTakeSettings>()->FramesName;
	FString SubFramesName = GetDefault<UMovieSceneTakeSettings>()->SubFramesName;
	FString RateName = GetDefault<UMovieSceneTakeSettings>()->RateName;
	FString SlateName = GetDefault<UMovieSceneTakeSettings>()->SlateName;

	EditorData.MetaData[0].SetIdentifiers(FName(*HoursName), FText::FromString(HoursName));
	EditorData.MetaData[1].SetIdentifiers(FName(*MinutesName), FText::FromString(MinutesName));
	EditorData.MetaData[2].SetIdentifiers(FName(*SecondsName), FText::FromString(SecondsName));
	EditorData.MetaData[3].SetIdentifiers(FName(*FramesName), FText::FromString(FramesName));
	EditorData.MetaData[4].SetIdentifiers(FName(*SubFramesName), FText::FromString(SubFramesName));
	EditorData.MetaData[5].SetIdentifiers(FName(*RateName), FText::FromString(RateName));
	EditorData.MetaData[6].SetIdentifiers(FName(*SlateName), FText::FromString(SlateName));

	Channels.Add(HoursCurve,     EditorData.MetaData[0], EditorData.ExternalValues[0]);
	Channels.Add(MinutesCurve,   EditorData.MetaData[1], EditorData.ExternalValues[1]);
	Channels.Add(SecondsCurve,   EditorData.MetaData[2], EditorData.ExternalValues[2]);
	Channels.Add(FramesCurve,    EditorData.MetaData[3], EditorData.ExternalValues[3]);
	Channels.Add(SubFramesCurve, EditorData.MetaData[4], EditorData.ExternalFloatValues[0]);
	Channels.Add(RateCurve,      EditorData.MetaData[5], EditorData.ExternalFloatValues[1]);
	Channels.Add(Slate,          EditorData.MetaData[6], EditorData.ExternalStringValues[0]);

#else

	Channels.Add(HoursCurve);
	Channels.Add(MinutesCurve);
	Channels.Add(SecondsCurve);
	Channels.Add(FramesCurve);
	Channels.Add(SubFramesCurve);
	Channels.Add(RateCurve);
	Channels.Add(Slate);

#endif
	
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

#undef LOCTEXT_NAMESPACE
