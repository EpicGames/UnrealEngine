// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTakeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTakeSettings)

#define LOCTEXT_NAMESPACE "MovieSceneTakeSettings"

UMovieSceneTakeSettings::UMovieSceneTakeSettings()
	: HoursName(TEXT("TCHour"))
	, MinutesName(TEXT("TCMinute"))
	, SecondsName(TEXT("TCSecond"))
	, FramesName(TEXT("TCFrame"))
	, SubFramesName(TEXT("TCSubframe"))
	, RateName(TEXT("TCRate"))
	, SlateName(TEXT("TCSlate"))
{
}

#if WITH_EDITOR
void UMovieSceneTakeSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		// Dragging spinboxes causes this to be called every frame so we wait until they've finished dragging before saving.
		SaveConfig();
	}
}
#endif

#undef LOCTEXT_NAMESPACE
