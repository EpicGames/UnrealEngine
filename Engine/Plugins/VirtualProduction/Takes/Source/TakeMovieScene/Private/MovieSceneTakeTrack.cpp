// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTakeTrack.h"
#include "MovieSceneTakeSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTakeTrack)

#define LOCTEXT_NAMESPACE "MovieSceneTakeTrack"


UMovieSceneTakeTrack::UMovieSceneTakeTrack(const FObjectInitializer& Init)
	: Super(Init)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(62, 72, 82, 150);
#endif
}

void UMovieSceneTakeTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}

bool UMovieSceneTakeTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneTakeSection::StaticClass();
}

UMovieSceneSection* UMovieSceneTakeTrack::CreateNewSection()
{
	return NewObject<UMovieSceneTakeSection>(this, NAME_None, RF_Transactional);
}

const TArray<UMovieSceneSection*>& UMovieSceneTakeTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneTakeTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

bool UMovieSceneTakeTrack::IsEmpty() const
{
	return (Sections.Num() == 0);
}

void UMovieSceneTakeTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

void UMovieSceneTakeTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneTakeTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneTakeTrack::GetDefaultDisplayName() const
{ 
	return LOCTEXT("TrackName", "Take"); 
}

#endif

#undef LOCTEXT_NAMESPACE



