// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeMovieSceneModule.h"

IMPLEMENT_MODULE(FTakeMovieSceneModule, TakeMovieScene)

void FTakeMovieSceneModule::StartupModule()
{
}

void FTakeMovieSceneModule::ShutdownModule()
{
}
