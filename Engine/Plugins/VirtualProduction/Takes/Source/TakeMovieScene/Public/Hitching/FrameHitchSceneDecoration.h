// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FrameHitchData.h"
#include "TimecodeHitchChannels.h"
#include "UObject/Object.h"
#include "FrameHitchSceneDecoration.generated.h"

/**
 * This decoration contain stores a target and actual FTimecode for each frame for the purposes of detecting whether the engine was running behind,
 * i.e. was hitching, on a previous frame during take recording.
 *
 * This data can be used to visualize hitches in the Sequencer UI.
 */
UCLASS()
class UFrameHitchSceneDecoration : public UObject
{
	GENERATED_BODY()
public:

	/** Target timecode that the frame was supposed to have */
	UPROPERTY()
	FTakeMovieSceneHitchTimecodeCurves TargetTimecode;
	/** Actual timecode that was recorded */
	UPROPERTY()
	FTakeMovieSceneHitchTimecodeCurves ActualTimecode;

	/** The frame rate the timecode provider was set up with. Determines the number of frames per second in TargetTimecode & ActualTimecode. */
	UPROPERTY()
	FFrameRate TimecodeProviderFrameRate;
	
	/**
	 * The frame rate Take Recorder was set up with. Users may (intentionally) set up the provider and sequencer to record at different rates.
	 * This information is important for analyzing hitches later as it affects the timecode we should expect each frame.
	 */
	UPROPERTY()
	FFrameRate RecordFrameRate;
	
	/** Evaluates the curve data at the specified time. */
	TAKEMOVIESCENE_API TOptional<UE::TakeMovieScene::FFrameHitchData> Evaluate(const FFrameTime& InFrameTime) const;
};
