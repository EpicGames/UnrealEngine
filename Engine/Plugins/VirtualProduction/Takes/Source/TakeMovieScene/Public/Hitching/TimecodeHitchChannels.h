// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneIntegerChannel.h"
#include "Misc/Optional.h"
#include "Misc/Timecode.h"
#include "TimecodeHitchChannels.generated.h"

USTRUCT()
struct FTakeMovieSceneHitchTimecodeCurves
{
	GENERATED_BODY()

	/** Hours curve data  */
	UPROPERTY()
	FMovieSceneIntegerChannel HoursCurve;
	/** Minutes curve data */
	UPROPERTY()
	FMovieSceneIntegerChannel MinutesCurve;
	/** Seconds curve data */
	UPROPERTY()
	FMovieSceneIntegerChannel SecondsCurve;
	/** Frames curve data */
	UPROPERTY()
	FMovieSceneIntegerChannel FramesCurve;

	/** Sets the curve content from the given arrays - which must be equal size. */
	TAKEMOVIESCENE_API void Set(TArray<FFrameNumber> Times, TArray<int32> Hours, TArray<int32> Minutes, TArray<int32> Seconds, TArray<int32> Frames);

	/** @return The timecode at the frame time. */
	TAKEMOVIESCENE_API TOptional<FTimecode> Evaluate(const FFrameTime& InFrameTime) const;

	/** @return Gets all the frame times at which we recorded. */
	TAKEMOVIESCENE_API TConstArrayView<FFrameNumber> GetFrameTimes() const;
};