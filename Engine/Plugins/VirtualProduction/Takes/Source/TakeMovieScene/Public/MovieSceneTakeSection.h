// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneStringChannel.h"
#include "MovieSceneTakeSection.generated.h"

/**
 * A section in a Take track
 */
UCLASS(MinimalAPI)
class UMovieSceneTakeSection 
	: public UMovieSceneSection
{
	GENERATED_UCLASS_BODY()

public:

	//~ UObject interface
	virtual void PostEditImport() override;

protected:

#if WITH_EDITORONLY_DATA
	/** Overloaded serializer to ensure that the channel proxy is updated correctly on load and duplicate */
	virtual void Serialize(FArchive& Ar) override;
#endif

private:

	void ReconstructChannelProxy();

public:

	struct FSectionData
	{
		FTimecode Timecode;
		FString Slate;
		float Rate = 24.0f;
	};

	/** Evaluate the curve data at the specified time and form a timecode / rate and slate values. If any curves failed to evaluate then the optional will be unset. */
	TAKEMOVIESCENE_API TOptional<FSectionData> Evaluate(FFrameTime InTime) const;

	/** Hours curve data */
	UPROPERTY()
	FMovieSceneIntegerChannel HoursCurve;

	/** Minutes curve data */
	UPROPERTY()
	FMovieSceneIntegerChannel MinutesCurve;

	/** Seconds curve data */
	UPROPERTY()
	FMovieSceneIntegerChannel SecondsCurve;

	/** Frames curve data */
	UPROPERTY()
	FMovieSceneIntegerChannel FramesCurve;

	/** Subframes curve data */
	UPROPERTY()
	FMovieSceneFloatChannel SubFramesCurve;

	/** Timecode rate curve data */
	UPROPERTY()
	FMovieSceneFloatChannel RateCurve;

	/** Slate data */
	UPROPERTY()
	FMovieSceneStringChannel Slate;
};
