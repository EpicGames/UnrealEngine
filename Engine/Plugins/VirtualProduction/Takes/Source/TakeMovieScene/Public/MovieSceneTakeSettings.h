// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieSceneTakeSettings.generated.h"

/**
 * Universal take recorder settings that apply to a whole take
 */
UCLASS(config=EditorSettings, MinimalAPI)
class UMovieSceneTakeSettings : public UObject
{
public:
	GENERATED_BODY()

	UMovieSceneTakeSettings();

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** Hours Name */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	FString HoursName;

	/** Minutes Name */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	FString MinutesName;

	/** Seconds Name */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	FString SecondsName;

	/** Frames Name */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	FString FramesName;

	/** SubFrames Name */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	FString SubFramesName;

	/** Rate Name */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	FString RateName;

	/** Slate Name */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	FString SlateName;
};
