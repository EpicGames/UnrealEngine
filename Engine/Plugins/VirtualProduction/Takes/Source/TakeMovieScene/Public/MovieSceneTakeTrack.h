// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneTakeTrack.generated.h"

/**
 * Handles manipulation of takes in a movie scene
 */
UCLASS( MinimalAPI )
class UMovieSceneTakeTrack : public UMovieSceneNameableTrack
{
	GENERATED_BODY()

public:

	UMovieSceneTakeTrack(const FObjectInitializer& Init);
	
	// UMovieSceneTrack interface
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual bool IsEmpty() const override;
	virtual void RemoveAllAnimationData() override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
	virtual bool CanRename() const { return false; }
#endif

private:
	/** The track's sections. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};
