// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TakeMovieScene : ModuleRules
{
	public TakeMovieScene(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
                "MovieScene",
                "MovieSceneTracks",
				"Engine",
            }
        );
	}
}
