// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"
#include "Misc/Optional.h"

namespace UE::TakeRecorder
{
	/** Guard around saving/restoring a console variable value. */
	template<typename TCVarType>
	class TGuardCVar
	{
	public:

		[[nodiscard]] TGuardCVar(const TCHAR* Name, const TCVarType& NewValue)
			: ConsoleVariable(IConsoleManager::Get().FindConsoleVariable(Name))
			, ValueToRestore([this, &NewValue]
			{
				if (ensure(ConsoleVariable))
				{
					TCVarType OldValue;
					ConsoleVariable->GetValue(OldValue);
					if (OldValue != NewValue)
					{
						return TOptional<TCVarType>(OldValue);
					}
				}
				return TOptional<TCVarType>();
			}())
		{
			if (ConsoleVariable && ValueToRestore)
			{
				ConsoleVariable->Set(NewValue);
			}
		}

		~TGuardCVar()
		{
			if (ConsoleVariable && ValueToRestore)
			{
				ConsoleVariable->Set(*ValueToRestore);
			}
		}

	private:

		IConsoleVariable* const ConsoleVariable = nullptr;
		const TOptional<TCVarType> ValueToRestore;
	};
}


