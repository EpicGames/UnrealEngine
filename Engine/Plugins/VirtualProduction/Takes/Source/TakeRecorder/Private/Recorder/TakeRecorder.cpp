// Copyright Epic Games, Inc. All Rights Reserved.

#include "Recorder/TakeRecorder.h"

#include "Algo/Accumulate.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CoreGlobals.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "LevelSequence.h"
#include "MetaData/MovieSceneShotMetaData.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "MovieSceneTimeHelpers.h"
#include "Recorder/TakeRecorderBlueprintLibrary.h"
#include "Recorder/TakeRecorderSubsystem.h"
#include "TakeMetaData.h"
#include "TakePreset.h"
#include "TakeRecorderOverlayWidget.h"
#include "TakeRecorderSources.h"
#include "TakeRecorderTracing.h"
#include "TakesUtils.h"
#include "Tickable.h"
#include "Stats/Stats.h"
#include "ITakeRecorderSourcesManager.h"

// Engine includes
#include "GameFramework/WorldSettings.h"
#include "Framework/Application/SlateApplication.h"

#if WITH_EDITOR
#include "ISequencer.h"
#include "ObjectTools.h"

#include "AnimatedRange.h"

#include "SequencerSettings.h"

// UnrealEd includes
#include "Editor.h"

// LevelEditor includes
#include "IAssetViewport.h"
#include "LevelEditor.h"
#include "MovieSceneToolHelpers.h"
#include "Subsystems/AssetEditorSubsystem.h"

#endif // WITH_EDITOR

#include "UObject/GCObjectScopeGuard.h"

// Slate includes
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Notifications/INotificationWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"

#include "Misc/GuardCVar.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorder)

#define LOCTEXT_NAMESPACE "TakeRecorder"

DEFINE_LOG_CATEGORY(ManifestSerialization);

class STakeRecorderNotification : public SCompoundWidget, public INotificationWidget
{
public:
	SLATE_BEGIN_ARGS(STakeRecorderNotification){}
	SLATE_END_ARGS()

	void SetOwner(TSharedPtr<SNotificationItem> InOwningNotification)
	{
		WeakOwningNotification = InOwningNotification;
	}

	void Construct(const FArguments& InArgs, UTakeRecorder* InTakeRecorder, ULevelSequence* InFinishedAsset = nullptr)
	{
		WeakRecorder = InTakeRecorder;
		WeakFinishedAsset = InFinishedAsset;
		TakeRecorderState = InTakeRecorder->GetState();

	#if WITH_EDITOR
		UTakeMetaData* TakeMetaData = InTakeRecorder->GetSequence()->FindMetaData<UTakeMetaData>();
		check(TakeMetaData);
	#endif // WITH_EDITOR

		ChildSlot
		[
			SNew(SBox)
		#if WITH_EDITOR
			.Padding(FMargin(15.0f))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.Padding(FMargin(0,0,0,5.0f))
				.HAlign(HAlign_Right)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FCoreStyle::Get().GetFontStyle(TEXT("NotificationList.FontBold")))
						.Text(FText::Format(LOCTEXT("RecordingTitleFormat", "Take {0} of slate {1}"), FText::AsNumber(TakeMetaData->GetTakeNumber()), FText::FromString(TakeMetaData->GetSlate())))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(15.f,0,0,0))
					[
						SAssignNew(Throbber, SThrobber)
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0,0,0,5.0f))
				.HAlign(HAlign_Right)
				[
					SAssignNew(TextBlock, STextBlock)
					.Font(FCoreStyle::Get().GetFontStyle(TEXT("NotificationList.FontLight")))
					.Text(GetDetailText())
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SAssignNew(Hyperlink, SHyperlink)
						.Text(LOCTEXT("BrowseToAsset", "Browse To..."))
						.OnNavigate(this, &STakeRecorderNotification::BrowseToAssetFolder)
						.Visibility(this, &STakeRecorderNotification::CanBrowseToAssetFolder)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(5.0f,0,0,0))
					.VAlign(VAlign_Center)
					[
						SAssignNew(Button, SButton)
						.Text(LOCTEXT("StopButton", "Stop"))
						.OnClicked(this, &STakeRecorderNotification::ButtonClicked)
					]
				]
			]
		#endif // WITH_EDITOR
		];
	}

private:

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		bool bCloseNotification = false;
		bool bCloseImmediately = false;
		if (WeakFinishedAsset.IsValid())
		{
			TextBlock->SetText(GetDetailText());

			Throbber->SetVisibility(EVisibility::Collapsed);
			Button->SetVisibility(EVisibility::Collapsed);

			return;
		}
		else if (WeakRecorder.IsStale())
		{
			// Reset so we don't continually close the notification
			bCloseImmediately = true;
		}
		else if (UTakeRecorder* Recorder = WeakRecorder.Get())
		{
			ETakeRecorderState NewTakeRecorderState = Recorder->GetState();

			if (NewTakeRecorderState == ETakeRecorderState::CountingDown || NewTakeRecorderState == ETakeRecorderState::Started)
			{
				// When counting down the text may change on tick
				TextBlock->SetText(GetDetailText());
			}

			if (NewTakeRecorderState != TakeRecorderState)
			{
				TextBlock->SetText(GetDetailText());

				if (NewTakeRecorderState == ETakeRecorderState::Stopped || NewTakeRecorderState == ETakeRecorderState::Cancelled)
				{
					Throbber->SetVisibility(EVisibility::Collapsed);
					Button->SetVisibility(EVisibility::Collapsed);

					bCloseNotification = true;
				}
			}

			TakeRecorderState = NewTakeRecorderState;
		}

		TSharedPtr<SNotificationItem> Owner = WeakOwningNotification.Pin();
		if ((bCloseNotification || bCloseImmediately) && Owner.IsValid())
		{
			if (bCloseImmediately)
			{
				Owner->SetFadeOutDuration(0.f);
				Owner->SetExpireDuration(0.f);
			}

			Owner->ExpireAndFadeout();

			// Remove our reference to the owner now that it's fading out
			Owner = nullptr;
		}
	}

	FText GetDetailText() const
	{
		if (WeakFinishedAsset.IsValid())
		{
			return LOCTEXT("CompleteText", "Recording Complete");
		}

		UTakeRecorder* Recorder = WeakRecorder.Get();
		if (Recorder)
		{
			if (Recorder->GetState() == ETakeRecorderState::CountingDown)
			{
				return FText::Format(LOCTEXT("CountdownText", "Recording in {0}s..."), FText::AsNumber(FMath::CeilToInt(Recorder->GetCountdownSeconds())));
			}
			else if (Recorder->GetState() == ETakeRecorderState::Stopped)
			{
				return LOCTEXT("CompleteText", "Recording Complete");
			}
			else if (Recorder->GetState() == ETakeRecorderState::Cancelled)
			{
				return LOCTEXT("CancelledText", "Recording Cancelled");
			}

			ULevelSequence* LevelSequence = Recorder->GetSequence();

		#if WITH_EDITOR
			UTakeMetaData* TakeMetaData = LevelSequence ? LevelSequence->FindMetaData<UTakeMetaData>() : nullptr;

			if (TakeMetaData)
			{
				FFrameRate FrameRate = TakeMetaData->GetFrameRate();
				FTimespan RecordingDuration = FDateTime::UtcNow() - TakeMetaData->GetTimestamp();

				FFrameNumber TotalFrames = FFrameNumber(static_cast<int32>(FrameRate.AsDecimal() * RecordingDuration.GetTotalSeconds()));

				FTimecode Timecode = FTimecode::FromFrameNumber(TotalFrames, FrameRate);

				return FText::Format(LOCTEXT("RecordingTimecodeText", "Recording...{0}"), FText::FromString(Timecode.ToString()));
			}
		#endif // WITH_EDITOR
		}

		return LOCTEXT("RecordingText", "Recording...");
	}

	virtual TSharedRef<SWidget> AsWidget() override
	{
		return AsShared();
	}

	// Unused
	virtual void OnSetCompletionState(SNotificationItem::ECompletionState InState) override
	{
	}

private:

	FReply ButtonClicked()
	{
		UTakeRecorder* Recorder = WeakRecorder.Get();
		if (Recorder)
		{
			Recorder->Stop();
		}
		return FReply::Handled();
	}

	void BrowseToAssetFolder() const
	{
		ULevelSequence* Asset = WeakFinishedAsset.Get();

		if (!Asset)
		{
			UTakeRecorder*  Recorder = WeakRecorder.Get();
			Asset = Recorder ? Recorder->GetSequence() : nullptr;
		}

#if WITH_EDITOR
		if (Asset)
		{
			TArray<FAssetData> Assets{ Asset };
			GEditor->SyncBrowserToObjects(Assets);
		}
#endif // WITH_EDITOR
	}

	EVisibility CanBrowseToAssetFolder() const
	{
		if (WeakRecorder.IsValid())
		{
			if (WeakRecorder.Get()->GetState() == ETakeRecorderState::Cancelled)
			{
				return EVisibility::Hidden;
			}
		}

		return EVisibility::Visible;
	}

private:
	TSharedPtr<SWidget> Button, Throbber, Hyperlink;
	TSharedPtr<STextBlock> TextBlock;

	ETakeRecorderState TakeRecorderState;
	TWeakPtr<SNotificationItem> WeakOwningNotification;
	TWeakObjectPtr<UTakeRecorder> WeakRecorder;

	/* Optional asset */
	TWeakObjectPtr<ULevelSequence> WeakFinishedAsset;
};


class FTickableTakeRecorder : public FTickableGameObject
{
public:

	TWeakObjectPtr<UTakeRecorder> WeakRecorder;

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FTickableTakeRecorder, STATGROUP_Tickables);
	}

	//Make sure it always ticks, otherwise we can miss recording, in particularly when time code is always increasing throughout the system.
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }

	virtual bool IsTickableInEditor() const override
	{
		return true;
	}

	virtual UWorld* GetTickableGameObjectWorld() const override
	{
			UTakeRecorder* Recorder = WeakRecorder.Get();
		return Recorder ? Recorder->GetWorld() : nullptr;
	}

	virtual void Tick(float DeltaTime) override
	{
		if (UTakeRecorder* Recorder = WeakRecorder.Get())
		{
			Recorder->Tick(DeltaTime);
		}
	}
};

FTickableTakeRecorder TickableTakeRecorder;

// Static members of UTakeRecorder
static TStrongObjectPtr<UTakeRecorder>& GetCurrentRecorder()
{
	static TStrongObjectPtr<UTakeRecorder> CurrentRecorder;
	return CurrentRecorder;
}
FOnTakeRecordingInitialized UTakeRecorder::OnRecordingInitializedEvent;

void FTakeRecorderParameterOverride::RegisterHandler(FName OverrideName, FTakeRecorderParameterDelegate Delegate)
{
	Delegates.FindOrAdd(MoveTemp(OverrideName), MoveTemp(Delegate));
}

void  FTakeRecorderParameterOverride::UnregisterHandler(FName OverrideName)
{
	Delegates.Remove(MoveTemp(OverrideName));
}

// Static functions for UTakeRecorder
UTakeRecorder* UTakeRecorder::GetActiveRecorder()
{
	return GetCurrentRecorder().Get();
}

FOnTakeRecordingInitialized& UTakeRecorder::OnRecordingInitialized()
{
	return OnRecordingInitializedEvent;
}

FTakeRecorderParameterOverride& UTakeRecorder::TakeInitializeParameterOverride()
{
	static FTakeRecorderParameterOverride Overrides;
	return Overrides;
}

void UTakeRecorder::SetCountdown(float InSeconds)
{
	CountdownSeconds = InSeconds;
}

bool UTakeRecorder::SetActiveRecorder(UTakeRecorder* NewActiveRecorder)
{
	if (GetCurrentRecorder().IsValid())
	{
		return false;
	}

	GetCurrentRecorder().Reset(NewActiveRecorder);
	TickableTakeRecorder.WeakRecorder = GetCurrentRecorder().Get();
	OnRecordingInitializedEvent.Broadcast(NewActiveRecorder);
	return true;
}

// Non-static api for UTakeRecorder

UTakeRecorder::UTakeRecorder(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	CountdownSeconds = 0.f;
	SequenceAsset = nullptr;
	OverlayWidget = nullptr;
}

void UTakeRecorder::SetDisableSaveTick(bool InValue)
{
	Parameters.bDisableRecordingAndSave = InValue;
}

namespace TakeInitHelper
{
FTakeRecorderParameters AccumulateParamsOverride(const FTakeRecorderParameters& InParam)
{
	TMap<FName, FTakeRecorderParameterDelegate>& TheDelegates = UTakeRecorder::TakeInitializeParameterOverride().Delegates;
	auto Op = [](FTakeRecorderParameters InParameters, const TPair<FName,FTakeRecorderParameterDelegate>& Pair)
	{
		return Pair.Value.Execute(MoveTemp(InParameters));
	};
	return Algo::Accumulate(TheDelegates, InParam, MoveTemp(Op));
}
}

bool UTakeRecorder::Initialize ( ULevelSequence* LevelSequenceBase, UTakeRecorderSources* Sources, UTakeMetaData* MetaData, const FTakeRecorderParameters& InParameters, FText* OutError )
{
	SCOPED_TAKERECORDER_TRACE(UTakeRecorder_Initialize);
	FGCObjectScopeGuard GCGuard(this);

	if (GetActiveRecorder())
	{
		if (OutError)
		{
			*OutError = LOCTEXT("RecordingInProgressError", "A recording is currently in progress.");
		}
		return false;
	}

	if (MetaData == nullptr)
	{
		if (OutError)
		{
			*OutError = LOCTEXT("NoMetaDataError", "No Meta Data specified.");
		}
		return false;
	}

	if (MetaData->GetSlate().IsEmpty())
	{
		if (OutError)
		{
			*OutError = LOCTEXT("NoSlateSpecifiedError", "No slate specified.");
		}
		return false;
	}

	UTakeMetaData::SetMostRecentMetaData(MetaData);

	OnRecordingPreInitializeEvent.Broadcast(this);

	UTakeRecorderBlueprintLibrary::OnTakeRecorderPreInitialize();
	if (UTakeRecorderSubsystem* TakeRecorderSubsystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>())
	{
		// The editor subsystem is normally responsible broadcasting their delegates, but they don't have a reliable way of binding to a
		// pre-initialize event.
		TakeRecorderSubsystem->GetOnRecordingPreInitializedEvent().Broadcast(this);
		TakeRecorderSubsystem->TakeRecorderPreInitialize.Broadcast();
	}

	FTakeRecorderParameters FinalParameters = TakeInitHelper::AccumulateParamsOverride(InParameters);
	if (FinalParameters.TakeRecorderMode == ETakeRecorderMode::RecordNewSequence)
	{
		if (!CreateDestinationAsset(*FinalParameters.Project.GetTakeAssetPath(), LevelSequenceBase, Sources, MetaData, OutError, FinalParameters.bOpenSequencer))
		{
			return false;
		}
	}
	else
	{
		if (!SetupDestinationAsset(FinalParameters, LevelSequenceBase, Sources, MetaData, OutError, FinalParameters.bOpenSequencer))
		{
			return false;
		}
	}

	// -----------------------------------------------------------
	// Anything after this point assumes successful initialization
	// -----------------------------------------------------------

	AddToRoot();

	Parameters = FinalParameters;
	State      = ETakeRecorderState::CountingDown;

	// Override parameters for recording into a current sequence
	if (Parameters.TakeRecorderMode == ETakeRecorderMode::RecordIntoSequence)
	{
		Parameters.Project.bStartAtCurrentTimecode = false;
#if WITH_EDITOR
		Parameters.User.bStopAtPlaybackEnd = true;
#endif // WITH_EDITOR
		Parameters.User.bAutoLock = false;
	}

	if (Parameters.Project.RecordingClockSource == EUpdateClockSource::Timecode)
	{
		Parameters.Project.bStartAtCurrentTimecode = true;
		UE_LOGF(LogTakesCore, Warning, "Overriding Start at Current Timecode to True since the clock source is synced to Timecode.");
	}

	// Perform any other parameter-configurable initialization. Must have a valid world at this point.
	InitializeFromParameters();

	// Figure out which world we're recording from
	DiscoverSourceWorld();

	// Open a recording notification

	if (ShouldShowNotifications())
	{
		TSharedRef<STakeRecorderNotification> Content = SNew(STakeRecorderNotification, this);

		FNotificationInfo Info(Content);
		Info.bFireAndForget = false;
		Info.ExpireDuration = 5.f;

		TSharedPtr<SNotificationItem> PendingNotification = FSlateNotificationManager::Get().AddNotification(Info);
		Content->SetOwner(PendingNotification);
	}

	ensure(SetActiveRecorder(this));

	// Suppress config file I/O during recording to prevent hitches from deferred
	// layout saves (FTabManager::SavePersistentLayout) and other config writes.
	// In-memory config updates still work normally; dirty configs are flushed
	// to disk once when recording stops.
	GConfig->DisableFileOperations();
	OnStopCleanup.Add([] {
		GConfig->EnableFileOperations();
		GConfig->Flush(false);
		});

#if WITH_EDITOR
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		// If a start frame was specified, adjust the playback range before rewinding to the beginning of the playback range
		UMovieScene* MovieScene = SequenceAsset->GetMovieScene();
		MovieScene->SetPlaybackRange(TRange<FFrameNumber>(Parameters.StartFrame, MovieScene->GetPlaybackRange().GetUpperBoundValue()));

		// Always start the recording at the beginning of the playback range
		Sequencer->SetLocalTime(MovieScene->GetPlaybackRange().GetLowerBoundValue());

		if (Parameters.TakeRecorderMode == ETakeRecorderMode::RecordNewSequence)
		{
			USequencerSettings* SequencerSettings = USequencerSettingsContainer::GetOrCreate<USequencerSettings>(TEXT("TakeRecorderSequenceEditor"));
		
			CachedAllowEditsMode = SequencerSettings->GetAllowEditsMode();
			CachedAutoChangeMode = SequencerSettings->GetAutoChangeMode();
		
			//When we start recording we don't want to track anymore.  It will be restored when stopping recording.
			SequencerSettings->SetAllowEditsMode(EAllowEditsMode::AllEdits);
			SequencerSettings->SetAutoChangeMode(EAutoChangeMode::None);

			Sequencer->SetSequencerSettings(SequencerSettings);
		}

		// Center the view range around the current time about to be captured
		FAnimatedRange Range = Sequencer->GetViewRange();
		FTimecode CurrentTime = FApp::GetTimecode();
		FFrameRate FrameRate = MovieScene->GetDisplayRate();
		FFrameNumber ViewRangeStart = CurrentTime.ToFrameNumber(FrameRate);
		double ViewRangeStartSeconds = Parameters.Project.bStartAtCurrentTimecode ? FrameRate.AsSeconds(ViewRangeStart) : MovieScene->GetPlaybackRange().GetLowerBoundValue() / MovieScene->GetTickResolution();
		FAnimatedRange NewRange(ViewRangeStartSeconds - 0.5f, ViewRangeStartSeconds + (Range.GetUpperBoundValue() - Range.GetLowerBoundValue()) + 0.5f);
		Sequencer->SetViewRange(NewRange, EViewRangeInterpolation::Immediate);
		Sequencer->SetClampRange(Sequencer->GetViewRange());
	}
#endif // WITH_EDITOR

	return true;
}

void UTakeRecorder::DiscoverSourceWorld()
{
	WeakWorld = TakesUtils::DiscoverSourceWorld();

	bool bPlayInGame = WeakWorld->WorldType == EWorldType::PIE || WeakWorld->WorldType == EWorldType::Game;
	// If recording via PIE, be sure to stop recording cleanly when PIE ends
	if ( bPlayInGame )
	{
		// If CountdownSeconds is zero and the framerate is high, then we can create the overlay but it
		// never ends up being visible. However, when the framerate is low (e.g. in debug) it does show
		// for a single frame, which is undesirable, so only make it if it's going to last some time.
		if (CountdownSeconds > 0)
		{
			UClass* Class = StaticLoadClass(UTakeRecorderOverlayWidget::StaticClass(), nullptr, TEXT("/Takes/UMG/DefaultRecordingOverlay.DefaultRecordingOverlay_C"));
			if (Class)
			{
				OverlayWidget = CreateWidget<UTakeRecorderOverlayWidget>(WeakWorld.Get(), Class);
				OverlayWidget->SetFlags(RF_Transient);
				OverlayWidget->SetRecorder(this);
				OverlayWidget->AddToViewport();
			}
		}
	#if WITH_EDITOR
		FEditorDelegates::EndPIE.AddUObject(this, &UTakeRecorder::HandlePIE);
	#endif // WITH_EDITOR
	}

#if WITH_EDITOR
	// If not recording via PIE, be sure to stop recording if PIE Starts
	if ( !bPlayInGame )
	{
		FEditorDelegates::BeginPIE.AddUObject(this, &UTakeRecorder::HandlePIE);//reuse same function
	}
#endif // WITH_EDITOR
}

bool UTakeRecorder::CreateDestinationAsset(const TCHAR* AssetPathFormat, ULevelSequence* LevelSequenceBase, UTakeRecorderSources* Sources, UTakeMetaData* MetaData, FText* OutError, bool bOpenSequencer)
{
	check(LevelSequenceBase && Sources && MetaData);

	FString   PackageName;

	if (!MetaData->TryGenerateRootAssetPath(AssetPathFormat, PackageName))
	{
		return false;
	}

	// Initialize a new package, ensuring that it has a unique name
	if (!TakesUtils::CreateNewAssetPackage<ULevelSequence>(PackageName, SequenceAsset, OutError, LevelSequenceBase))
	{
		return false;
	}

#if WITH_EDITOR
	// Copy the sources into the level sequence for future reference (and potentially mutation throughout recording)
	SequenceAsset->CopyMetaData(Sources);
#endif // WITH_EDITOR

	UMovieScene*   MovieScene    = SequenceAsset->GetMovieScene();

#if WITH_EDITOR
	UTakeMetaData* AssetMetaData = SequenceAsset->CopyMetaData(MetaData);
#else // WITH_EDITOR
	UTakeMetaData* AssetMetaData = DuplicateObject(MetaData, SequenceAsset);
#endif // WITH_EDITOR

	// Ensure the asset meta-data is unlocked for the recording (it is later Locked when the recording finishes)
	AssetMetaData->Unlock();
	AssetMetaData->ClearFlags(RF_Transient);

	FDateTime UtcNow = FDateTime::UtcNow();
	AssetMetaData->SetTimestamp(UtcNow);

	// @todo: duration / tick resolution / sample rate / frame rate needs some clarification between sync clocks, template sequences and meta data
	if (AssetMetaData->GetDuration() > 0)
	{
		TRange<FFrameNumber> PlaybackRange = TRange<FFrameNumber>::Inclusive(0, ConvertFrameTime(AssetMetaData->GetDuration(), AssetMetaData->GetFrameRate(), MovieScene->GetTickResolution()).CeilToFrame());
		MovieScene->SetPlaybackRange(PlaybackRange);
	}
	MovieScene->SetDisplayRate(AssetMetaData->GetFrameRate());

	SequenceAsset->MarkPackageDirty();
	FAssetRegistryModule::AssetCreated(SequenceAsset);

	bool bSuccess = true;
#if WITH_EDITOR
	if (bOpenSequencer && GIsEditor)
	{
		WeakSequencer = TakesUtils::OpenSequencer(SequenceAsset, OutError);
		bSuccess = WeakSequencer.IsValid();
	}
#endif // WITH_EDITOR
	return bSuccess;
}

bool UTakeRecorder::SetupDestinationAsset(const FTakeRecorderParameters& InParameters, ULevelSequence* LevelSequenceBase, UTakeRecorderSources* Sources, UTakeMetaData* MetaData, FText* OutError, bool bOpenSequencer)
{
	check(LevelSequenceBase && Sources && MetaData);

	SequenceAsset = LevelSequenceBase;

#if WITH_EDITOR
	if (bOpenSequencer)
	{
		WeakSequencer = TakesUtils::OpenSequencer(LevelSequenceBase, OutError);
		if (!WeakSequencer.IsValid())
		{
			return false;
		}
	}

	// The SequenceAsset is either LevelSequenceBase or the currently focused sequence
	if (TSharedPtr<ISequencer> PinnedSequencer = WeakSequencer.Pin())
	{
		SequenceAsset = Cast<ULevelSequence>(PinnedSequencer->GetFocusedMovieSceneSequence());
	}
#endif // WITH_EDITOR

	check(SequenceAsset);
	
#if WITH_EDITOR
	// Copy the sources into the level sequence for future reference (and potentially mutation throughout recording)
	SequenceAsset->CopyMetaData(Sources);
#endif // WITH_EDITOR

	UMovieScene*   MovieScene    = SequenceAsset->GetMovieScene();

#if WITH_EDITOR
	UTakeMetaData* AssetMetaData = SequenceAsset->CopyMetaData(MetaData);
#else // WITH_EDITOR
	UTakeMetaData* AssetMetaData = DuplicateObject(MetaData, SequenceAsset);
#endif // WITH_EDITOR

	// Ensure the asset meta-data is unlocked for the recording (it is later Locked when the recording finishes)
	AssetMetaData->Unlock();
	AssetMetaData->ClearFlags(RF_Transient);

	FDateTime UtcNow = FDateTime::UtcNow();
	AssetMetaData->SetTimestamp(UtcNow);

	// When recording into an existing level sequence, set the asset metadata to the sequence's display rate
	if (InParameters.TakeRecorderMode == ETakeRecorderMode::RecordIntoSequence)
	{
		AssetMetaData->SetFrameRate(MovieScene->GetDisplayRate());
	}

	SequenceAsset->MarkPackageDirty();
	
	return true;
}

void UTakeRecorder::InitializeFromParameters()
{
	// Initialize the countdown delay
	CountdownSeconds = Parameters.User.CountdownSeconds;

	// Set the end recording frame if enabled
	StopRecordingFrame = Parameters.User.bStopAtPlaybackEnd ? SequenceAsset->GetMovieScene()->GetPlaybackRange().GetUpperBoundValue() : TOptional<FFrameNumber>();

#if WITH_EDITOR
	// Apply immersive mode if the parameters demand it
	if (Parameters.User.bMaximizeViewport)
	{
		TSharedPtr<IAssetViewport> ActiveLevelViewport = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor").GetFirstActiveViewport();

		// If it's already immersive we just leave it alone
		if (ActiveLevelViewport.IsValid() && !ActiveLevelViewport->IsImmersive())
		{
			ActiveLevelViewport->MakeImmersive(true/*bWantImmersive*/, false/*bAllowAnimation*/);

			// Restore it when we're done
			auto RestoreImmersiveMode = [WeakViewport = TWeakPtr<IAssetViewport>(ActiveLevelViewport)]
			{
				if (TSharedPtr<IAssetViewport> CleaupViewport = WeakViewport.Pin())
				{
					CleaupViewport->MakeImmersive(false/*bWantImmersive*/, false/*bAllowAnimation*/);
				}
			};
			OnStopCleanup.Add(RestoreImmersiveMode);
		}
	}
#endif // WITH_EDITOR
}

bool UTakeRecorder::ShouldShowNotifications()
{
#if WITH_EDITOR
	// -TAKERECORDERISHEADLESS in the command line can force headless behavior and disable the notifications.
	static const bool bCmdLineTakeRecorderIsHeadless = FParse::Param(FCommandLine::Get(), TEXT("TAKERECORDERISHEADLESS"));

	return Parameters.Project.bShowNotifications
		&& !bCmdLineTakeRecorderIsHeadless
		&& !FApp::IsUnattended()
		&& !GIsRunningUnattendedScript;
#else // WITH_EDITOR
	return false;
#endif // WITH_EDITOR
}

UWorld* UTakeRecorder::GetWorld() const
{
	return WeakWorld.Get();
}

void UTakeRecorder::Tick(float DeltaTime)
{
	// If the recording world has been destroyed (e.g. sandbox switch, map change), cancel gracefully.
	if (!GetWorld())
	{
		Cancel();
		return;
	}

	if (State == ETakeRecorderState::CountingDown)
	{
		NumberOfTicksAfterPre = 0;
		CountdownSeconds = FMath::Max(0.f, CountdownSeconds - DeltaTime);
		if (CountdownSeconds > 0.f)
		{
			return;
		}
		PreRecord();
	}
	else if (State == ETakeRecorderState::PreRecord)
	{
		if (++NumberOfTicksAfterPre == 2) //seems we need 2 ticks to make sure things are settled
		{
			State = ETakeRecorderState::TickingAfterPre;
		}
	}
	else if (State == ETakeRecorderState::TickingAfterPre)
	{
		NumberOfTicksAfterPre = 0;
		Start();
		InternalTick(0.0f);
	}
	else if (State == ETakeRecorderState::Started)
	{
		InternalTick(DeltaTime);
	}
}

FQualifiedFrameTime UTakeRecorder::GetRecordTime() const
{
	float TimeDilation = 1.f;

#if WITH_EDITOR
	if (!WeakSequencer.IsValid())
#endif // WITH_EDITOR
	{
		const UWorld* RecordingWorld = GetWorld();
		check(RecordingWorld);
		if (const AWorldSettings* WorldSettings = RecordingWorld->GetWorldSettings())
		{
			// If we're not using Sequencer we have to apply time dilation ourselves as this normally happens in Sequencer.cpp
			// when the TimeController is ticked.
			TimeDilation = WorldSettings->CinematicTimeDilation;
		}
	}
	
#if WITH_EDITOR
	return TakesUtils::GetRecordTime(WeakSequencer.Pin(), SequenceAsset, TimecodeAtStart,
		Parameters.Project.bStartAtCurrentTimecode, TimeDilation);
#else // WITH_EDITOR
	return TakesUtils::GetRecordTime(SequenceAsset, TimecodeAtStart,
		Parameters.Project.bStartAtCurrentTimecode, TimeDilation);
#endif // WITH_EDITOR
}

namespace UE::TakeRecorder
{
static bool operator<(const FTimecode& InLeft, const FTimecode& InRight)
{
	return InLeft.Hours < InRight.Hours
		|| (InLeft.Hours <= InRight.Hours && InLeft.Minutes < InRight.Minutes)
		|| (InLeft.Hours <= InRight.Hours && InLeft.Minutes <= InRight.Minutes && InLeft.Seconds < InRight.Seconds)
		|| (InLeft.Hours <= InRight.Hours && InLeft.Minutes <= InRight.Minutes && InLeft.Seconds <= InRight.Seconds
			&& InLeft.Frames < InRight.Frames)
		|| (InLeft.Hours <= InRight.Hours && InLeft.Minutes <= InRight.Minutes && InLeft.Seconds <= InRight.Seconds
			&& InLeft.Frames <= InRight.Frames && InLeft.Subframe < InRight.Subframe);
}

static void ShowRolloverNotification()
{
	if (FSlateApplication::IsInitialized())
	{
		FNotificationInfo Info(LOCTEXT("RolloverMessage.Title", "Timecode rolled over."));
		Info.SubText = LOCTEXT("RolloverMessage.Subtext", "Recording was stopped because timecode rolled over.");
		Info.bFireAndForget = true;
		Info.ExpireDuration = 4.f;
		
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}
}

void UTakeRecorder::InternalTick(float DeltaTime)
{
	SCOPED_TAKERECORDER_TRACE(UTakeRecorder_InternalTick);
	using namespace UE::MovieScene;

	using namespace UE::TakeRecorder;
	// We assume timecode providers do not change after recording is started.
	// If no timecode provider is set, then FApp::GetTimecode() == TimecodeAtStart == FTimecode(0,0,0,0).
	if (Parameters.User.bStopOnRollover && FApp::GetTimecode() < TimecodeAtStart)
	{
		ShowRolloverNotification();
		Stop();
		return;
	}

	FScopedSignedObjectModifyDefer FlushOnTick(true);


	FQualifiedFrameTime RecordTime = GetRecordTime();
	
	UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindOrAddSources(SequenceAsset);

	if (!Parameters.bDisableRecordingAndSave)
	{
		CurrentFrameTime = Sources->TickRecording(SequenceAsset, RecordTime, DeltaTime);
	}
	else
	{
		CurrentFrameTime = Sources->AdvanceTime(RecordTime, DeltaTime);
	}

#if WITH_EDITOR
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		FAnimatedRange Range = Sequencer->GetViewRange();
		UMovieScene* MovieScene = SequenceAsset->GetMovieScene();
		if (MovieScene)
		{
			FFrameRate FrameRate = MovieScene->GetTickResolution();
			double CurrentTimeSeconds = FrameRate.AsSeconds(CurrentFrameTime) + 0.5f;
			CurrentTimeSeconds = CurrentTimeSeconds > Range.GetUpperBoundValue() ? CurrentTimeSeconds : Range.GetUpperBoundValue();
			TRange<double> NewRange(Range.GetLowerBoundValue(), CurrentTimeSeconds);
			Sequencer->SetViewRange(NewRange, EViewRangeInterpolation::Immediate);
			Sequencer->SetClampRange(Sequencer->GetViewRange());
		}
	}
#endif // WITH_EDITOR

	OnTickRecordingEvent.Broadcast(this, RecordTime);

	if (StopRecordingFrame.IsSet() && CurrentFrameTime.FrameNumber >= StopRecordingFrame.GetValue())
	{
		Stop();
	}
}

void UTakeRecorder::PreRecord()
{
	SCOPED_TAKERECORDER_TRACE(UTakeRecorder_PreRecord);
	
	State = ETakeRecorderState::PreRecord;

	UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindSources(SequenceAsset);
	check(Sources);

#if WITH_EDITOR
	UTakeMetaData* AssetMetaData = SequenceAsset->FindMetaData<UTakeMetaData>();
#else // WITH_EDITOR
	// Dummy out just a new UTakeMetaData object as it will not persist from a runtime environment anyway.
	UTakeMetaData* AssetMetaData = NewObject<UTakeMetaData>();
#endif // WITH_EDITOR

	//Set the flag to specify if we should auto save the serialized data or not when recording.
	MovieSceneSerializationNamespace::bAutoSerialize = Parameters.User.bAutoSerialize;
	if (Parameters.User.bAutoSerialize)
	{
		FString AssetName;
		if (AssetMetaData->TryGenerateRootAssetPath(Parameters.Project.GetTakeAssetPath(), AssetName))
		{
			FString AssetPath = FPaths::ProjectSavedDir() + AssetName;
			FPaths::RemoveDuplicateSlashes(AssetPath);
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if (!PlatformFile.DirectoryExists(*AssetPath))
			{
				PlatformFile.CreateDirectoryTree(*AssetPath);
			}

			ManifestSerializer.SetLocalCaptureDir(AssetPath);
			FName SerializedType("Sequence");
			FString Name = SequenceAsset->GetName();
			FManifestFileHeader Header(Name, SerializedType, FGuid());
			FText Error;
			FString FileName = FString::Printf(TEXT("%s_%s"), *(SerializedType.ToString()), *(Name));

			if (!ManifestSerializer.OpenForWrite(FileName, Header, Error))
			{
				UE_LOGF(ManifestSerialization, Warning, "Error Opening Sequence Sequencer File: Subject '%ls' Error '%ls'", *(Name), *(Error.ToString()));
			}
		}
	}

	FTakeRecorderSourcesSettings TakeRecorderSourcesSettings;
	TakeRecorderSourcesSettings.bStartAtCurrentTimecode = Parameters.Project.bStartAtCurrentTimecode;
	TakeRecorderSourcesSettings.bRecordSourcesIntoSubSequences = Parameters.Project.bRecordSourcesIntoSubSequences;
	TakeRecorderSourcesSettings.bRecordToPossessable = Parameters.Project.bRecordToPossessable;

#if WITH_EDITOR
	TakeRecorderSourcesSettings.bSaveRecordedAssets = (!Parameters.bDisableRecordingAndSave && Parameters.User.bSaveRecordedAssets) || GEditor == nullptr;
#else // WITH_EDITOR
	TakeRecorderSourcesSettings.bSaveRecordedAssets = (!Parameters.bDisableRecordingAndSave && Parameters.User.bSaveRecordedAssets);
#endif // WITH_EDITOR

	TakeRecorderSourcesSettings.bRemoveRedundantTracks = Parameters.User.bRemoveRedundantTracks;
	TakeRecorderSourcesSettings.bAutoLock = Parameters.User.bAutoLock;
	TakeRecorderSourcesSettings.SubSequenceDirectory = Parameters.Project.SubSequenceDirectory;

#if WITH_EDITOR
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer)
	{
		TakeRecorderSourcesSettings.TransformOrigin = MovieSceneToolHelpers::GetTransformOriginForFocusedSequence(Sequencer);
	}
#endif // WITH_EDITOR

	Sources->SetSettings(TakeRecorderSourcesSettings);

	if (!Parameters.bDisableRecordingAndSave)
	{
		UMovieScene* MovieScene = SequenceAsset->GetMovieScene();
		FQualifiedFrameTime SequencerTime;
		if (MovieScene)
		{
			FTimecode Timecode = FApp::GetTimecode();
			FFrameRate FrameRate = MovieScene->GetDisplayRate();
			FFrameRate TickResolution = MovieScene->GetTickResolution();
			FFrameNumber PlaybackStartFrame = Parameters.Project.bStartAtCurrentTimecode ? FFrameRate::TransformTime(FFrameTime(Timecode.ToFrameNumber(FrameRate)), FrameRate, TickResolution).FloorToFrame() : MovieScene->GetPlaybackRange().GetLowerBoundValue();
			SequencerTime = FQualifiedFrameTime(PlaybackStartFrame, TickResolution);
		}

		Sources->PreRecording(SequenceAsset, SequencerTime, Parameters.User.bAutoSerialize ? &ManifestSerializer : nullptr);
	}
	else
	{
		Sources->SetCachedAssets(SequenceAsset, Parameters.User.bAutoSerialize ? &ManifestSerializer : nullptr);
	}

#if WITH_EDITOR
	// Refresh sequencer in case the movie scene data has mutated (ie. existing object bindings removed because they will be recorded again)
	if (Sequencer.IsValid())
	{
		// Sequencer spawnable support.
		{
			// RecreateCurveEditor must come before RefreshTree. Stale ids may be present when Take Recorder
			// is recording a spawnable from sequencer, which will eventually trigger ensure(ParentInfo)
			// in the curve editor if we don't recreate it.
			Sequencer->RecreateCurveEditor();
		}
		Sequencer->RefreshTree();
	}
#endif // WITH_EDITOR

	// Apply engine Time Dilation after the countdown, otherwise the countdown will be dilated as well!
	UWorld* RecordingWorld = GetWorld();
	check(RecordingWorld);
	if (AWorldSettings* WorldSettings = RecordingWorld->GetWorldSettings())
	{
		const float ExistingCinematicTimeDilation = WorldSettings->CinematicTimeDilation;

		const bool bInvalidTimeDilation = Parameters.User.EngineTimeDilation == 0.f;

		if (bInvalidTimeDilation)
		{
			UE_LOGF(LogTakesCore, Warning, "Time dilation cannot be 0. Ignoring time dilation for this recording.");
		}

		if (Parameters.User.EngineTimeDilation != ExistingCinematicTimeDilation && !bInvalidTimeDilation)
		{
			WorldSettings->CinematicTimeDilation = Parameters.User.EngineTimeDilation;

			// Restore it when we're done
			auto RestoreTimeDilation = [ExistingCinematicTimeDilation, WeakWorldSettings = MakeWeakObjectPtr(WorldSettings)]
			{
				if (AWorldSettings* CleaupWorldSettings = WeakWorldSettings.Get())
				{
					CleaupWorldSettings->CinematicTimeDilation = ExistingCinematicTimeDilation;
				}
			};
			OnStopCleanup.Add(RestoreTimeDilation);
		}
	}
}

void UTakeRecorder::Start()
{
	SCOPED_TAKERECORDER_TRACE(UTakeRecorder_Start);
	using namespace UE::MovieScene;

	FTimecode Timecode = FApp::GetTimecode();

	State = ETakeRecorderState::Started;


	CurrentFrameTime = FFrameTime(0);
	TimecodeAtStart = Timecode;

#if WITH_EDITOR
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	// Discard any entity tokens we have so that restore state does not take effect when we delete any sections that recording will be replacing.
	if (Sequencer.IsValid())
	{
		Sequencer->PreAnimatedState.DiscardEntityTokens();
	}
#endif // WITH_EDITOR

	UMovieScene* MovieScene = SequenceAsset->GetMovieScene();
	if (MovieScene)
	{
		CachedPlaybackRange = MovieScene->GetPlaybackRange();
		CachedClockSource = MovieScene->GetClockSource();
		MovieScene->SetClockSource(Parameters.Project.RecordingClockSource);

#if WITH_EDITOR
		if (Sequencer.IsValid())
		{
			Sequencer->ResetTimeController();
		}
#endif // WITH_EDITOR

		FFrameRate FrameRate = MovieScene->GetDisplayRate();
		FFrameRate TickResolution = MovieScene->GetTickResolution();
		FFrameNumber PlaybackStartFrame = Parameters.Project.bStartAtCurrentTimecode ? FFrameRate::TransformTime(FFrameTime(Timecode.ToFrameNumber(FrameRate)), FrameRate, TickResolution).FloorToFrame() : MovieScene->GetPlaybackRange().GetLowerBoundValue();

		// Transform all the sections to start the playback start frame
		FFrameNumber DeltaFrame = PlaybackStartFrame - MovieScene->GetPlaybackRange().GetLowerBoundValue();
		if (DeltaFrame != 0)
		{
			for (UMovieSceneSection* Section : MovieScene->GetAllSections())
			{
				Section->MoveSection(DeltaFrame);
			}
		}

		OnFrameModifiedEvent.Broadcast(this, PlaybackStartFrame);

		if (StopRecordingFrame.IsSet())
		{
			StopRecordingFrame = StopRecordingFrame.GetValue() + DeltaFrame;
		}

		// Set infinite playback range when starting recording. Playback range will be clamped to the bounds of the sections at the completion of the recording
		MovieScene->SetPlaybackRange(TRange<FFrameNumber>(PlaybackStartFrame, TNumericLimits<int32>::Max() - 1), false);

#if WITH_EDITOR
		if (Sequencer.IsValid())
		{
			Sequencer->SetGlobalTime(PlaybackStartFrame);
			Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
		}
#endif // WITH_EDITOR
	}

	UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindSources(SequenceAsset);
	check(Sources);

#if WITH_EDITOR
	UTakeMetaData* AssetMetaData = SequenceAsset->FindMetaData<UTakeMetaData>();
#else // WITH_EDITOR
	UTakeMetaData* AssetMetaData = NewObject<UTakeMetaData>();
#endif // WITH_EDITOR

	FDateTime UtcNow = FDateTime::UtcNow();
	AssetMetaData->SetTimestamp(UtcNow);
	AssetMetaData->SetTimecodeIn(Timecode);

	if (!Parameters.bDisableRecordingAndSave)
	{
		SCOPED_TAKERECORDER_TRACE(UTakeRecorder_StartRecordingSources);
		
		FQualifiedFrameTime RecordTime = GetRecordTime();

		Sources->StartRecording(SequenceAsset, RecordTime, Parameters.User.bAutoSerialize ? &ManifestSerializer : nullptr);

		// Record immediately so that there's a key on the first frame of recording
		Sources->TickRecording(SequenceAsset, RecordTime, 0.1f);
		OnTickRecordingEvent.Broadcast(this, RecordTime);
	}

	if (!ShouldShowNotifications())
	{
		// Log in lieu of the notification widget
		UE_LOGF(LogTakesCore, Log, "Started recording");
	}

	OnRecordingStartedEvent.Broadcast(this);

	UTakeRecorderBlueprintLibrary::OnTakeRecorderStarted();

#if WITH_EDITOR
	if (Sequencer.IsValid())
	{
		CompileSuppression = MakeUnique<FScopedVolatilityManagerSuppression>(Sequencer->GetSharedPlaybackState().ToSharedPtr());
	}
#endif // WITH_EDITOR
}

void UTakeRecorder::Stop()
{
	const bool bCancelled = false;
	StopInternal(bCancelled);
}

void UTakeRecorder::Cancel()
{
	const bool bCancelled = true;
	StopInternal(bCancelled);
}

void UTakeRecorder::StopInternal(const bool bCancelled)
{
	SCOPED_TAKERECORDER_TRACE(UTakeRecorder_StopInternal);
	static bool bStoppedRecording = false;

	if (bStoppedRecording)
	{
		return;
	}

	double StartTime = FPlatformTime::Seconds();

	TGuardValue<bool> ReentrantGuard(bStoppedRecording, true);

#if WITH_EDITOR
	if (Parameters.TakeRecorderMode == ETakeRecorderMode::RecordNewSequence)
	{
		USequencerSettings* SequencerSettings = USequencerSettingsContainer::GetOrCreate<USequencerSettings>(TEXT("TakeRecorderSequenceEditor"));

		SequencerSettings->SetAllowEditsMode(CachedAllowEditsMode);
		SequencerSettings->SetAutoChangeMode(CachedAutoChangeMode);
	}
#endif // WITH_EDITOR

	ManifestSerializer.Close();

#if WITH_EDITOR
	FEditorDelegates::EndPIE.RemoveAll(this);
	FEditorDelegates::BeginPIE.RemoveAll(this);
#endif // WITH_EDITOR
	
	const bool bDidEverStartRecording = State == ETakeRecorderState::Started;
	const bool bRecordingFinished = !bCancelled && bDidEverStartRecording;
	State = bRecordingFinished ? ETakeRecorderState::Stopped : ETakeRecorderState::Cancelled;

#if WITH_EDITOR
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Stopped);
	}
#endif // WITH_EDITOR

	UMovieScene* MovieScene = SequenceAsset ? SequenceAsset->GetMovieScene() : nullptr;
	if (MovieScene)
	{
		MovieScene->SetClockSource(CachedClockSource);

#if WITH_EDITOR
		if (Sequencer.IsValid())
		{
			Sequencer->ResetTimeController();
		}
#endif // WITH_EDITOR
	}

	CompileSuppression.Reset();

	if (bDidEverStartRecording)
	{
		if (!ShouldShowNotifications())
		{
			// Log in lieu of the notification widget
			if (bRecordingFinished)
			{
				UE_LOGF(LogTakesCore, Log, "Stopped recording");
			}
			else
			{
				UE_LOGF(LogTakesCore, Log, "Recording cancelled");
			}
		}

		OnRecordingStoppedEvent.Broadcast(this);

		UTakeRecorderBlueprintLibrary::OnTakeRecorderStopped();

		if (MovieScene)
		{
			TRange<FFrameNumber> Range = MovieScene->GetPlaybackRange();

			//Set Range to what we recorded instead of that large number, this let's us reliably set camera cut times.
			FFrameNumber PlaybackEndFrame = CurrentFrameTime.FrameNumber;

			if (StopRecordingFrame.IsSet())
			{
				PlaybackEndFrame = StopRecordingFrame.GetValue();
			}
			
			MovieScene->SetPlaybackRange(TRange<FFrameNumber>(Range.GetLowerBoundValue(), PlaybackEndFrame));

#if WITH_EDITOR
			if (Sequencer)
			{
				Sequencer->ResetTimeController();
			}
#endif // WITH_EDITOR
		}

		if (!Parameters.bDisableRecordingAndSave)
		{
			UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindSources(SequenceAsset);
			check(Sources);
			Sources->StopRecording(SequenceAsset, bCancelled);
		}

		// Restore the playback range to what it was before recording.
		if (Parameters.TakeRecorderMode == ETakeRecorderMode::RecordIntoSequence)
		{
			if (MovieScene)
			{
				MovieScene->SetPlaybackRange(CachedPlaybackRange);
			}
		}
		else
		{
			TakesUtils::ResetViewAndWorkRange(MovieScene);
		}

		if (bRecordingFinished)
		{
		#if WITH_EDITOR
			// Lock the sequence so that it can't be changed without implicitly unlocking it now
			if (Parameters.User.bAutoLock)
			{
				SequenceAsset->GetMovieScene()->SetReadOnly(true);
			}

			// We have recorded this sequence, mark it.
			UMovieSceneShotMetaData* MovieSceneMetaData = SequenceAsset->FindOrAddMetaData<UMovieSceneShotMetaData>();
			MovieSceneMetaData->SetIsRecorded(true);
		#endif // WITH_EDITOR

		#if WITH_EDITOR
			UTakeMetaData* AssetMetaData = SequenceAsset->FindMetaData<UTakeMetaData>();
		#else // WITH_EDITOR
			UTakeMetaData* AssetMetaData = NewObject<UTakeMetaData>();
		#endif // WITH_EDITOR
			check(AssetMetaData);

			if (MovieScene)
			{
				FFrameRate DisplayRate = MovieScene->GetDisplayRate();
				FFrameRate TickResolution = MovieScene->GetTickResolution();
				FTimecode Timecode = FTimecode::FromFrameNumber(FFrameRate::TransformTime(CurrentFrameTime, TickResolution, DisplayRate).FloorToFrame(), DisplayRate);

				AssetMetaData->SetTimecodeOut(Timecode);
			}

#if WITH_EDITOR
			if (GEditor && GEditor->GetEditorWorldContext().World())
			{
				AssetMetaData->SetLevelOrigin(GEditor->GetEditorWorldContext().World()->PersistentLevel);
			}
#endif // WITH_EDITOR

			if (Parameters.User.bAutoLock)
			{
				// When auto-locking, lock the meta data so it can't be changed without implicitly unlocking it now
				AssetMetaData->Lock();
			}

			if (Parameters.User.bSaveRecordedAssets)
			{
#if WITH_EDITOR
				if (Sequencer)
				{
					// Ensure we don't have any external references prior to saving. This automatically handles possessables vs spawnables.
					TakesUtils::CleanupExternalReferences(SequenceAsset);
					
					// Sequencer has a feature for scrubbing to a particular frame and capturing it as thumbnail.
					// In 5.5, this was added but hidden behind a CVar for stability reason.
					// In BaseTakes.ini, by default, we configure our Sequencer instance to capture the MiddleFrame.
					UE::TakeRecorder::TGuardCVar Guard(TEXT("Sequencer.EnableRelevantThumbnails"), true);
					// Save using the sequencer's save logic if possible - this will generate a relevant thumbnail.
					Sequencer->Save();
				}
				else
#endif // WITH_EDITOR
				{
					// Fallback to "normal" saving if sequencer is not available (seems like it should be though?). Asset won't have any thumbnail.
					TakesUtils::SaveAsset(SequenceAsset);
				}
			}

#if WITH_EDITOR
			// Rebuild sequencer because subsequences could have been added or bindings removed
			if (Sequencer)
			{
				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
			}
#endif // WITH_EDITOR
		}
	}

	// Perform any other cleanup that has been defined for this recording
	for (const TFunction<void()>& Cleanup : OnStopCleanup)
	{
		Cleanup();
	}
	OnStopCleanup.Reset();

	// reset the current recorder and stop us from being ticked
	if (GetCurrentRecorder().Get() == this)
	{
		UTakeMetaData::SetMostRecentMetaData(nullptr);
		
		GetCurrentRecorder().Reset();
		TickableTakeRecorder.WeakRecorder = nullptr;

		if (bRecordingFinished)
		{
			OnRecordingFinishedEvent.Broadcast(this);
			UTakeRecorderBlueprintLibrary::OnTakeRecorderFinished(SequenceAsset);
		}
		else
		{
			OnRecordingCancelledEvent.Broadcast(this);
			UTakeRecorderBlueprintLibrary::OnTakeRecorderCancelled();
		}
	}

#if WITH_EDITOR
	// Tear down the Sequencer instance before we delete the SequenceAsset.
	Sequencer.Reset();
#endif // WITH_EDITOR

	// Delete the asset after OnTakeRecorderFinished and OnTakeRecorderCancelled because the ScopedSequencerPanel 
	// will still have the current sequence before it is returned to the Pending Take.
	if (Parameters.TakeRecorderMode == ETakeRecorderMode::RecordNewSequence && !bRecordingFinished)
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			// Recording was canceled before it started, so delete the asset. Note we can only do this on editor
			// nodes. On -game nodes, this cannot be performed. This can only happen with Mult-user and -game node
			// recording.
			//
			FAssetRegistryModule::AssetDeleted(SequenceAsset);
		}
#endif // WITH_EDITOR

		// Move the asset to the transient package so that new takes with the same number can be created in its place
		FName DeletedPackageName = MakeUniqueObjectName(nullptr, UPackage::StaticClass(), *(FString(TEXT("/Temp/") + SequenceAsset->GetName() + TEXT("_Cancelled"))));
		SequenceAsset->GetOutermost()->Rename(*DeletedPackageName.ToString());

		SequenceAsset->ClearFlags(RF_Standalone | RF_Public);
		SequenceAsset->RemoveFromRoot();
		SequenceAsset->MarkAsGarbage();
		SequenceAsset = nullptr;
	}

	if (OverlayWidget)
	{
		OverlayWidget->RemoveFromParent();
		OverlayWidget->RemoveFromRoot();
		OverlayWidget->MarkAsGarbage();
		OverlayWidget = nullptr;
	}

	if (SequenceAsset)
	{
		UPackage* const Package = SequenceAsset->GetOutermost();
		FString const PackageName = Package->GetName();

		double ElapsedTime = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogTakesCore, Log, "Finished processing %ls in %0.2f seconds", *PackageName, ElapsedTime);
	}

	RemoveFromRoot();
}

FOnTakeRecordingPreInitialize& UTakeRecorder::OnRecordingPreInitialize()
{
	return OnRecordingPreInitializeEvent;
}

FOnTakeRecordingStarted& UTakeRecorder::OnRecordingStarted()
{
	return OnRecordingStartedEvent;
}

FOnTakeRecordingStopped& UTakeRecorder::OnRecordingStopped()
{
	return OnRecordingStoppedEvent;
}

FOnTakeRecordingFinished& UTakeRecorder::OnRecordingFinished()
{
	return OnRecordingFinishedEvent;
}

FOnTakeRecordingCancelled& UTakeRecorder::OnRecordingCancelled()
{
	return OnRecordingCancelledEvent;
}

FOnStartPlayFrameModified& UTakeRecorder::OnStartPlayFrameModified()
{
	return OnFrameModifiedEvent;
}

FOnTickRecording& UTakeRecorder::OnTickRecording()
{
	return OnTickRecordingEvent;
}

#if WITH_EDITOR
void UTakeRecorder::HandlePIE(bool bIsSimulating)
{
	ULevelSequence* FinishedAsset = GetSequence();

	if (ShouldShowNotifications())
	{
		TSharedRef<STakeRecorderNotification> Content = SNew(STakeRecorderNotification, this, FinishedAsset);

		FNotificationInfo Info(Content);
		Info.ExpireDuration = 5.f;

		TSharedPtr<SNotificationItem> PendingNotification = FSlateNotificationManager::Get().AddNotification(Info);
		PendingNotification->SetCompletionState(SNotificationItem::CS_Success);
	}
	
	Stop();

	if (FinishedAsset)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(FinishedAsset);
	}
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

