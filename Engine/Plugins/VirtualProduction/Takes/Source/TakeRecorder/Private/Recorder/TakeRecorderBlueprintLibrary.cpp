// Copyright Epic Games, Inc. All Rights Reserved.

#include "Recorder/TakeRecorderBlueprintLibrary.h"
#include "Recorder/TakeRecorder.h"
#include "Recorder/TakeRecorderPanel.h"
#include "ITakeRecorderModule.h"
#include "TakeRecorderSettings.h"
#include "UObject/Package.h"

#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "LevelEditor.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderBlueprintLibrary)

namespace TakeRecorderBlueprintLibrary
{
	static UTakeRecorderPanel* CurrentTakeRecorderPanel;
	static FOnTakeRecorderPanelChanged TakeRecorderPanelChanged;
	static FOnTakeRecorderPreInitialize TakeRecorderPreInitialize;
	static FOnTakeRecorderStarted TakeRecorderStarted;
	static FOnTakeRecorderStopped TakeRecorderStopped;
	static FOnTakeRecorderFinished TakeRecorderFinished;
	static FOnTakeRecorderCancelled TakeRecorderCancelled;
	static FOnTakeRecorderMarkedFrameAdded TakeRecorderMarkedFrameAdded;
}


bool UTakeRecorderBlueprintLibrary::IsTakeRecorderEnabled()
{
#if WITH_EDITOR
	return !IsRunningGame();
#else
	return false;
#endif 
}

UTakeRecorder* UTakeRecorderBlueprintLibrary::StartRecording(ULevelSequence* LevelSequence, UTakeRecorderSources* Sources, UTakeMetaData* MetaData, const FTakeRecorderParameters& Parameters)
{
	if (IsRecording())
	{
		// @todo: script error?
		return nullptr;
	}

	if (LevelSequence == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The LevelSequence is invalid."), ELogVerbosity::Error);
		return nullptr;
	}
	if (Sources == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The Sources is invalid."), ELogVerbosity::Error);
		return nullptr;
	}
	if (MetaData == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The MetaData is invalid."), ELogVerbosity::Error);
		return nullptr;
	}

	FText ErrorText = NSLOCTEXT("TakeRecorderBlueprintLibrary", "UnknownError", "An unknown error occurred when trying to start recording");

	UTakeRecorder* NewRecorder = NewObject<UTakeRecorder>(GetTransientPackage(), NAME_None, RF_Transient);
	if (NewRecorder->Initialize(LevelSequence, Sources, MetaData, Parameters, &ErrorText))
	{
		return NewRecorder;
	}
	else if (ensure(!ErrorText.IsEmpty()))
	{
		FFrame::KismetExecutionMessage(*ErrorText.ToString(), ELogVerbosity::Error);
	}

	return nullptr;
}

FTakeRecorderParameters UTakeRecorderBlueprintLibrary::GetDefaultParameters()
{
	FTakeRecorderParameters DefaultParams;

	DefaultParams.User            = GetDefault<UTakeRecorderUserSettings>()->Settings;
	DefaultParams.Project         = GetDefault<UTakeRecorderProjectSettings>()->Settings;
	DefaultParams.HitchProtection = GetDefault<UTakeRecorderProjectSettings>()->HitchProtectionSettings;

	return DefaultParams;
}

void UTakeRecorderBlueprintLibrary::SetDefaultParameters(const FTakeRecorderParameters& InDefaultParameters)
{
	GetMutableDefault<UTakeRecorderUserSettings>()->Settings = InDefaultParameters.User;
	GetMutableDefault<UTakeRecorderProjectSettings>()->Settings = InDefaultParameters.Project;
	GetMutableDefault<UTakeRecorderProjectSettings>()->HitchProtectionSettings = InDefaultParameters.HitchProtection;
}

bool UTakeRecorderBlueprintLibrary::IsRecording()
{
	return UTakeRecorder::GetActiveRecorder() != nullptr && UTakeRecorder::GetActiveRecorder()->GetState() == ETakeRecorderState::Started;
}

UTakeRecorder* UTakeRecorderBlueprintLibrary::GetActiveRecorder()
{
	return UTakeRecorder::GetActiveRecorder();
}

void UTakeRecorderBlueprintLibrary::StopRecording()
{
	UTakeRecorder* Existing = UTakeRecorder::GetActiveRecorder();
	if (Existing)
	{
		Existing->Stop();
	}
}

void UTakeRecorderBlueprintLibrary::CancelRecording()
{
	UTakeRecorder* Existing = UTakeRecorder::GetActiveRecorder();
	if (Existing)
	{
		Existing->Cancel();
	}
}

UTakeRecorderPanel* UTakeRecorderBlueprintLibrary::OpenTakeRecorderPanel()
{
	UTakeRecorderPanel* Existing = GetTakeRecorderPanel();
	if (Existing)
	{
		return Existing;
	}

#if WITH_EDITOR
	if (!IsRunningGame())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(ITakeRecorderModule::TakeRecorderTabName);
	}
	else
#endif // WITH_EDITOR
	{
		FFrame::KismetExecutionMessage(TEXT("The Take Recorder Panel will not open because the game is running."), ELogVerbosity::Log);
	}

	return GetTakeRecorderPanel();
}

UTakeRecorderPanel* UTakeRecorderBlueprintLibrary::GetTakeRecorderPanel()
{
	return TakeRecorderBlueprintLibrary::CurrentTakeRecorderPanel;
}

void UTakeRecorderBlueprintLibrary::SetOnTakeRecorderPanelChanged(FOnTakeRecorderPanelChanged OnTakeRecorderPanelChanged)
{
	TakeRecorderBlueprintLibrary::TakeRecorderPanelChanged = OnTakeRecorderPanelChanged;
};

void UTakeRecorderBlueprintLibrary::SetOnTakeRecorderPreInitialize(FOnTakeRecorderPreInitialize OnTakeRecorderPreInitialize)
{
	TakeRecorderBlueprintLibrary::TakeRecorderPreInitialize = OnTakeRecorderPreInitialize;
};

void UTakeRecorderBlueprintLibrary::SetOnTakeRecorderStarted(FOnTakeRecorderStarted OnTakeRecorderStarted)
{
	TakeRecorderBlueprintLibrary::TakeRecorderStarted = OnTakeRecorderStarted;
};

void UTakeRecorderBlueprintLibrary::SetOnTakeRecorderStopped(FOnTakeRecorderStopped OnTakeRecorderStopped)
{
	TakeRecorderBlueprintLibrary::TakeRecorderStopped = OnTakeRecorderStopped;
};

void UTakeRecorderBlueprintLibrary::SetOnTakeRecorderFinished(FOnTakeRecorderFinished OnTakeRecorderFinished)
{
	TakeRecorderBlueprintLibrary::TakeRecorderFinished = OnTakeRecorderFinished;
};

void UTakeRecorderBlueprintLibrary::SetOnTakeRecorderCancelled(FOnTakeRecorderCancelled OnTakeRecorderCancelled)
{
	TakeRecorderBlueprintLibrary::TakeRecorderCancelled = OnTakeRecorderCancelled;
};

void UTakeRecorderBlueprintLibrary::SetOnTakeRecorderMarkedFrameAdded(FOnTakeRecorderMarkedFrameAdded OnTakeRecorderMarkedFrameAdded)
{
	TakeRecorderBlueprintLibrary::TakeRecorderMarkedFrameAdded = OnTakeRecorderMarkedFrameAdded;
};

void UTakeRecorderBlueprintLibrary::OnTakeRecorderPreInitialize()
{
	TakeRecorderBlueprintLibrary::TakeRecorderPreInitialize.ExecuteIfBound();
}

void UTakeRecorderBlueprintLibrary::OnTakeRecorderStarted()
{
	TakeRecorderBlueprintLibrary::TakeRecorderStarted.ExecuteIfBound();
}

void UTakeRecorderBlueprintLibrary::OnTakeRecorderStopped()
{
	TakeRecorderBlueprintLibrary::TakeRecorderStopped.ExecuteIfBound();
}

void UTakeRecorderBlueprintLibrary::OnTakeRecorderFinished(ULevelSequence* InSequenceAsset)
{
	TakeRecorderBlueprintLibrary::TakeRecorderFinished.ExecuteIfBound(InSequenceAsset);
}

void UTakeRecorderBlueprintLibrary::OnTakeRecorderCancelled()
{
	TakeRecorderBlueprintLibrary::TakeRecorderCancelled.ExecuteIfBound();
}

void UTakeRecorderBlueprintLibrary::OnTakeRecorderMarkedFrameAdded(const FMovieSceneMarkedFrame& InMarkedFrame)
{
	TakeRecorderBlueprintLibrary::TakeRecorderMarkedFrameAdded.ExecuteIfBound(InMarkedFrame);
}

void UTakeRecorderBlueprintLibrary::SetTakeRecorderPanel(UTakeRecorderPanel* InNewPanel)
{
	if (TakeRecorderBlueprintLibrary::CurrentTakeRecorderPanel != InNewPanel)
	{
		if (TakeRecorderBlueprintLibrary::CurrentTakeRecorderPanel)
		{
			// Old panel is no longer valid
			TakeRecorderBlueprintLibrary::CurrentTakeRecorderPanel->ClosePanel();

			TakeRecorderBlueprintLibrary::CurrentTakeRecorderPanel->RemoveFromRoot();
			TakeRecorderBlueprintLibrary::CurrentTakeRecorderPanel = nullptr;
		}

		if (InNewPanel && InNewPanel->IsPanelOpen())
		{
			InNewPanel->AddToRoot();
			TakeRecorderBlueprintLibrary::CurrentTakeRecorderPanel = InNewPanel;
		}

		TakeRecorderBlueprintLibrary::TakeRecorderPanelChanged.ExecuteIfBound();
	}
}
