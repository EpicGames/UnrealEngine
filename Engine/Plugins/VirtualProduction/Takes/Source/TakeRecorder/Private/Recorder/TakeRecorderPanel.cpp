// Copyright Epic Games, Inc. All Rights Reserved.

#include "Recorder/TakeRecorderPanel.h"
#include "ITakeRecorderTabContent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderPanel)

bool UTakeRecorderPanel::IsPanelOpen() const
{
	return WeakTabContent.Pin().IsValid();
}

void UTakeRecorderPanel::InitializePanel(TWeakPtr<ITakeRecorderTabContent> InTabContent)
{
	WeakTabContent = InTabContent;
}

void UTakeRecorderPanel::ClosePanel()
{
	WeakTabContent = nullptr;
}

bool UTakeRecorderPanel::ValidateTabContent() const
{
	if (!IsPanelOpen())
	{
		FFrame::KismetExecutionMessage(TEXT("This take recorder panel is not open. Either re-call OpenTakeRecorderPanel or GetTakeRecorderPanel to get the current UI panel."), ELogVerbosity::Error);
		return false;
	}
	return true;
}

ETakeRecorderPanelMode UTakeRecorderPanel::GetMode() const
{
	if (!ValidateTabContent())
	{
		return ETakeRecorderPanelMode::NewRecording;
	}

	TOptional<ETakeRecorderPanelMode> Mode = WeakTabContent.Pin()->GetMode();
	return Mode.IsSet() ? Mode.GetValue() : ETakeRecorderPanelMode::NewRecording;
}

void UTakeRecorderPanel::SetupForRecording_TakePreset(UTakePreset* TakePresetAsset)
{
	if (ValidateTabContent())
	{
		WeakTabContent.Pin()->SetupForRecording(TakePresetAsset);
	}
}

void UTakeRecorderPanel::SetupForRecording_LevelSequence(ULevelSequence* LevelSequenceAsset)
{
	if (ValidateTabContent())
	{
		WeakTabContent.Pin()->SetupForRecording(LevelSequenceAsset);
	}
}

void UTakeRecorderPanel::SetupForRecordingInto_LevelSequence(ULevelSequence* LevelSequenceAsset)
{
	if (ValidateTabContent())
	{
		WeakTabContent.Pin()->SetupForRecordingInto(LevelSequenceAsset);
	}
}

void UTakeRecorderPanel::SetupForEditing(UTakePreset* TakePreset)
{
	if (ValidateTabContent())
	{
		WeakTabContent.Pin()->SetupForEditing(TakePreset);
	}
}

void UTakeRecorderPanel::SetupForViewing(ULevelSequence* LevelSequenceAsset)
{
	if (ValidateTabContent())
	{
		WeakTabContent.Pin()->SetupForViewing(LevelSequenceAsset);
	}
}

void UTakeRecorderPanel::ClearPendingTake()
{
	if (ValidateTabContent())
	{
		WeakTabContent.Pin()->ClearPendingTake();
	}
}

ULevelSequence* UTakeRecorderPanel::GetLevelSequence() const
{
	if (!ValidateTabContent())
	{
		return nullptr;
	}

	return WeakTabContent.Pin()->GetLevelSequence();

}

ULevelSequence* UTakeRecorderPanel::GetLastRecordedLevelSequence() const
{
	if (!ValidateTabContent())
	{
		return nullptr;
	}

	return WeakTabContent.Pin()->GetLastRecordedLevelSequence();
}

UTakeMetaData* UTakeRecorderPanel::GetTakeMetaData() const
{
	if (!ValidateTabContent())
	{
		return nullptr;
	}

	return WeakTabContent.Pin()->GetTakeMetaData();
}

FFrameRate UTakeRecorderPanel::GetFrameRate() const
{
	if (!ValidateTabContent())
	{
		return FFrameRate();
	}
	return WeakTabContent.Pin()->GetFrameRate();
}

void UTakeRecorderPanel::SetFrameRate(FFrameRate InFrameRate)
{
	if (ValidateTabContent())
	{
		WeakTabContent.Pin()->SetFrameRate(InFrameRate);
	}
}

void UTakeRecorderPanel::SetFrameRateFromTimecode(bool bInFromTimecode)
{
	if (!ValidateTabContent())
	{
		WeakTabContent.Pin()->SetFrameRateFromTimecode(bInFromTimecode);
	}
}

UTakeRecorderSources* UTakeRecorderPanel::GetSources() const
{
	if (!ValidateTabContent())
	{
		return nullptr;
	}

	return WeakTabContent.Pin()->GetSources();
}

void UTakeRecorderPanel::StartRecording() const
{
	if (ValidateTabContent())
	{
		return WeakTabContent.Pin()->StartRecording();
	}
}

void UTakeRecorderPanel::StopRecording() const
{
	if (ValidateTabContent())
	{
		return WeakTabContent.Pin()->StopRecording();
	}
}

bool UTakeRecorderPanel::CanStartRecording(FText& OutErrorText) const
{
	if (ValidateTabContent())
	{
		return WeakTabContent.Pin()->CanStartRecording(OutErrorText);
	}

	return false;
}


