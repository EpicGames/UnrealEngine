// Copyright Epic Games, Inc. All Rights Reserved.

#include "Recorder/TakeRecorderParameters.h"
#include "Recorder/TakeRecorder.h"
#include "TakeRecorderDirectoryHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderParameters)

FTakeRecorderUserParameters::FTakeRecorderUserParameters()
	: bMaximizeViewport(false)
	, CountdownSeconds(0.f)
	, EngineTimeDilation(1.f)
	, bResetPlayhead(true)
	, bStopAtPlaybackEnd(false)
	, bStopOnRollover(true)
	, bRemoveRedundantTracks(true)
	, ReduceKeysTolerance(KINDA_SMALL_NUMBER)
	, bSaveRecordedAssets(false)
	, bAutoLock(true)
	, bAutoSerialize(false)
{
	// Defaults for all user parameter structures
	// User defaults should be set in UTakeRecorderUserSettings
	// So as not to affect structs created with script
}

FTakeRecorderProjectParameters::FTakeRecorderProjectParameters()
	: SubSequenceDirectory(TEXT("{takeName}_Subscenes"))
	, RecordingClockSource(EUpdateClockSource::RelativeTimecode)
	, bStartAtCurrentTimecode(true)
	, bRecordTimecode(false)
	, bRecordSourcesIntoSubSequences(false)
	, bRecordToPossessable(false)
	, SpawnableOverwriteBehavior(ETakeRecorderSpawnableOverwriteBehavior::CreateNew)
	, AttachRecordBehaviour(ETakeRecorderAttachRecordBehaviour::Hierarchy)
	, bShowNotifications(true)
{
}

FString FTakeRecorderProjectParameters::GetResolvedRootTakeSaveDir() const
{
	return UE::TakeRecorder::Private::ResolvePathToProject(RootTakeSaveDir.Path);  
}

FString FTakeRecorderProjectParameters::GetTakeAssetPath() const
{
	return UE::TakeRecorder::Private::ResolvePathToProject(RootTakeSaveDir.Path) / TakeSaveDir;
}

FTakeRecorderParameters::FTakeRecorderParameters() 
	: TakeRecorderMode(ETakeRecorderMode::RecordNewSequence)
{}

bool FTakeRecorderSequenceParameters::IsValid() const
{
	return *this != FTakeRecorderSequenceParameters();
}

bool FTakeRecorderSequenceParameters::operator==(const FTakeRecorderSequenceParameters& Other) const
{
	return BasePreset == Other.BasePreset
		&& BaseSequence == Other.BaseSequence
		&& RecordIntoSequence == Other.RecordIntoSequence
		&& SequenceToView == Other.SequenceToView;
}

bool FTakeRecorderSequenceParameters::operator!=(const FTakeRecorderSequenceParameters& Other) const
{
	return !(*this == Other);
}
