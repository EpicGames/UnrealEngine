// Copyright Epic Games, Inc. All Rights Reserved.

#include "Recorder/TakeRecorderSubsystem.h"

#include "TakeRecorderSettings.h"
#include "Recorder/TakeRecorder.h"
#include "Recorder/TakeRecorderParameters.h"
#include "TakeRecorderSource.h"
#include "TakeRecorderSources.h"
#include "TakeRecorderSubsystemImplementation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderSubsystem)

#define LOCTEXT_NAMESPACE "TakeRecorderSubsystem"

void UTakeRecorderSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	Implementation = NewObject<UTakeRecorderSubsystemImplementation>();
	CastChecked<UTakeRecorderSubsystemImplementation>(Implementation.GetObject())->InitializeImplementation(this);
}

void UTakeRecorderSubsystem::Deinitialize()
{
	Super::Deinitialize();

	if (Implementation && Implementation.GetObject())
	{
		CastChecked<UTakeRecorderSubsystemImplementation>(Implementation.GetObject())->DeinitializeImplementation();
	}
}

void UTakeRecorderSubsystem::SetTargetSequence(const FTakeRecorderSequenceParameters& InData)
{
	Implementation->SetTargetSequence(InData);
}

void UTakeRecorderSubsystem::SetRecordIntoLevelSequence(ULevelSequence* LevelSequence)
{
	Implementation->SetRecordIntoLevelSequence(LevelSequence);
}

bool UTakeRecorderSubsystem::CanReviewLastRecording() const
{
	return Implementation->CanReviewLastRecording();
}

bool UTakeRecorderSubsystem::ReviewLastRecording()
{
	return Implementation->ReviewLastRecording();
}

bool UTakeRecorderSubsystem::StartRecording(bool bOpenSequencer, bool bShowErrorMessage)
{
	return Implementation->StartRecording(bOpenSequencer, bShowErrorMessage);
}

void UTakeRecorderSubsystem::StopRecording()
{
	Implementation->StopRecording();
}

void UTakeRecorderSubsystem::CancelRecording(bool bShowConfirmMessage)
{
	Implementation->CancelRecording(bShowConfirmMessage);
}

void UTakeRecorderSubsystem::ResetToPendingTake()
{
	Implementation->ResetToPendingTake();
}

void UTakeRecorderSubsystem::ClearPendingTake()
{
	Implementation->ClearPendingTake();
}

UTakePreset* UTakeRecorderSubsystem::GetPendingTake() const
{
	return Implementation->GetPendingTake();
}

void UTakeRecorderSubsystem::RevertChanges()
{
	Implementation->RevertChanges();
}

bool UTakeRecorderSubsystem::IsRecordingUsingExistingSequence() const
{
	return Implementation->IsRecordingUsingExistingSequence();
}

UTakeRecorderSource* UTakeRecorderSubsystem::AddSource(const TSubclassOf<UTakeRecorderSource> InSourceClass)
{
	return Implementation->AddSource(InSourceClass);
}

void UTakeRecorderSubsystem::RemoveSource(UTakeRecorderSource* InSource)
{
	Implementation->RemoveSource(InSource);
}

void UTakeRecorderSubsystem::ClearSources()
{
	Implementation->ClearSources();
}

UTakeRecorderSources* UTakeRecorderSubsystem::GetSources() const
{
	return Implementation->GetSources();
}

TArrayView<UTakeRecorderSource* const> UTakeRecorderSubsystem::GetAllSources() const
{
	return Implementation->GetAllSources();
}

TArray<UTakeRecorderSource*> UTakeRecorderSubsystem::GetAllSourcesCopy() const
{
	return Implementation->GetAllSourcesCopy();
}

UTakeRecorderSource* UTakeRecorderSubsystem::GetSourceByClass(const TSubclassOf<UTakeRecorderSource> InSourceClass) const
{
	return Implementation->GetSourceByClass(InSourceClass);
}

void UTakeRecorderSubsystem::AddSourceForActor(AActor* InActor, bool bReduceKeys, bool bShowProgress)
{
	Implementation->AddSourceForActor(InActor, bReduceKeys, bShowProgress);
}

void UTakeRecorderSubsystem::RemoveActorFromSources(AActor* InActor)
{
	Implementation->RemoveActorFromSources(InActor);
}

AActor* UTakeRecorderSubsystem::GetSourceActor(UTakeRecorderSource* InSource) const
{
	return Implementation->GetSourceActor(InSource);
}

ETakeRecorderState UTakeRecorderSubsystem::GetState() const
{
	return Implementation->GetState();
}

void UTakeRecorderSubsystem::SetTakeNumber(int32 InNewTakeNumber, bool bEmitChanged)
{
	Implementation->SetTakeNumber(InNewTakeNumber, bEmitChanged);
}

int32 UTakeRecorderSubsystem::GetNextTakeNumber(const FString& InSlate) const
{
	return Implementation->GetNextTakeNumber(InSlate);
}

void UTakeRecorderSubsystem::GetNumberOfTakes(const FString& InSlate, int32& OutMaxTake, int32& OutNumTakes) const
{
	Implementation->GetNumberOfTakes(InSlate, OutMaxTake, OutNumTakes);
}

void UTakeRecorderSubsystem::IncrementTakeNumber()
{
	Implementation->IncrementTakeNumber();
}

TArray<FAssetData> UTakeRecorderSubsystem::GetSlates(FName InPackagePath) const
{
	return Implementation->GetSlates(InPackagePath);
}

void UTakeRecorderSubsystem::SetSlateName(const FString& InSlateName, bool bEmitChanged)
{
	Implementation->SetSlateName(InSlateName, bEmitChanged);
}

bool UTakeRecorderSubsystem::MarkFrame()
{
	return Implementation->MarkFrame();
}

FFrameRate UTakeRecorderSubsystem::GetFrameRate() const
{
	return Implementation->GetFrameRate();
}

void UTakeRecorderSubsystem::SetFrameRate(FFrameRate InFrameRate)
{
	Implementation->SetFrameRate(InFrameRate);
}

void UTakeRecorderSubsystem::SetFrameRateFromTimecode()
{
	Implementation->SetFrameRateFromTimecode();
}

void UTakeRecorderSubsystem::ImportPreset(const FAssetData& InPreset)
{
	Implementation->ImportPreset(InPreset);
}

bool UTakeRecorderSubsystem::IsReviewing() const
{
	return Implementation->IsReviewing();
}

bool UTakeRecorderSubsystem::IsRecording() const
{
	return Implementation->IsRecording();
}

bool UTakeRecorderSubsystem::TryGetSequenceCountdown(float& OutValue) const
{
	return Implementation->TryGetSequenceCountdown(OutValue);
}

void UTakeRecorderSubsystem::SetSequenceCountdown(float InSeconds)
{
	Implementation->SetSequenceCountdown(InSeconds);
}

TArray<UObject*> UTakeRecorderSubsystem::GetSourceRecordSettings(UTakeRecorderSource* InSource) const
{
	return Implementation->GetSourceRecordSettings(InSource);
}

FTakeRecorderParameters UTakeRecorderSubsystem::GetGlobalRecordSettings() const
{
	return Implementation->GetGlobalRecordSettings();
}

void UTakeRecorderSubsystem::SetGlobalRecordSettings(const FTakeRecorderParameters& InParameters)
{
	Implementation->SetGlobalRecordSettings(InParameters);
}

UTakeMetaData* UTakeRecorderSubsystem::GetTakeMetaData() const
{
	return Implementation->GetTakeMetaData();
}

ULevelSequence* UTakeRecorderSubsystem::GetLevelSequence() const
{
	return Implementation->GetLevelSequence();
}

ULevelSequence* UTakeRecorderSubsystem::GetSuppliedLevelSequence() const
{
	return Implementation->GetSuppliedLevelSequence();
}

ULevelSequence* UTakeRecorderSubsystem::GetRecordingLevelSequence() const
{
	return Implementation->GetRecordingLevelSequence();
}

ULevelSequence* UTakeRecorderSubsystem::GetRecordIntoLevelSequence() const
{
	return Implementation->GetRecordIntoLevelSequence();
}

ULevelSequence* UTakeRecorderSubsystem::GetLastRecordedLevelSequence() const
{
	return Implementation->GetLastRecordedLevelSequence();
}

UTakePreset* UTakeRecorderSubsystem::GetTransientPreset() const
{
	return Implementation->GetTransientPreset();
}

ETakeRecorderMode UTakeRecorderSubsystem::GetTakeRecorderMode() const
{
	return Implementation->GetTakeRecorderMode();
}

UTakeRecorderNamingTokensData* UTakeRecorderSubsystem::GetNamingTokensData() const
{
	return Implementation->GetNamingTokensData();
}

bool UTakeRecorderSubsystem::HasPendingChanges() const
{
	return Implementation->HasPendingChanges();
}

UClass* UTakeRecorderSubsystem::GetPresetTargetRecordClass() const
{
	return Implementation->GetPresetTargetRecordClass();
}

void UTakeRecorderSubsystem::SetPresetTargetRecordClass(const TSubclassOf<ULevelSequence> InClass)
{
	Implementation->SetPresetTargetRecordClass(InClass);
}

UObject* UTakeRecorderSubsystem::GetAdditionalSettings(TSubclassOf<UObject> SettingsType)
{
	if (SettingsType)
	{
		const TArray<TWeakObjectPtr<UObject>> AdditionalSettingsObjects  = GetMutableDefault<UTakeRecorderProjectSettings>()->AdditionalSettings;
		for (const TWeakObjectPtr<UObject>& AdditionalSettingsObject : AdditionalSettingsObjects)
		{
			if (AdditionalSettingsObject.IsValid() && AdditionalSettingsObject->GetClass() == SettingsType)
			{
				return AdditionalSettingsObject.Get();
			}
		}
	}
	return nullptr;
}

FOnTakeRecordingInitialized& UTakeRecorderSubsystem::GetOnRecordingPreInitializedEvent()
{
	return OnRecordingPreInitializeEvent;
}

FOnTakeRecordingInitialized& UTakeRecorderSubsystem::GetOnRecordingInitializedEvent()
{
	return ORecordingInitializedEvent;
}

FOnTakeRecordingStarted& UTakeRecorderSubsystem::GetOnRecordingStartedEvent()
{
	return OnRecordingStartedEvent;
}

FOnTakeRecordingStopped& UTakeRecorderSubsystem::GetOnRecordingStoppedEvent()
{
	return OnRecordingStoppedEvent;
}

FOnTakeRecordingFinished& UTakeRecorderSubsystem::GetOnRecordingFinishedEvent()
{
	return OnRecordingFinishedEvent;
}

FOnTakeRecordingCancelled& UTakeRecorderSubsystem::GetOnRecordingCancelledEvent()
{
	return OnRecordingCancelledEvent;
}

UTakeRecorderSources::FOnSourceAdded& UTakeRecorderSubsystem::GetOnRecordingSourceAddedEvent()
{
	return OnRecordingSourceAddedEvent;
}

UTakeRecorderSources::FOnSourceRemoved& UTakeRecorderSubsystem::GetOnRecordingSourceRemovedEvent()
{
	return OnRecordingSourceRemovedEvent;
}

UTakeRecorderSources::FOnSourceModified& UTakeRecorderSubsystem::GetOnRecordingSourceModifiedEvent()
{
	return OnRecordingSourceModifiedEvent;
}

FOnPendingTakeCleared& UTakeRecorderSubsystem::GetOnPendingTakeClearedEvent()
{
	return OnPendingTakeClearedEvent;
}

#undef LOCTEXT_NAMESPACE
