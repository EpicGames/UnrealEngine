// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderSubsystemImplementation.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ITakeRecorderModule.h"
#include "ITakeRecorderNamingTokensModule.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "Recorder/TakeRecorder.h"
#include "Recorder/TakeRecorderBlueprintLibrary.h"
#include "Recorder/TakeRecorderParameters.h"
#include "Recorder/TakeRecorderSubsystem.h"
#include "TakeMetaData.h"
#include "TakePresetSettings.h"
#include "TakeRecorderSettings.h"
#include "TakeRecorderSource.h"
#include "TakeRecorderSources.h"
#include "TakesCoreBlueprintLibrary.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "TakeRecorderNamingTokensData.h"
#include "ITakeRecorderSourcesManager.h"

#if WITH_EDITOR
#include "Dialog/SMessageDialog.h"
#include "ILevelSequenceEditorToolkit.h"
#include "NamingTokensEngineSubsystem.h"
#include "NamingTokens.h"
#include "ScopedTransaction.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderSubsystemImplementation)

#define LOCTEXT_NAMESPACE "TakeRecorderSubsystemImplementation"

#define WITH_OWNING_SUBSYSTEM(CALL)                                               \
do {                                                                              \
	if (UTakeRecorderSubsystem* OwningSubsystem = OwningSubsystemWeakPtr.Get()) { \
		OwningSubsystem->CALL;                                                    \
	}                                                                             \
} while (0)

ETickableTickType UTakeRecorderSubsystemImplementation::GetTickableTickType() const
{
	// This is to prevent registration until we call SetTargetSequence.
	return ETickableTickType::Never;
}

UWorld* UTakeRecorderSubsystemImplementation::GetTickableGameObjectWorld() const
{
	return FTickableGameObject::GetTickableGameObjectWorld();
}

bool UTakeRecorderSubsystemImplementation::IsTickable() const
{
	return bHasTargetSequenceBeenSet;
}

bool UTakeRecorderSubsystemImplementation::IsTickableWhenPaused() const
{
	return FTickableGameObject::IsTickableWhenPaused();
}

bool UTakeRecorderSubsystemImplementation::IsTickableInEditor() const
{
	return true;
}

void UTakeRecorderSubsystemImplementation::Tick(float DeltaTime)
{
	CacheMetaData();
}

TStatId UTakeRecorderSubsystemImplementation::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UTakeRecorderSubsystemImplementation, STATGROUP_Tickables);
}

void UTakeRecorderSubsystemImplementation::InitializeImplementation(UTakeRecorderSubsystem* OwningSubsystem)
{
	check(OwningSubsystem);
	OwningSubsystemWeakPtr = OwningSubsystem;
	
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	OnAssetRegistryFilesLoadedHandle = AssetRegistryModule.Get().OnFilesLoaded().AddUObject(this, &UTakeRecorderSubsystemImplementation::OnAssetRegistryFilesLoaded);
	OnRecordingInitializedHandle = UTakeRecorder::OnRecordingInitialized().AddUObject(this, &UTakeRecorderSubsystemImplementation::OnRecordingInitialized);

	UTakeRecorderSources::OnSourceAdded().AddUObject(this, &UTakeRecorderSubsystemImplementation::OnSourceAdded);
	UTakeRecorderSources::OnSourceRemoved().AddUObject(this, &UTakeRecorderSubsystemImplementation::OnSourceRemoved);
	UTakeRecorderSources::OnSourceModified().AddUObject(this, &UTakeRecorderSubsystemImplementation::OnSourceModified);

	UTakeMetaData::OnTakeSlateChanged().AddUObject(this, &UTakeRecorderSubsystemImplementation::OnTakeSlateChanged);
	UTakeMetaData::OnTakeNumberChanged().AddUObject(this, &UTakeRecorderSubsystemImplementation::OnTakeNumberChanged);

	NamingTokensData = NewObject<UTakeRecorderNamingTokensData>();
	NamingTokensData->SetFlags(RF_Transactional);

	BindToNamingTokenEvents();
	
	// If the UTakePresetSettings::TargetRecordClass changes, the level sequence in the TransientPreset must be regenerated.
	UTakePresetSettings::Get()->OnSettingsChanged().AddUObject(this, &UTakeRecorderSubsystemImplementation::OnTakePresetSettingsChanged);
}

void UTakeRecorderSubsystemImplementation::DeinitializeImplementation()
{
	UTakeRecorder::OnRecordingInitialized().Remove(OnRecordingInitializedHandle);

	UTakeRecorderSources::OnSourceAdded().RemoveAll(this);
	UTakeRecorderSources::OnSourceRemoved().RemoveAll(this);
	UTakeRecorderSources::OnSourceModified().RemoveAll(this);

	UTakeMetaData::OnTakeSlateChanged().RemoveAll(this);
	UTakeMetaData::OnTakeNumberChanged().RemoveAll(this);

	UTakePresetSettings::Get()->OnSettingsChanged().RemoveAll(this);
	
#if WITH_EDITOR
	FEditorDelegates::OnPreForceDeleteObjects.Remove(OnPreForceDeleteObjectsHandle);
	OnPreForceDeleteObjectsHandle.Reset();
#endif // WITH_EDITOR
	
	if (const FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry"))
	{
		if (IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet())
		{
			AssetRegistry->OnFilesLoaded().Remove(OnAssetRegistryFilesLoadedHandle);
		}
	}

	if (ITakeRecorderModule::IsAvailable())
	{
		ITakeRecorderModule::Get().GetLastLevelSequenceProvider().Unbind();
	}

	UnbindNamingTokensEvents();

	bHasTargetSequenceBeenSet = false;
	SetTickableTickType(ETickableTickType::Never);

	OwningSubsystemWeakPtr.Reset();
}

void UTakeRecorderSubsystemImplementation::SetTargetSequence(const FTakeRecorderSequenceParameters& InData)
{
	TargetSequenceData = InData;

	ITakeRecorderModule& TakeRecorderModule = ITakeRecorderModule::Get();
	TakeRecorderModule.GetLastLevelSequenceProvider().Unbind();
	TakeRecorderModule.GetLastLevelSequenceProvider().BindUObject(this, &UTakeRecorderSubsystemImplementation::SetLastLevelSequence);
	
	// If a recording is currently underway, initialize to that now
	if (const UTakeRecorder* ActiveRecorder = UTakeRecorder::GetActiveRecorder())
	{
		RecordingLevelSequence = ActiveRecorder->GetSequence();
	}
	else
	{
		RecordingLevelSequence = nullptr;
	}
	
	TransientPreset = AllocateTransientPreset();
	
	bRecordingUsingExistingSequence = false;
	
	// Copy the base preset into the transient preset if it was provided.
	// We do this first so that anything that asks for its Level Sequence
	// on construction gets the right one
	if (InData.BasePreset)
	{
		TransientPreset->CopyFrom(InData.BasePreset);
	}
	else if (InData.BaseSequence)
	{
		bRecordingUsingExistingSequence = true;
		TransientPreset->CopyFrom(InData.BaseSequence);

		ULevelSequence* LevelSequence = TransientPreset->GetLevelSequence();
		ITakeRecorderSourcesManager::GetChecked().CopySources(InData.BaseSequence, LevelSequence);

	#if WITH_EDITOR
		if (LevelSequence)
		{
			LevelSequence->GetMovieScene()->SetReadOnly(false);
		}

		if (UTakeMetaData* TakeMetaDataLevelSequence = LevelSequence ? LevelSequence->FindMetaData<UTakeMetaData>() : nullptr)
		{
			TakeMetaDataLevelSequence->Unlock();
			TakeMetaDataLevelSequence->SetTimestamp(FDateTime(0));
		}
	#endif // WITH_EDITOR
	}
	else if (InData.RecordIntoSequence)
	{
		bRecordingUsingExistingSequence = true;
		SetRecordIntoLevelSequence(InData.RecordIntoSequence);
	}
	else if (InData.SequenceToView)
	{
		SuppliedLevelSequence = InData.SequenceToView;
		RecordIntoLevelSequence = nullptr; // We may have switched from recording to reviewing.
	}
	
	bAutoApplyTakeNumber = true;
	
	CacheMetaData();
	
	if (TakeMetaData && !TakeMetaData->IsLocked())
	{
		const int32 NextTakeNumber = GetNextTakeNumber(TakeMetaData->GetSlate());
		if (NextTakeNumber != TakeMetaData->GetTakeNumber())
		{
			TakeMetaData->SetTakeNumber(NextTakeNumber);
		}
	}
	
	/** Clear the dirty flag since the preset was just initialized. */
	TransientPreset->GetOutermost()->SetDirtyFlag(false);

	bHasTargetSequenceBeenSet = true;
	SetTickableTickType(ETickableTickType::Conditional);
}

void UTakeRecorderSubsystemImplementation::SetRecordIntoLevelSequence(ULevelSequence* LevelSequence)
{
	SuppliedLevelSequence = nullptr;
	RecordIntoLevelSequence = LevelSequence;

#if WITH_EDITOR
	if (RecordIntoLevelSequence)
	{
		RecordIntoLevelSequence->GetMovieScene()->SetReadOnly(false);
	}

	if (UTakeMetaData* LocalTakeMetaData = RecordIntoLevelSequence ? RecordIntoLevelSequence->FindOrAddMetaData<UTakeMetaData>() : nullptr)
	{
		// Set up take metadata to match this level sequence's info, ie. match the frame rate, use the level sequence name as the slate
		LocalTakeMetaData->Unlock();
		LocalTakeMetaData->SetTimestamp(FDateTime(0));
		LocalTakeMetaData->SetSlate(LevelSequence->GetName());
		LocalTakeMetaData->SetTakeNumber(0);
		LocalTakeMetaData->SetFrameRate(LevelSequence->GetMovieScene()->GetDisplayRate());
		LocalTakeMetaData->SetFrameRateFromTimecode(false);
	}
#endif // WITH_EDITOR
}

bool UTakeRecorderSubsystemImplementation::CanReviewLastRecording() const
{
	if (GetSuppliedLevelSequence() != nullptr)
	{
		return false;
	}
	ITakeRecorderModule& TakeRecorderModule = ITakeRecorderModule::Get();
	FCanReviewLastRecordedLevelSequence& CanReview = TakeRecorderModule.GetCanReviewLastRecordedLevelSequenceDelegate();
	if (CanReview.IsBound())
	{
		return CanReview.Execute();
	}
	
	return true;
}

bool UTakeRecorderSubsystemImplementation::ReviewLastRecording()
{
	if (LastRecordedLevelSequence.IsValid())
	{
		SuppliedLevelSequence = LastRecordedLevelSequence.Get();
		return true;
	}

	return false;
}

bool UTakeRecorderSubsystemImplementation::StartRecording(bool bOpenSequencer, bool bShowErrorMessage)
{
	static bool bStartedRecording = false;

	if (bStartedRecording || IsReviewing() /** Shouldn't be starting a recording if we are reviewing. */)
	{
		return false;
	}

	if (!GIsEditor)
	{
		// Sequencer requires an editor attached.
		bOpenSequencer = false;
	}

	TGuardValue<bool> ReentrantGuard(bStartedRecording, true);

	ULevelSequence*       LevelSequence = GetLevelSequence();
	UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindSources(LevelSequence);
	
	if (LevelSequence && Sources)
	{
		FTakeRecorderParameters Parameters;
		Parameters.User    = GetDefault<UTakeRecorderUserSettings>()->Settings;
		Parameters.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;
		Parameters.TakeRecorderMode = GetTakeRecorderMode();
		Parameters.StartFrame = LevelSequence->GetMovieScene()->GetPlaybackRange().GetLowerBoundValue();
		Parameters.bOpenSequencer = bOpenSequencer;

		FText ErrorText = LOCTEXT("UnknownError", "An unknown error occurred when trying to start recording");

	#if WITH_EDITOR
		if (GIsEditor)
		{
			IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(LevelSequence, false);
			ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);

			if (LevelSequenceEditor && LevelSequenceEditor->GetSequencer())
			{
				// If not resetting the playhead, store the current time as the start frame for recording. 
				// This will ultimately be the start of the playback range and the recording will begin from that time.
				if (!Parameters.User.bResetPlayhead)
				{
					Parameters.StartFrame = LevelSequenceEditor->GetSequencer()->GetLocalTime().Time.FrameNumber;
				}
			}
		}
	#endif // WITH_EDITOR

		UTakeRecorder* NewRecorder = NewObject<UTakeRecorder>(GetTransientPackage(), NAME_None, RF_Transient);

		if (!NewRecorder->Initialize(LevelSequence, Sources, TakeMetaData, Parameters, &ErrorText))
		{
			if (bShowErrorMessage && ensure(!ErrorText.IsEmpty()))
			{
				FNotificationInfo Info(ErrorText);
				Info.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
			}
			return false;
		}

		OnRecordingStarted(NewRecorder);
		
		return true;
	}
	
	return false;
}

void UTakeRecorderSubsystemImplementation::StopRecording()
{
	if (UTakeRecorder* CurrentRecording = UTakeRecorder::GetActiveRecorder())
	{
		CurrentRecording->Stop();
	}
}

void UTakeRecorderSubsystemImplementation::CancelRecording(bool bShowConfirmMessage)
{
	if (UTakeRecorder* CurrentRecording = UTakeRecorder::GetActiveRecorder())
	{
		auto DoCancel = [](const TWeakObjectPtr<UTakeRecorder>& WeakRecording)
			{
				if (WeakRecording.IsValid())
				{
					WeakRecording.Get()->Cancel();
				}
			};

	#if WITH_EDITOR
		if (bShowConfirmMessage)
		{
			const TSharedRef<SMessageDialog> ConfirmDialog = SNew(SMessageDialog)
				.Title(FText(LOCTEXT("ConfirmCancelRecordingTitle", "Cancel Recording?")))
				.Message(LOCTEXT("ConfirmCancelRecording", "Are you sure you want to cancel the current recording?"))
				.Buttons({
					SCustomDialog::FButton(LOCTEXT("Yes", "Yes"), FSimpleDelegate::CreateLambda([DoCancel, WeakRecording = TWeakObjectPtr<UTakeRecorder>(CurrentRecording)]()
						{ 
							DoCancel(WeakRecording);
						})),
					SCustomDialog::FButton(LOCTEXT("No", "No"))
				});

			// Non modal so that the recording continues to update
			ConfirmDialog->Show();
		}
		else
	#endif // WITH_EDITOR
		{
			DoCancel(TWeakObjectPtr<UTakeRecorder>(CurrentRecording));
		}
	}
}

void UTakeRecorderSubsystemImplementation::ResetToPendingTake()
{
	if (IsReviewing())
	{
		LastRecordedLevelSequence = SuppliedLevelSequence;
	}
	
	// We purposely don't reset bRecordingUsingExistingSequence here, because the pending take may still be considered recorded over and
	// certain workflows from the panel rely on this. ClearPendingTake may be called if we want to reset bRecordingUsingExistingSequence.
	
	SuppliedLevelSequence = nullptr;
	RecordIntoLevelSequence = nullptr;
	
	TransientPreset = AllocateTransientPreset();
}

void UTakeRecorderSubsystemImplementation::ClearPendingTake()
{
	bRecordingUsingExistingSequence = false;
	
	if (IsReviewing())
	{
		LastRecordedLevelSequence = SuppliedLevelSequence;
	}

	UTakeRecorderSources* BaseSources = nullptr;

	if (const ULevelSequence* CurrentLevelSequence = GetLevelSequence())
	{
		BaseSources = ITakeRecorderSourcesManager::GetChecked().FindSources(CurrentLevelSequence);
	}

	SuppliedLevelSequence = nullptr;
	RecordIntoLevelSequence = nullptr;

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("ClearPendingTake_Transaction", "Clear Pending Take"));
#endif // WITH_EDITOR

	TransientPreset->Modify();
	TransientPreset->CreateLevelSequence();

	ULevelSequence* LevelSequence = TransientPreset->GetLevelSequence();
	if (LevelSequence && BaseSources)
	{
		LevelSequence->CopyMetaData(BaseSources);
	}

	WITH_OWNING_SUBSYSTEM(GetOnPendingTakeClearedEvent().Broadcast());
	WITH_OWNING_SUBSYSTEM(PendingTakeCleared.Broadcast());
}

UTakePreset* UTakeRecorderSubsystemImplementation::GetPendingTake() const
{
	const ITakeRecorderModule& TakeRecorderModule = ITakeRecorderModule::Get();
	return TakeRecorderModule.GetPendingTake();
}

void UTakeRecorderSubsystemImplementation::RevertChanges()
{
	UTakePreset* PresetOrigin = GetTakeMetaData()->GetPresetOrigin();

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("RevertChanges_Transaction", "Revert Changes"));
#endif // WITH_EDITOR

	TransientPreset->Modify();
	TransientPreset->CopyFrom(PresetOrigin);
	TransientPreset->GetOutermost()->SetDirtyFlag(false);
}

bool UTakeRecorderSubsystemImplementation::IsRecordingUsingExistingSequence() const
{
	return GetTakeRecorderMode() == ETakeRecorderMode::RecordIntoSequence || bRecordingUsingExistingSequence;
}

UTakeRecorderSource* UTakeRecorderSubsystemImplementation::AddSource(const TSubclassOf<UTakeRecorderSource> InSourceClass)
{
	UTakeRecorderSources* Sources = GetSources();

	if (*InSourceClass && Sources)
	{
	#if WITH_EDITOR
		FScopedTransaction Transaction(FText::Format(LOCTEXT("AddNewSource_Transaction", "Add New {0} Source"), InSourceClass->GetDisplayNameText()));
	#endif // WITH_EDITOR
		Sources->Modify();

		return Sources->AddSource(InSourceClass);
	}

	return nullptr;
}

void UTakeRecorderSubsystemImplementation::RemoveSource(UTakeRecorderSource* InSource)
{
	UTakeRecorderSources* Sources = GetSources();
	if (Sources && InSource)
	{
	#if WITH_EDITOR
		FScopedTransaction Transaction(FText::Format(LOCTEXT("RemoveSource_Transaction", "Remove {0} Source"), InSource->GetDisplayText()));
	#endif // WITH_EDITOR
		Sources->Modify();
		Sources->RemoveSource(InSource);
	}
}

void UTakeRecorderSubsystemImplementation::ClearSources()
{
	if (UTakeRecorderSources* Sources = GetSources())
	{
	#if WITH_EDITOR
		FScopedTransaction Transaction(LOCTEXT("ClearSources_Transaction", "Clear Sources"));
	#endif // WITH_EDITOR
		Sources->Modify();
		for (UTakeRecorderSource* Source : Sources->GetSourcesCopy())
		{
			Sources->RemoveSource(Source);
		}
	}
}

UTakeRecorderSources* UTakeRecorderSubsystemImplementation::GetSources() const
{
	ULevelSequence* LevelSequence = GetLevelSequence();
	return ITakeRecorderSourcesManager::GetChecked().FindOrAddSources(LevelSequence);
}

TArrayView<UTakeRecorderSource* const> UTakeRecorderSubsystemImplementation::GetAllSources() const
{
	if (const UTakeRecorderSources* Sources = GetSources())
	{
		return Sources->GetSources();
	}

	return {};
}

TArray<UTakeRecorderSource*> UTakeRecorderSubsystemImplementation::GetAllSourcesCopy() const
{
	return TArray<UTakeRecorderSource*>(GetAllSources());
}

UTakeRecorderSource* UTakeRecorderSubsystemImplementation::GetSourceByClass(const TSubclassOf<UTakeRecorderSource> InSourceClass) const
{
	const TArrayView<UTakeRecorderSource* const> Sources = GetAllSources();
	for (UTakeRecorderSource* Source : Sources)
	{
		if (Source->IsA(InSourceClass))
		{
			return Source;
		}
	}

	return nullptr;
}

void UTakeRecorderSubsystemImplementation::AddSourceForActor(AActor* InActor, bool bReduceKeys, bool bShowProgress)
{
	if (UTakeRecorderSources* Sources = GetSources())
	{
		ITakeRecorderModule& TakeRecorderModule = ITakeRecorderModule::Get();
		TakeRecorderModule.GetSourcesExtensionData().OnAddActorSource.ExecuteIfBound(Sources, { InActor }, bReduceKeys, bShowProgress);
	}
}

void UTakeRecorderSubsystemImplementation::RemoveActorFromSources(AActor* InActor)
{
	if (UTakeRecorderSources* Sources = GetSources())
	{
		ITakeRecorderModule& TakeRecorderModule = ITakeRecorderModule::Get();
		TakeRecorderModule.GetSourcesExtensionData().OnRemoveActorSource.ExecuteIfBound(Sources, { InActor });
	}
}

AActor* UTakeRecorderSubsystemImplementation::GetSourceActor(UTakeRecorderSource* InSource) const
{
	ITakeRecorderModule& TakeRecorderModule = ITakeRecorderModule::Get();
	if (TakeRecorderModule.GetSourcesExtensionData().OnGetSourceActor.IsBound())
	{
		return TakeRecorderModule.GetSourcesExtensionData().OnGetSourceActor.Execute(InSource);
	}

	return nullptr;
}

ETakeRecorderState UTakeRecorderSubsystemImplementation::GetState() const
{
	if (const UTakeRecorder* Recorder = UTakeRecorder::GetActiveRecorder())
	{
		return Recorder->GetState();
	}

	return ETakeRecorderState::PreInitialization;
}

void UTakeRecorderSubsystemImplementation::SetTakeNumber(int32 InNewTakeNumber, bool bEmitChanged)
{
	if (TakeMetaData)
	{
	#if WITH_EDITOR
		FScopedTransaction Transaction(LOCTEXT("SetTake_Transaction", "Set Take Number"));
	#endif // WITH_EDITOR
		TakeMetaData->Modify();
		TakeMetaData->SetTakeNumber(InNewTakeNumber, bEmitChanged);
		bAutoApplyTakeNumber = false;
	}
}

int32 UTakeRecorderSubsystemImplementation::GetNextTakeNumber(const FString& InSlate) const
{
	return UTakesCoreBlueprintLibrary::ComputeNextTakeNumber(InSlate);
}

void UTakeRecorderSubsystemImplementation::GetNumberOfTakes(const FString& InSlate, int32& OutMaxTake, int32& OutNumTakes) const
{
	int32 MaxTake = 0;

	TArray<FAssetData> Takes = UTakesCoreBlueprintLibrary::FindTakes(InSlate);

	for (const FAssetData& Asset : Takes)
	{
		FAssetDataTagMapSharedView::FFindTagResult TakeNumberTag = Asset.TagsAndValues.FindTag(UTakeMetaData::AssetRegistryTag_TakeNumber);

		int32 ThisTakeNumber = 0;
		if (TakeNumberTag.IsSet() && LexTryParseString(ThisTakeNumber, *TakeNumberTag.GetValue()))
		{
			MaxTake = FMath::Max(MaxTake, ThisTakeNumber);
		}
	}

	OutMaxTake = MaxTake;
	OutNumTakes = Takes.Num();
}

void UTakeRecorderSubsystemImplementation::IncrementTakeNumber()
{
	if (TransientTakeMetaData)
	{
		// Increment the transient take meta data if necessary
		int32 NextTakeNumber = GetNextTakeNumber(TransientTakeMetaData->GetSlate());

		if (TransientTakeMetaData->GetTakeNumber() != NextTakeNumber)
		{
			TransientTakeMetaData->SetTakeNumber(NextTakeNumber);
		}
	}
	
	// Update the preset take number at the end of recording
	
	if (const ULevelSequence* LevelSequence = TransientPreset ? TransientPreset->GetLevelSequence() : nullptr)
	{
	#if WITH_EDITOR
		if (UTakeMetaData* MetaData = LevelSequence->FindMetaData<UTakeMetaData>())
		{
			const int32 NextTakeNumber = GetNextTakeNumber(MetaData->GetSlate());
			MetaData->SetTakeNumber(NextTakeNumber);
		}
	#endif // WITH_EDITOR
	}

	bAutoApplyTakeNumber = true;
}

TArray<FAssetData> UTakeRecorderSubsystemImplementation::GetSlates(FName InPackagePath) const
{
	const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();

	FARFilter Filter;
	Filter.ClassPaths.Add(ULevelSequence::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;
	Filter.TagsAndValues.Add(UTakeMetaData::AssetRegistryTag_Slate);
	if (!InPackagePath.IsNone())
	{
		Filter.PackagePaths.Add(InPackagePath);
		Filter.bRecursivePaths = true;
	}

	TArray<FAssetData> AllAssets;
	AssetRegistry.GetAssets(Filter, AllAssets);
	
	return AllAssets;
}

void UTakeRecorderSubsystemImplementation::SetSlateName(const FString& InSlateName, bool bEmitChanged)
{
	if (TakeMetaData && TakeMetaData->GetSlate() != InSlateName)
	{
	#if WITH_EDITOR
		FScopedTransaction Transaction(LOCTEXT("SetSlate_Transaction", "Set Take Slate"));
	#endif // WITH_EDITOR
		TakeMetaData->Modify();

		TakeMetaData->SetSlate(InSlateName, bEmitChanged);

		// Compute the correct starting take number
		const int32 NextTakeNumber = GetNextTakeNumber(TakeMetaData->GetSlate());
		if (NextTakeNumber != TakeMetaData->GetTakeNumber())
		{
			TakeMetaData->SetTakeNumber(NextTakeNumber, bEmitChanged);
		}
	}
}

bool UTakeRecorderSubsystemImplementation::MarkFrame()
{
	if (UTakeRecorderBlueprintLibrary::IsRecording() && TakeMetaData)
	{
		const FFrameRate FrameRate = TakeMetaData->GetFrameRate();

		const FTimespan RecordingDuration = FDateTime::UtcNow() - TakeMetaData->GetTimestamp();

		const FFrameNumber ElapsedFrame = FFrameNumber(static_cast<int32>(FrameRate.AsDecimal() * RecordingDuration.GetTotalSeconds()));

		const ULevelSequence* LevelSequence = GetLevelSequence();
		if (!LevelSequence)
		{
			return false;
		}

		UMovieScene* MovieScene = LevelSequence->GetMovieScene();
		if (!MovieScene)
		{
			return false;
		}

		const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		const FFrameRate TickResolution = MovieScene->GetTickResolution();

		FMovieSceneMarkedFrame MarkedFrame;

		const UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindSources(LevelSequence);
		if (Sources && Sources->GetSettings().bStartAtCurrentTimecode)
		{
			MarkedFrame.FrameNumber = FFrameRate::TransformTime(FFrameTime(FApp::GetTimecode().ToFrameNumber(DisplayRate)), DisplayRate, TickResolution).FloorToFrame();
		}
		else
		{
			MarkedFrame.FrameNumber = ConvertFrameTime(ElapsedFrame, DisplayRate, TickResolution).CeilToFrame();
		}

		const int32 MarkedFrameIndex = MovieScene->AddMarkedFrame(MarkedFrame);
		UTakeRecorderBlueprintLibrary::OnTakeRecorderMarkedFrameAdded(MovieScene->GetMarkedFrames()[MarkedFrameIndex]);
		if (OwningSubsystemWeakPtr.IsValid())
		{
			OwningSubsystemWeakPtr->TakeRecorderMarkedFrameAdded.Broadcast(MovieScene->GetMarkedFrames()[MarkedFrameIndex]);
		}

		return true;
	}

	return false;
}

FFrameRate UTakeRecorderSubsystemImplementation::GetFrameRate() const
{
	if (TakeMetaData)
	{
		return TakeMetaData->GetFrameRate();
	}

	return FFrameRate();
}

void UTakeRecorderSubsystemImplementation::SetFrameRate(FFrameRate InFrameRate)
{
	SetFrameRateImpl(InFrameRate, false);
}

void UTakeRecorderSubsystemImplementation::SetFrameRateFromTimecode()
{
	SetFrameRateImpl(FApp::GetTimecodeFrameRate(), true);
}

void UTakeRecorderSubsystemImplementation::ImportPreset(const FAssetData& InPreset)
{
	SuppliedLevelSequence = nullptr;
	RecordIntoLevelSequence = nullptr;

	UTakePreset* Take = CastChecked<UTakePreset>(InPreset.GetAsset());
#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("ImportPreset_Transaction", "Import Take Preset"));
#endif // WITH_EDITOR

	TransientPreset->Modify();
	TransientPreset->CopyFrom(Take);
	TransientPreset->GetOutermost()->SetDirtyFlag(false);

	GetTakeMetaData()->SetPresetOrigin(Take);
}

bool UTakeRecorderSubsystemImplementation::IsReviewing() const
{
	return !IsRecording() && SuppliedLevelSequence &&
		(TakeMetaData && TakeMetaData->Recorded()
			&& GetTakeRecorderMode() != ETakeRecorderMode::RecordIntoSequence);
}

bool UTakeRecorderSubsystemImplementation::IsRecording() const
{
	return UTakeRecorderBlueprintLibrary::GetActiveRecorder() ? true : false;
}

bool UTakeRecorderSubsystemImplementation::TryGetSequenceCountdown(float& OutValue) const
{
	const UTakeRecorder* CurrentRecording = UTakeRecorder::GetActiveRecorder();
	const bool           bIsCountingDown  = CurrentRecording && CurrentRecording->GetState() == ETakeRecorderState::CountingDown;
	
	OutValue = bIsCountingDown ? CurrentRecording->GetCountdownSeconds() : 0.f;
	return bIsCountingDown;
}

void UTakeRecorderSubsystemImplementation::SetSequenceCountdown(float InSeconds)
{
	if (UTakeRecorder* CurrentRecording = UTakeRecorder::GetActiveRecorder())
	{
		CurrentRecording->SetCountdown(InSeconds);
	}
}

TArray<UObject*> UTakeRecorderSubsystemImplementation::GetSourceRecordSettings(UTakeRecorderSource* InSource) const
{
	return InSource->GetAdditionalSettingsObjects();
}

FTakeRecorderParameters UTakeRecorderSubsystemImplementation::GetGlobalRecordSettings() const
{
	return UTakeRecorderBlueprintLibrary::GetDefaultParameters();
}

void UTakeRecorderSubsystemImplementation::SetGlobalRecordSettings(const FTakeRecorderParameters& InParameters)
{
	UTakeRecorderBlueprintLibrary::SetDefaultParameters(InParameters);
}

UTakeMetaData* UTakeRecorderSubsystemImplementation::GetTakeMetaData() const
{
	return TakeMetaData;
}

ULevelSequence* UTakeRecorderSubsystemImplementation::GetLevelSequence() const
{
	if (SuppliedLevelSequence)
	{
		return SuppliedLevelSequence;
	}
	else if (RecordIntoLevelSequence)
	{
		return RecordIntoLevelSequence;
	}
	else if (RecordingLevelSequence)
	{
		return RecordingLevelSequence;
	}
	else if (TransientPreset)
	{
		return TransientPreset->GetLevelSequence();
	}
	return nullptr;
}

ULevelSequence* UTakeRecorderSubsystemImplementation::GetSuppliedLevelSequence() const
{
	return SuppliedLevelSequence;
}

ULevelSequence* UTakeRecorderSubsystemImplementation::GetRecordingLevelSequence() const
{
	return RecordingLevelSequence;
}

ULevelSequence* UTakeRecorderSubsystemImplementation::GetRecordIntoLevelSequence() const
{
	return RecordIntoLevelSequence;
}

ULevelSequence* UTakeRecorderSubsystemImplementation::GetLastRecordedLevelSequence() const
{
	return LastRecordedLevelSequence.Get();
}

UTakePreset* UTakeRecorderSubsystemImplementation::GetTransientPreset() const
{
	return TransientPreset;
}

ETakeRecorderMode UTakeRecorderSubsystemImplementation::GetTakeRecorderMode() const
{
	if (RecordIntoLevelSequence != nullptr)
	{
		return ETakeRecorderMode::RecordIntoSequence;
	}

	return ETakeRecorderMode::RecordNewSequence;
}

UTakeRecorderNamingTokensData* UTakeRecorderSubsystemImplementation::GetNamingTokensData() const
{
	check(NamingTokensData);
	return NamingTokensData;
}

bool UTakeRecorderSubsystemImplementation::HasPendingChanges() const
{
	const UMovieScene* MovieScene = GetLevelSequence() ? GetLevelSequence()->GetMovieScene() : nullptr;
	const bool bHasUserMadeChanges = MovieScene
		&& (!MovieScene->GetTracks().IsEmpty()
			|| !MovieScene->GetBindings().IsEmpty()
			|| MovieScene->GetPossessableCount() > 0
			|| MovieScene->GetSpawnableCount() > 0);
	return bHasUserMadeChanges;
}

UClass* UTakeRecorderSubsystemImplementation::GetPresetTargetRecordClass() const
{
	return UTakePresetSettings::Get()->GetTargetRecordClass();
}

void UTakeRecorderSubsystemImplementation::SetPresetTargetRecordClass(const TSubclassOf<ULevelSequence> InClass)
{
	UTakePresetSettings::Get()->SetTargetRecordClass(InClass.Get());
}

UTakePreset* UTakeRecorderSubsystemImplementation::AllocateTransientPreset()
{
	return UTakePreset::AllocateTransientPreset(GetDefault<UTakeRecorderUserSettings>()->LastOpenedPreset.Get());
}

void UTakeRecorderSubsystemImplementation::CacheMetaData()
{
	UTakeMetaData* NewMetaDataThisTick = nullptr;

	if (const ULevelSequence* LevelSequence = GetLevelSequence())
	{
	#if WITH_EDITOR
		NewMetaDataThisTick = LevelSequence->FindMetaData<UTakeMetaData>();
	#endif // WITH_EDITOR
	}

	// If it's null we use the transient meta-data
	if (!NewMetaDataThisTick)
	{
		// if the transient meta-data doesn't exist, create it now
		if (!TransientTakeMetaData)
		{
			TransientTakeMetaData = UTakeMetaData::CreateFromDefaults(GetTransientPackage(), NAME_None);
			TransientTakeMetaData->SetFlags(RF_Transactional | RF_Transient);

			UpdateTransientDefaultSlateName();

			// Compute the correct starting take number
			int32 NextTakeNumber = GetNextTakeNumber(TransientTakeMetaData->GetSlate());
			if (TransientTakeMetaData->GetTakeNumber() != NextTakeNumber)
			{
				TransientTakeMetaData->SetTakeNumber(NextTakeNumber, false);
			}

			UTakeMetaData::SetMostRecentMetaData(TransientTakeMetaData);
		}

		NewMetaDataThisTick = TransientTakeMetaData;
	}

	check(NewMetaDataThisTick);
	if (NewMetaDataThisTick != TakeMetaData)
	{
		TakeMetaData = NewMetaDataThisTick;

	#if WITH_EDITOR
		if (!OnPreForceDeleteObjectsHandle.IsValid())
		{
			// Since this is a strong reference, we need to be able to clear it if the user is attempting to remove it, but let's not always
			// hook this event since the subsystem won't be active for general UE operation.
			OnPreForceDeleteObjectsHandle = FEditorDelegates::OnPreForceDeleteObjects.AddUObject(this, &UTakeRecorderSubsystemImplementation::OnPreForceDeleteObjects);
		}
	#endif // WITH_EDITOR
	}

	//Set MovieScene Display Rate to the Preset Frame Rate.
	if (const ULevelSequence* Sequence = GetLevelSequence())
	{
		if (UMovieScene* MovieScene = Sequence->GetMovieScene())
		{
			MovieScene->SetDisplayRate(TakeMetaData->GetFrameRate());
		}
	}

	check(TakeMetaData);
}

void UTakeRecorderSubsystemImplementation::UpdateTransientDefaultSlateName()
{
	const FString DefaultSlate = GetDefault<UTakeRecorderProjectSettings>()->Settings.DefaultSlate;
	if (TransientTakeMetaData && TransientTakeMetaData->GetSlate() != DefaultSlate)
	{
		TransientTakeMetaData->SetSlate(DefaultSlate, false);
	}
}

void UTakeRecorderSubsystemImplementation::SetFrameRateImpl(const FFrameRate& InFrameRate, bool bFromTimecode)
{
	if (TakeMetaData)
	{
		TakeMetaData->SetFrameRateFromTimecode(bFromTimecode);
		TakeMetaData->SetFrameRate(InFrameRate);
	}
	const ULevelSequence* Sequence = GetLevelSequence();
	if (UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr)
	{
		MovieScene->SetDisplayRate(InFrameRate);
	}
}

void UTakeRecorderSubsystemImplementation::OnAssetRegistryFilesLoaded()
{
	if (bAutoApplyTakeNumber && TransientTakeMetaData)
	{
		const int32 NextTakeNumber = GetNextTakeNumber(TransientTakeMetaData->GetSlate());
		TransientTakeMetaData->SetTakeNumber(NextTakeNumber);
	}
}

void UTakeRecorderSubsystemImplementation::OnRecordingInitialized(UTakeRecorder* Recorder)
{
	if (!bHasTargetSequenceBeenSet)
	{
		// Not initialized, take recorder triggered outside of the subsystem.
		SetTargetSequence();
	}
	
	// This needs to be stored with a strong ptr before our panels refresh, otherwise the weak sequencer ref will be lost during initialize.
	RecordingLevelSequence = Recorder->GetSequence();
	// Recache the meta-data here since we know that the sequence has probably changed as a result of the recording being started
	CacheMetaData();
	OnRecordingFinishedHandle = Recorder->OnRecordingFinished().AddUObject(this, &UTakeRecorderSubsystemImplementation::OnRecordingFinished);
	OnRecordingStoppedHandle = Recorder->OnRecordingStopped().AddUObject(this, &UTakeRecorderSubsystemImplementation::OnRecordingStopped);
	OnRecordingCancelledHandle = Recorder->OnRecordingCancelled().AddUObject(this, &UTakeRecorderSubsystemImplementation::OnRecordingCancelled);
	
	WITH_OWNING_SUBSYSTEM(GetOnRecordingInitializedEvent().Broadcast(Recorder));
	WITH_OWNING_SUBSYSTEM(TakeRecorderInitialized.Broadcast());
}


void UTakeRecorderSubsystemImplementation::OnRecordingStarted(UTakeRecorder* Recorder)
{
	WITH_OWNING_SUBSYSTEM(GetOnRecordingStartedEvent().Broadcast(Recorder));
	WITH_OWNING_SUBSYSTEM(TakeRecorderStarted.Broadcast());
}

void UTakeRecorderSubsystemImplementation::OnRecordingStopped(UTakeRecorder* Recorder)
{
	Recorder->OnRecordingStopped().Remove(OnRecordingStoppedHandle);

	WITH_OWNING_SUBSYSTEM(GetOnRecordingStoppedEvent().Broadcast(Recorder));
	WITH_OWNING_SUBSYSTEM(TakeRecorderStopped.Broadcast());
}

void UTakeRecorderSubsystemImplementation::OnRecordingFinished(UTakeRecorder* Recorder)
{
	LastRecordedLevelSequence = RecordingLevelSequence;
	RecordingLevelSequence = nullptr;

	IncrementTakeNumber();
	
	if (TransientTakeMetaData)
	{
		// Increment the transient take meta data if necessary
		int32 NextTakeNumber = GetNextTakeNumber(TransientTakeMetaData->GetSlate());

		if (TransientTakeMetaData->GetTakeNumber() != NextTakeNumber)
		{
			TransientTakeMetaData->SetTakeNumber(NextTakeNumber);
		}

		bAutoApplyTakeNumber = true;
	}

	Recorder->OnRecordingFinished().Remove(OnRecordingFinishedHandle);

	WITH_OWNING_SUBSYSTEM(GetOnRecordingFinishedEvent().Broadcast(Recorder));
	WITH_OWNING_SUBSYSTEM(TakeRecorderFinished.Broadcast(LastRecordedLevelSequence.Get()));
}

void UTakeRecorderSubsystemImplementation::OnRecordingCancelled(UTakeRecorder* Recorder)
{
	RecordingLevelSequence = nullptr;

	Recorder->OnRecordingFinished().Remove(OnRecordingFinishedHandle);
	Recorder->OnRecordingCancelled().Remove(OnRecordingCancelledHandle);

	WITH_OWNING_SUBSYSTEM(GetOnRecordingCancelledEvent().Broadcast(Recorder));
	WITH_OWNING_SUBSYSTEM(TakeRecorderCancelled.Broadcast());
}

void UTakeRecorderSubsystemImplementation::OnTakeSlateChanged(const FString& InSlate, UTakeMetaData* InTakeMetaData)
{
	WITH_OWNING_SUBSYSTEM(TakeRecorderSlateChanged.Broadcast(InSlate, InTakeMetaData));
}

void UTakeRecorderSubsystemImplementation::OnTakeNumberChanged(int32 InTakeNumber, UTakeMetaData* InTakeMetaData)
{
	WITH_OWNING_SUBSYSTEM(TakeRecorderTakeNumberChanged.Broadcast(InTakeNumber, InTakeMetaData));
}

void UTakeRecorderSubsystemImplementation::SetLastLevelSequence(ULevelSequence* InSequence)
{
	LastRecordedLevelSequence = InSequence;
}

void UTakeRecorderSubsystemImplementation::OnSourceAdded(UTakeRecorderSource* Source)
{
	WITH_OWNING_SUBSYSTEM(GetOnRecordingSourceAddedEvent().Broadcast(Source));
	WITH_OWNING_SUBSYSTEM(TakeRecorderSourceAdded.Broadcast(Source));
}

void UTakeRecorderSubsystemImplementation::OnSourceRemoved(UTakeRecorderSource* Source)
{
	WITH_OWNING_SUBSYSTEM(GetOnRecordingSourceRemovedEvent().Broadcast(Source));
	WITH_OWNING_SUBSYSTEM(TakeRecorderSourceRemoved.Broadcast(Source));
}

void UTakeRecorderSubsystemImplementation::OnSourceModified(UTakeRecorderSource* Source)
{
	WITH_OWNING_SUBSYSTEM(GetOnRecordingSourceModifiedEvent().Broadcast(Source));
	WITH_OWNING_SUBSYSTEM(TakeRecorderSourceModified.Broadcast(Source));
}

void UTakeRecorderSubsystemImplementation::BindToNamingTokenEvents()
{
#if WITH_EDITOR
	if (GEngine && !GetNamingTokensData()->TakeRecorderNamingTokens.IsValid())
	{
		GetNamingTokensData()->TakeRecorderNamingTokens = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>()->GetNamingTokens(ITakeRecorderNamingTokensModule::GetTakeRecorderNamespace());
		if (ensure(GetNamingTokensData()->TakeRecorderNamingTokens.IsValid()))
		{
			const TStrongObjectPtr<UNamingTokens> TakeRecorderNamingTokensPin = GetNamingTokensData()->TakeRecorderNamingTokens.Pin();
			TakeRecorderNamingTokensPin->RegisterExternalTokens(GetNamingTokensData()->NamingTokensExternalGuid);
			TakeRecorderNamingTokensPin->GetOnPreEvaluateEvent().AddUObject(this, &UTakeRecorderSubsystemImplementation::OnTakeRecorderNamingTokensPreEvaluate);
		}
	}
#endif // WITH_EDITOR
}

void UTakeRecorderSubsystemImplementation::UnbindNamingTokensEvents()
{
#if WITH_EDITOR
	if (const TStrongObjectPtr<UNamingTokens> TakeRecorderNamingTokensPin = GetNamingTokensData()->TakeRecorderNamingTokens.Pin())
	{
		TakeRecorderNamingTokensPin->UnregisterExternalTokens(GetNamingTokensData()->NamingTokensExternalGuid);
		TakeRecorderNamingTokensPin->GetOnPreEvaluateEvent().RemoveAll(this);
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UTakeRecorderSubsystemImplementation::OnTakeRecorderNamingTokensPreEvaluate(const FNamingTokensEvaluationData& InEvaluationData)
{
	if (GetNamingTokensData()->TakeRecorderNamingTokens.IsValid())
	{
		TArray<FNamingTokenData>& ExternalTokens = GetNamingTokensData()->TakeRecorderNamingTokens->GetExternalTokensChecked(GetNamingTokensData()->NamingTokensExternalGuid);

		ExternalTokens.Reset(GetNamingTokensData()->UserDefinedTokens.Num());
		for (const TTuple<FNamingTokenData, FText>& UserToken : GetNamingTokensData()->UserDefinedTokens)
		{
			ExternalTokens.Add({
				UserToken.Key.TokenKey,
				UserToken.Key.DisplayName,
				// @todo naming tokens - If we support TOptional<FText> that can be used instead of a lambda
				FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([Value = UserToken.Value]() {
					return Value;
				})
			});
		}
	}
}
#endif // WITH_EDITOR

void UTakeRecorderSubsystemImplementation::OnTakePresetSettingsChanged()
{
	// This effectively regenerates the level sequence in the preset.
	// Existing data is discarded. In the future, we should try to migrate it.
	// FTakePresetRecorderCustomization handles asking the user whether it is ok to discard the changes. Once this event fires, the decision has been made.
#if WITH_EDITOR
	if (!GIsTransacting)
	{
		ClearPendingTake();
	}
#endif // WITH_EDITOR
}

void UTakeRecorderSubsystemImplementation::OnPreForceDeleteObjects(const TArray<UObject*>& InObjects)
{
	// Take meta data is cached as a strong reference, but can prevent us from deleting owning level sequences.
	// We need to keep a strong reference to it for Take Recorder functionality, especially in the panel, and it should persist
	// even with the panel closed. Listening for a force delete allows us to safely clear the reference.
	const ULevelSequence* LevelSequence = GetLevelSequence();
	if (!LevelSequence && LastRecordedLevelSequence.IsValid())
	{
		LevelSequence = LastRecordedLevelSequence.Get();
	}
	if (LevelSequence && TakeMetaData && InObjects.Contains(LevelSequence)
	#if WITH_EDITOR
		&& LevelSequence->FindMetaData<UTakeMetaData>() == TakeMetaData
	#endif // WITH_EDITOR
		)
	{
		if (IsRecording())
		{
			// Need to cancel or we will be in a bad state. TakeRecorder will tick but be without a level sequence.
			CancelRecording(false);
		}
		TakeMetaData = nullptr;

	#if WITH_EDITOR
		FEditorDelegates::OnPreForceDeleteObjects.Remove(OnPreForceDeleteObjectsHandle);
		OnPreForceDeleteObjectsHandle.Reset();
	#endif // WITH_EDITOR
	}
}

#undef WITH_OWNING_SUBSYSTEM
#undef LOCTEXT_NAMESPACE
