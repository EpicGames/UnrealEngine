// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Recorder/ITakeRecorderSubsystemInterface.h"
#include "Recorder/TakeRecorderParameters.h"
#include "Tickable.h"

#include "TakeRecorderSubsystemImplementation.generated.h"

class UTakeRecorder;
class UTakeRecorderSource;
class UTakeRecorderSubsystem;
struct FNamingTokensEvaluationData;

UCLASS()
class UTakeRecorderSubsystemImplementation final : public UObject, public ITakeRecorderSubsystemInterface, public FTickableGameObject
{
	GENERATED_BODY()

public:
	// ~Begin FTickableGameObject
	virtual ETickableTickType GetTickableTickType() const override;
	virtual UWorld* GetTickableGameObjectWorld() const override;
	virtual bool IsTickable() const override;
	virtual bool IsTickableWhenPaused() const override;
	virtual bool IsTickableInEditor() const override;
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	// ~End FTickableGameObject

	/** Perform implementation specific initialization. */
	void InitializeImplementation(UTakeRecorderSubsystem* OwningSubsystem);

	/** Perform implementation specific shutdown. */
	void DeinitializeImplementation();

	// ~Begin ITakeRecorderSubsystem interface
	virtual void SetTargetSequence(const FTakeRecorderSequenceParameters& InData = FTakeRecorderSequenceParameters()) override;
	virtual void SetRecordIntoLevelSequence(ULevelSequence* LevelSequence) override;
	virtual bool CanReviewLastRecording() const override;
	virtual bool ReviewLastRecording() override;
	virtual bool StartRecording(bool bOpenSequencer = true, bool bShowErrorMessage = true) override;
	virtual void StopRecording() override;
	virtual void CancelRecording(bool bShowConfirmMessage = true) override;
	virtual void ResetToPendingTake() override;
	virtual void ClearPendingTake() override;
	virtual UTakePreset* GetPendingTake() const override;
	virtual void RevertChanges() override;
	virtual bool IsRecordingUsingExistingSequence() const override;
	virtual UTakeRecorderSource* AddSource(const TSubclassOf<UTakeRecorderSource> InSourceClass) override;
	virtual void RemoveSource(UTakeRecorderSource* InSource) override;
	virtual void ClearSources() override;
	virtual UTakeRecorderSources* GetSources() const override;
	virtual TArrayView<UTakeRecorderSource* const> GetAllSources() const override;
	virtual TArray<UTakeRecorderSource*> GetAllSourcesCopy() const override;
	virtual UTakeRecorderSource* GetSourceByClass(const TSubclassOf<UTakeRecorderSource> InSourceClass) const override;
	virtual void AddSourceForActor(AActor* InActor, bool bReduceKeys = true, bool bShowProgress = true) override;
	virtual void RemoveActorFromSources(AActor* InActor) override;
	virtual AActor* GetSourceActor(UTakeRecorderSource* InSource) const override;
	virtual ETakeRecorderState GetState() const override;
	virtual void SetTakeNumber(int32 InNewTakeNumber, bool bEmitChanged = true) override;
	virtual int32 GetNextTakeNumber(const FString& InSlate) const override;
	virtual void GetNumberOfTakes(const FString& InSlate, int32& OutMaxTake, int32& OutNumTakes) const override;
	virtual void IncrementTakeNumber() override;
	virtual TArray<FAssetData> GetSlates(FName InPackagePath = NAME_None) const override;
	virtual void SetSlateName(const FString& InSlateName, bool bEmitChanged = true) override;
	virtual bool MarkFrame() override;
	virtual FFrameRate GetFrameRate() const override;
	virtual void SetFrameRate(FFrameRate InFrameRate) override;
	virtual void SetFrameRateFromTimecode() override;
	virtual void ImportPreset(const FAssetData& InPreset) override;
	virtual bool IsReviewing() const override;
	virtual bool IsRecording() const override;
	virtual bool TryGetSequenceCountdown(float& OutValue) const override;
	virtual void SetSequenceCountdown(float InSeconds) override;
	virtual TArray<UObject*> GetSourceRecordSettings(UTakeRecorderSource* InSource) const override;
	virtual FTakeRecorderParameters GetGlobalRecordSettings() const override;
	virtual void SetGlobalRecordSettings(const FTakeRecorderParameters& InParameters) override;
	virtual UTakeMetaData* GetTakeMetaData() const override;
	virtual ULevelSequence* GetLevelSequence() const override;
	virtual ULevelSequence* GetSuppliedLevelSequence() const override;
	virtual ULevelSequence* GetRecordingLevelSequence() const override;
	virtual ULevelSequence* GetRecordIntoLevelSequence() const override;
	virtual ULevelSequence* GetLastRecordedLevelSequence() const override;
	virtual UTakePreset* GetTransientPreset() const override;
	virtual ETakeRecorderMode GetTakeRecorderMode() const override;
	virtual UTakeRecorderNamingTokensData* GetNamingTokensData() const override;
	virtual bool HasPendingChanges() const override;
	virtual UClass* GetPresetTargetRecordClass() const override;
	virtual void SetPresetTargetRecordClass(const TSubclassOf<ULevelSequence> InClass) override;
	// ~End ITakeRecorderSubsystem interface
	
private:
	/**
	 * Allocate the preset required for interacting with this subsystem. Re-uses an existing preset if necessary.
	 */
	static UTakePreset* AllocateTransientPreset();

	/** Cache the current available metadata. */
	void CacheMetaData();

	/** Update our transient slate data with the current default slate name. */
	void UpdateTransientDefaultSlateName();

	/** Sets the frame rate. */
	void SetFrameRateImpl(const FFrameRate& InFrameRate, bool bFromTimecode);

	void OnAssetRegistryFilesLoaded();
	void OnRecordingInitialized(UTakeRecorder* Recorder);
	void OnRecordingStarted(UTakeRecorder* Recorder);
	void OnRecordingStopped(UTakeRecorder* Recorder);
	void OnRecordingFinished(UTakeRecorder* Recorder);
	void OnRecordingCancelled(UTakeRecorder* Recorder);
	void OnTakeSlateChanged(const FString& InSlate, UTakeMetaData* InTakeMetaData);
	void OnTakeNumberChanged(int32 InTakeNumber, UTakeMetaData* InTakeMetaData);
	
	/** Update the last level sequence. */
	void SetLastLevelSequence(ULevelSequence *InSequence);

	/** Callback when any source is added. */
	void OnSourceAdded(UTakeRecorderSource* Source);

	/** Callback when any source is removed. */
	void OnSourceRemoved(UTakeRecorderSource* Source);
	
	/** Callback when any sources is modified. */
	void OnSourceModified(UTakeRecorderSource* Source);

	/** Setup bindings to our naming tokens. */
	void BindToNamingTokenEvents();
	
	/** Remove bindings to our naming tokens. */
	void UnbindNamingTokensEvents();

	/** Called before our naming tokens class evaluates. Used to populate naming token data with our custom token definitions. */
	void OnTakeRecorderNamingTokensPreEvaluate(const FNamingTokensEvaluationData& InEvaluationData);
	
	/** Handles the UTakePresetSettings::RecordTargetClass changing: Recreates the transient level sequence if recording a transaction. */
	void OnTakePresetSettingsChanged();

	/** Callback when the engine is attempting to force delete objects. */
	void OnPreForceDeleteObjects(const TArray<UObject*>& InObjects);
	
private:
	/** The public facing engine subsystem we are implementing. */
	TWeakObjectPtr<UTakeRecorderSubsystem> OwningSubsystemWeakPtr;
	
	/** Last data used for initialization. */
	FTakeRecorderSequenceParameters TargetSequenceData;

	/** A transient preset available for the subsystem. */
	UPROPERTY(Transient)
	TObjectPtr<UTakePreset> TransientPreset;

	/** Current supplied level sequence. */
	UPROPERTY()
	TObjectPtr<ULevelSequence> SuppliedLevelSequence;

	/** Current record into level sequence. */
	UPROPERTY()
	TObjectPtr<ULevelSequence> RecordIntoLevelSequence;

	/** Current recording level sequence. */
	UPROPERTY()
	TObjectPtr<ULevelSequence> RecordingLevelSequence;

	/** The last recorded level sequence. */
	UPROPERTY()
	TWeakObjectPtr<ULevelSequence> LastRecordedLevelSequence;

	/** Take meta-data cached from the level sequence if it exists. */
	UPROPERTY()
	TObjectPtr<UTakeMetaData> TakeMetaData;
	
	/** Transient take meta-data owned by this subsystem. Only used if none exists on the level sequence already. */
	UPROPERTY(Transient)
	TObjectPtr<UTakeMetaData> TransientTakeMetaData;

	/** Stored data relevant to our naming tokens. Managed as UObject for transactions. */
	UPROPERTY(Transient)
	TObjectPtr<UTakeRecorderNamingTokensData> NamingTokensData;
	
	FDelegateHandle OnAssetRegistryFilesLoadedHandle;
	FDelegateHandle OnRecordingInitializedHandle, OnRecordingStoppedHandle, OnRecordingFinishedHandle, OnRecordingCancelledHandle;
	FDelegateHandle OnPreForceDeleteObjectsHandle;
	
	/** Whether we should auto apply the next available take number when asset discovery has finished or not */
	bool bAutoApplyTakeNumber = false;

	/** If SetTargetSequence has been called, and we're fully initialized. */
	bool bHasTargetSequenceBeenSet = false;
	
	/** Helps determine if we are recording into an existing take or using one as a base. */
	bool bRecordingUsingExistingSequence = false;
};
