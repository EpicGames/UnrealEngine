// Copyright Epic Games, Inc. All Rights Reserved.

#include "SerializedRecorder.h"
#include "TakeRecorderSources.h"
#include "Serializers/MovieSceneSectionSerialization.h"
#include "Serializers/MovieSceneActorSerialization.h"
#include "Serializers/MovieSceneManifestSerialization.h"
#include "Serializers/MovieScenePropertySerialization.h"
#include "Serializers/MovieSceneTransformSerialization.h"
#include "Serializers/MovieSceneSpawnSerialization.h"
#include "Serializers/MovieSceneAnimationSerialization.h"
#include "Serializers/MovieSceneSerializedType.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Sections/MovieSceneSubSection.h"
#include "MovieScenePossessable.h"
#include "MovieSceneFolder.h"
#include "EngineUtils.h"
#include "TrackRecorders/IMovieSceneTrackRecorderFactory.h"
#include "TrackRecorders/MovieSceneTrackRecorder.h"
#include "Features/IModularFeatures.h"
#include "LevelSequence.h"
#include "TrackRecorders/MovieScenePropertyTrackRecorder.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/MovementComponent.h"
#include "GameFramework/Character.h"

#include "AssetRegistry/AssetData.h"
#include "UObject/UObjectIterator.h"
#include "LevelSequenceActor.h"

#if WITH_EDITOR
#include "SequencerUtilities.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "SerializedRecorder"

bool FSerializedRecorder::LoadRecordedSequencerFile(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FString& InFileName, TFunction<void()> InCompletionCallback)
{

	TMovieSceneSerializer<FSerializedTypeFileHeader, FSerializedTypeFileHeader> Serializer;

	FSerializedTypeFileHeader Header;
	FText Error;
	bool bFileExists = Serializer.DoesFileExist(InFileName);
	if (bFileExists)
	{
		if (Serializer.OpenForRead(InFileName, Header, Error))
		{
			//This is somewhat modular here but probably want to make it more so.
			//The 'manifest' types like Actor (and eventually maybe Sequence) are hardcoded and loaded here.
			//The Section Recorders though are mostly modular and we use the factory and recorder interfaces to load in the data, expect for Transform (and eventually Animation)
			//Since they are also hardcoded as special modules
			Serializer.Close();
			UMovieScene* InMovieScene = InMovieSceneSequence->GetMovieScene();

			if (Header.SerializedType == FName(TEXT("Sequence")))
			{
				return LoadSequenceFile(InMovieSceneSequence, PlaybackContext, InFileName, InCompletionCallback);
			}
			else if (Header.SerializedType == FName(TEXT("Actor")))
			{
				return LoadActorFile(InMovieSceneSequence, PlaybackContext, InFileName, InCompletionCallback);
			}
			else if (Header.SerializedType == FName(TEXT("Property")))
			{
				return LoadPropertyFile(InMovieSceneSequence, PlaybackContext, InFileName, InCompletionCallback);
			}
			else if (Header.SerializedType == FName(TEXT("SubSequence")))
			{
				return LoadSubSequenceFile(InMovieSceneSequence, PlaybackContext, InFileName, InCompletionCallback);
			}
			
			static const FName MovieSceneSectionRecorderFactoryName("MovieSceneTrackRecorderFactory");
			TArray<IMovieSceneTrackRecorderFactory*> ModularFactories = IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneTrackRecorderFactory>(MovieSceneSectionRecorderFactoryName);
			for (IMovieSceneTrackRecorderFactory* Factory : ModularFactories)
			{
				if (Factory->IsSerializable() && Factory->GetSerializedType() == Header.SerializedType)
				{
					UMovieSceneTrackRecorder* SectionRecorder = Factory->CreateTrackRecorderForObject();
					if (SectionRecorder)
					{
						SectionRecorder->AddToRoot();
						if (SectionRecorder->LoadRecordedFile(InFileName, InMovieScene, ActorGuidToActorMap, [SectionRecorder, InCompletionCallback]()
								{
									SectionRecorder->RemoveFromRoot();
									InCompletionCallback();
								})
						)
						{
							return true;
						}
						else
						{
							SectionRecorder->RemoveFromRoot();
						}
					}
				}
			}
		}
		else
		{
			Serializer.Close();
		}
	}
	return false;
}

bool FSerializedRecorder::LoadActorFile(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FString& InFileName, TFunction<void()> InCompletionCallback)
{
	FText Error;
	FActorFileHeader Header;
	FActorSerializedFrame Frame;
	if (bLoadSequenceFile == false)
	{
		ActorGuidToActorMap.Empty();
	}
	TSharedPtr<FActorSerializer> Serializer = MakeShared<FActorSerializer>();

	bool bFileExists = Serializer->DoesFileExist(InFileName);
	if (bFileExists)
	{
		if (Serializer->OpenForRead(InFileName, Header, Error))
		{
			AActor* Actor = SetActorPossesableOrSpawnable(InMovieSceneSequence, PlaybackContext, Header);

			Serializer->GetDataRanges([this, Actor, Serializer, InFileName, InMovieSceneSequence, PlaybackContext, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
			{
				auto OnReadComplete = [this, Actor, Serializer, InFileName, InMovieSceneSequence, PlaybackContext, Header, InCompletionCallback]()
				{
					const TArray<FActorSerializedFrame>& InFrames = Serializer->ResultData;
					if (InFrames.Num() > 0)
					{
						for (const FActorSerializedFrame& SerializedFrame : InFrames)
						{
							const FActorProperty& VisitedFrame = SerializedFrame.Frame;
							if (VisitedFrame.Type == EActoryPropertyType::ComponentType)
							{
								SetComponentPossessable(InMovieSceneSequence, PlaybackContext, Actor, Header, VisitedFrame);
							}
							else
							{
								FString PathPart, FilenamePart, ExtensionPart;
								FPaths::Split(InFileName, PathPart, FilenamePart, ExtensionPart);
								FilenamePart = FString("/") + VisitedFrame.SerializedType.ToString() + FString("_") + VisitedFrame.UObjectName;
								if (VisitedFrame.Type == EActoryPropertyType::PropertyType)
								{
									FilenamePart = FilenamePart + FString("_") + VisitedFrame.PropertyName;
								}
								LoadRecordedSequencerFile(InMovieSceneSequence, PlaybackContext, PathPart + FilenamePart, InCompletionCallback);
							}
						}
					}
					Serializer->Close();
					InCompletionCallback();

				}; //callback
				Serializer->ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);
			});
			return true;
		}
		else
		{
			Serializer->Close();
		}
	}
	return false;
}

bool FSerializedRecorder::LoadSequenceFile(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FString& InFileName, TFunction<void()> InCompletionCallback)
{
	FText Error;
	FManifestFileHeader Header;
	FManifestSerializedFrame Frame;
	bLoadSequenceFile = true;
	ActorGuidToActorMap.Empty();

	TSharedPtr<FManifestSerializer> Serializer = MakeShared<FManifestSerializer>();

	bool bFileExists = Serializer->DoesFileExist(InFileName);
	if (bFileExists)
	{
		if (Serializer->OpenForRead(InFileName, Header, Error))
		{
			Serializer->GetDataRanges([this, Serializer, InFileName, InMovieSceneSequence, PlaybackContext, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
			{
				auto OnReadComplete = [this, Serializer, InFileName, InMovieSceneSequence, PlaybackContext, Header, InCompletionCallback]()
				{
					const TArray<FManifestSerializedFrame>& InFrames = Serializer->ResultData;
					if (InFrames.Num() > 0)
					{
						for (const FManifestSerializedFrame& SerializedFrame : InFrames)
						{
							const FManifestProperty& VisitedFrame = SerializedFrame.Frame;

							FString PathPart, FilenamePart, ExtensionPart;
							FPaths::Split(InFileName, PathPart, FilenamePart, ExtensionPart);
							FilenamePart = FString("/") + VisitedFrame.UObjectName + FString("/") + VisitedFrame.SerializedType.ToString() + FString("_") + VisitedFrame.UObjectName;
							LoadRecordedSequencerFile(InMovieSceneSequence, PlaybackContext, PathPart + FilenamePart, InCompletionCallback);
						}
					}
					bLoadSequenceFile = false;
					Serializer->Close();
					InCompletionCallback();

				}; //callback
				Serializer->ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);
			});
		}
		else
		{
			bLoadSequenceFile = false;
			Serializer->Close();
		}
		return true;

	}
	return false;
}


bool FSerializedRecorder::LoadSubSequenceFile(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FString& InFileName, TFunction<void()> InCompletionCallback)
{
	FText Error;
	FManifestFileHeader Header;
	FManifestSerializedFrame Frame;

	TSharedPtr<FManifestSerializer> Serializer = MakeShared<FManifestSerializer>();

	bool bFileExists = Serializer->DoesFileExist(InFileName);
	if (bFileExists)
	{
		if (Serializer->OpenForRead(InFileName, Header, Error))
		{
			ULevelSequence* InRootSequence = CastChecked<ULevelSequence>(InMovieSceneSequence);

			ULevelSequence* TargetSequence = InRootSequence;
			const FString& SubSequenceName = Header.Name;
			TargetSequence = UTakeRecorderSources::CreateSubSequenceForSource(InRootSequence, SubSequenceName, SubSequenceName);
			if (TargetSequence)
			{
				// If there's already a Subscene Track for our sub-sequence we need to remove that track before create a new one. No data is lost in this process as the
				// sequence that the subscene points to has been copied by CreateSubSequenceForSource so a new track pointed to the new subsequence includes all the old data.
				TOptional<int32> RowIndex;
				const FString SequenceName = FPaths::GetBaseFilename(TargetSequence->GetPathName());
				UMovieSceneSubTrack* SubsceneTrack = nullptr;

				for (UMovieSceneTrack* Track : InRootSequence->GetMovieScene()->GetTracks())
				{
					if (Track->IsA<UMovieSceneSubTrack>())
					{
						// Look through each section in the track to see if it has a sub-sequence that matches our new sequence.
						for (UMovieSceneSection* Section : Track->GetAllSections())
						{
							UMovieSceneSubSection* SubSection = CastChecked<UMovieSceneSubSection>(Section);
							UMovieSceneSequence* SubSequence = SubSection->GetSequence();

							// Store the row index so we can re-inject the section at the same index to preserve the hierarchical evaluation order.
							if (FPaths::GetBaseFilename(SubSequence->GetPathName()) == SequenceName)
							{
								SubsceneTrack = CastChecked<UMovieSceneSubTrack>(Track);
								SubsceneTrack->RemoveSection(*Section);
								RowIndex = Section->GetRowIndex();
								break;
							}
						}
					}
				}

				// We need to add the new subsequence to the root sequence immediately so that it shows up in the UI and you can tell that things
				// are being recorded, otherwise they don't show up until recording stops and then it magically pops in.
				if (!SubsceneTrack)
				{
					SubsceneTrack = CastChecked<UMovieSceneSubTrack>(InRootSequence->GetMovieScene()->AddTrack(UMovieSceneSubTrack::StaticClass()));
				}

				// We create a new sub track for every Source so that we can name the Subtrack after the Source instead of just the sections within it.
			#if WITH_EDITOR
				SubsceneTrack->SetDisplayName(FText::FromString(SubSequenceName));
			#endif // WITH_EDITOR

				// When we create the Subscene Track we'll make sure a folder is created for it to sort into and add the new Subscene Track as a child of it.
				/*
				if (bCreateSequencerFolders)
				{
					UMovieSceneFolder* Folder = AddFolderForSource(Source, InRootSequence->GetMovieScene());
					Folder->AddChildTrack(SubsceneTrack);
				}
				*/

				// There isn't already a section for our new sub sequence, so we'll just append it to the end
				if (!RowIndex.IsSet())
				{
					RowIndex = SubsceneTrack->GetMaxRowIndex() + 1;
				}

				// We initialize the sequence to start at zero and be a 0 frame length section as there is no data in the sections yet.
				// We'll have to update these sections each frame as the recording progresses so they appear to get longer like normal
				// tracks do as we record into them.
				FFrameNumber RecordStartTime = FFrameNumber(0);
				UMovieSceneSubSection* NewSubSection = SubsceneTrack->AddSequence(TargetSequence, RecordStartTime, 0);

				NewSubSection->SetRowIndex(RowIndex.GetValue());
				SubsceneTrack->FixRowIndices();

				Serializer->GetDataRanges([this, Serializer, InFileName, TargetSequence, PlaybackContext, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
				{
					auto OnReadComplete = [this, Serializer, InFileName, TargetSequence, PlaybackContext, Header, InCompletionCallback]()
					{
						const TArray<FManifestSerializedFrame>& InFrames = Serializer->ResultData;
						if (InFrames.Num() > 0)
						{
							for (const FManifestSerializedFrame& SerializedFrame : InFrames)
							{
								const FManifestProperty& VisitedFrame = SerializedFrame.Frame;

								FString PathPart, FilenamePart, ExtensionPart;
								FPaths::Split(InFileName, PathPart, FilenamePart, ExtensionPart);
								FilenamePart = FString("/") + VisitedFrame.UObjectName + FString("/") + VisitedFrame.SerializedType.ToString() + FString("_") + VisitedFrame.UObjectName;
								LoadRecordedSequencerFile(TargetSequence, PlaybackContext, PathPart + FilenamePart, InCompletionCallback);
							}
						}
						Serializer->Close();
						InCompletionCallback();

					}; //callback
					Serializer->ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);
				});
				return true;
			}
			else
			{
				Serializer->Close();
			}
		}
		else
		{
			Serializer->Close();
		}
	}
	return false;
}

bool FSerializedRecorder::LoadPropertyFile(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FString& InFileName, TFunction<void()> InCompletionCallback)
{
	TMovieSceneSerializer<FPropertyFileHeader, FPropertyFileHeader> HeaderSerializer;
	FPropertyFileHeader Header;

	FMovieScenePropertyTrackRecorderFactory TrackRecorderFactory;
	FText Error;

	bool bFileExists = HeaderSerializer.DoesFileExist(InFileName);
	if (bFileExists)
	{
		if (HeaderSerializer.OpenForRead(InFileName, Header, Error))
		{
			HeaderSerializer.Close();
			UMovieScene* InMovieScene = InMovieSceneSequence->GetMovieScene();
			UMovieSceneTrackRecorder* SectionRecorder = TrackRecorderFactory.CreateTrackRecorderForPropertyEnum(Header.PropertyType, Header.PropertyName);
			if (SectionRecorder)
			{
				return SectionRecorder->LoadRecordedFile(InFileName, InMovieScene, ActorGuidToActorMap, InCompletionCallback);
			}
			return false;
		}
		else
		{
			HeaderSerializer.Close();
		}
	}
	return false;
}

AActor* FSerializedRecorder::SetActorPossesableOrSpawnable(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FActorFileHeader& ActorHeader)
{
	// first create a possessable for this component to be controlled by
	UMovieScene* MovieScene = InMovieSceneSequence->GetMovieScene();

	AActor* Actor = nullptr;
	FGuid PossessableGuid;

	if (ActorHeader.bRecordToPossessable)
	{		
		if (PlaybackContext)
		{
			for (TActorIterator<AActor> ActorItr(PlaybackContext); ActorItr; ++ActorItr)
			{
				AActor *InActor = *ActorItr;
				if (InActor && InActor->GetName() == *ActorHeader.UObjectName)
				{
					Actor = InActor;
					break;
				}
			}
		}
		if (Actor)
		{
			UClass* InitPossessedObjectClass = UClass::TryFindTypeSlow<UClass>(ActorHeader.ClassName);
			PossessableGuid = MovieScene->AddPossessable(ActorHeader.Label, InitPossessedObjectClass);
			InMovieSceneSequence->BindPossessableObject(PossessableGuid, *Actor, Actor->GetWorld());
		}
	}
	else
	{
		//Can't call this so need another way to serialize the UObject Properties, similar to what
		//happens with UEngine::CopyPropertiesForUnrelatedObjects, which is called below
		//by MakeSpawnableTemplateFromInstance.  
		//CachedObjectTemplate = CastChecked<AActor>(InMovieSceneSequence->MakeSpawnableTemplateFromInstance(*ActorToRecord, Header.TemplateName));
		//
		UClass* SpawnableClass = UClass::TryFindTypeSlow<UClass>(ActorHeader.ClassName);
		UObject* NewInstance = NewObject<UObject>(MovieScene, SpawnableClass, FName(*ActorHeader.TemplateName));
		//this is where UEngine::CopyPropertiesForUnrelatedObjects happens
		Actor = CastChecked<AActor>(NewInstance);
		if (Actor->GetAttachParentActor() != nullptr)
		{
			// We don't support spawnables and attachments right now
			// @todo: map to attach track?
			Actor->DetachFromActor(FDetachmentTransformRules(FAttachmentTransformRules(EAttachmentRule::KeepRelative, false), false));
		}
		//end of MakeSpawnableTemplateFromInstance hack

	#if WITH_EDITOR
		Actor->SetActorLabel(ActorHeader.Label); //spawnable has samen Name used for matching.
	#endif // WITH_EDITOR


	#if WITH_EDITOR
		PossessableGuid = MovieScene->AddPossessable(ActorHeader.Label, SpawnableClass);
		UE::Sequencer::FCreateBindingParams CreateBindingParams;
		CreateBindingParams.bAllowCustomBinding = true;
		CreateBindingParams.bSpawnable = true;
		CreateBindingParams.BindingNameOverride = ActorHeader.Label;
		CreateBindingParams.ReplacementGuid = PossessableGuid;
		FSequencerUtilities::CreateOrReplaceBinding(nullptr, InMovieSceneSequence, Actor, CreateBindingParams);
	#else // WITH_EDITOR
		// Review the Guid here and how the binding is created at runtime.
		PossessableGuid = MovieSceneHelpers::TryCreateCustomSpawnableBinding(InMovieSceneSequence, Actor);
	#endif // WITH_EDITOR

		if (PossessableGuid.IsValid())
		{
			// Override the Skeletal Mesh components animation modes so that they can play back the recorded
			// animation asset instead of their original animation source (such as Animation Blueprint)
			TInlineComponentArray<USkeletalMeshComponent*> SkeletalMeshComponents;
			Actor->GetComponents(SkeletalMeshComponents);
			for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
			{
				SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
				SkeletalMeshComponent->bEnableUpdateRateOptimizations = false;
				SkeletalMeshComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
				SkeletalMeshComponent->SetForcedLOD(1);
			}

			// Disable auto-possession on recorded Pawns so that when the Spawnable is spawned it doesn't auto-possess the player
			// and override their current live player pawn.
			if (Actor->IsA(APawn::StaticClass()))
			{
				APawn* Pawn = CastChecked<APawn>(Actor);
				Pawn->AutoPossessPlayer = EAutoReceiveInput::Disabled;
			}

			// Disable any Movement Components so that things such as RotatingMovementComponent or ProjectileMovementComponent don't suddenly
			// start moving and overriding our position at runtime.
			// takerecorder-todo: This should ideally check to see if you recorded the transform of the root object or not before assuming you
			// don't want its movement?
			TInlineComponentArray<UMovementComponent*> MovementComponents;
			Actor->GetComponents(MovementComponents);
			for (UMovementComponent* MovementComponent : MovementComponents)
			{
				MovementComponent->bAutoActivate = false;
			}
		}
	}
	ActorGuidToActorMap.Add(PossessableGuid, Actor);

	// Tag the possessable/spawnable with the original actor label so we can find it later
	if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(PossessableGuid))
	{
		if (!Possessable->Tags.Contains(*(ActorHeader.Label)))
		{
			Possessable->Tags.AddUnique(FName(*(ActorHeader.Label)));
		}
		Possessable->SetName(ActorHeader.Label);
	}

	// look for a folder to put us in
	if (ActorHeader.FolderName.IsValid())
	{
		UMovieSceneFolder* FolderToUse = nullptr;

	#if WITH_EDITOR
		for (UMovieSceneFolder* Folder : MovieScene->GetRootFolders())
		{
			if (Folder->GetFolderName() == ActorHeader.FolderName)
			{
				FolderToUse = Folder;
				break;
			}
		}
	#endif // WITH_EDITOR

		if (FolderToUse == nullptr)
		{
			FolderToUse = NewObject<UMovieSceneFolder>(MovieScene, NAME_None, RF_Transactional);
			FolderToUse->SetFolderName(ActorHeader.FolderName);
		#if WITH_EDITOR
			MovieScene->AddRootFolder(FolderToUse);
		#endif // WITH_EDITOR
		}

		FolderToUse->AddChildObjectBinding(PossessableGuid);
	}
	return Actor;
}

void FSerializedRecorder::SetComponentPossessable(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, AActor* Actor, const FActorFileHeader& ActorHeader, const FActorProperty& ActorProperty)
{
	if (Actor)
	{
		// first create a possessable for this component to be controlled by
		UMovieScene* InMovieScene = InMovieSceneSequence->GetMovieScene();

		UClass* InitPossessedObjectClass = UClass::TryFindTypeSlow<UClass>(ActorProperty.ClassName);
		FGuid PossessableGuid = InMovieScene->AddPossessable(ActorProperty.UObjectName, InitPossessedObjectClass);

		// Set up parent/child guids for possessables within spawnables
		FMovieScenePossessable* ChildPossessablePtr = InMovieScene->FindPossessable(PossessableGuid);
		ChildPossessablePtr->SetParent(PossessableGuid, InMovieScene);

		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (Component->GetName() == ActorProperty.UObjectName)
			{
				InMovieSceneSequence->BindPossessableObject(PossessableGuid, *Component, Actor);
				break;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE