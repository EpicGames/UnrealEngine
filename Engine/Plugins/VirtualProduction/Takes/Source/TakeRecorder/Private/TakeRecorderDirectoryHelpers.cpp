// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderDirectoryHelpers.h"

#if WITH_EDITOR
#include "EditorDirectories.h"
#else // WITH_EDITOR
#include "Misc/Paths.h"
#endif // WITH_EDITOR

#include "Misc/PackageName.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformFile.h"
namespace UE::TakeRecorder::Private
{

bool IsValidPath(const FString& InPathBase)
{
	FString TestPath = InPathBase + TEXT("/") + "TestPackage";
	return !InPathBase.IsEmpty() && FPackageName::IsValidLongPackageName(TestPath, false);
}

/**
 * Splits the path into two parts the base (potentionally) and the sub-folders. For example,
 *
 *  /Game/Cinematics/Takes will be split to /Game and /Cinematics/Takes
 *
 ** */
TTuple<FString, FString> SplitPackagePath(const FString& InPath)
{
	int32 DelimIndex = InPath.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromStart, 1);

	FString RightPart;
	FString LeftPart;
	if (DelimIndex != INDEX_NONE)
	{
		RightPart = InPath.Right(InPath.Len() - DelimIndex);
		LeftPart = InPath.Left(DelimIndex);
	}
	else
	{
		LeftPart = InPath;
	}

	return {LeftPart, RightPart};
}

FString GetDefaultProjectPath()
{
#if WITH_EDITOR
	// Determine the starting path. Try to use the most recently used directory
	FString AssetPath;
	const FString DefaultFilesystemDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
	if (DefaultFilesystemDirectory.IsEmpty() || !FPackageName::TryConvertFilenameToLongPackageName(DefaultFilesystemDirectory, AssetPath))
	{
		return TEXT("/Game");
	}
	if (AssetPath.EndsWith(TEXT("/")))
	{
		return AssetPath.Left(AssetPath.Len()-1);
	}
	return AssetPath;
#else // WITH_EDITOR
	// When the editor is not present, return the Project Content Directory. This is the behaviour of default path of
	// FEditorDirectories if there is no last directory, and so will mimick its behaviour.
	return FPaths::ProjectContentDir();
#endif // WITH_EDITOR
}

FString ResolvePathToProject(const FString& InPath)
{
	const FString FixedPathStr = InPath.StartsWith(TEXT("/")) ? InPath : TEXT("/") + InPath;
	const FString DefaultProjectPath = GetDefaultProjectPath();

	const FString FullProjectPath = DefaultProjectPath + FixedPathStr;
	const bool bFullProjectPathExists = FPlatformFileManager::Get().GetPlatformFile().DirectoryExists(*FullProjectPath);
	
	TTuple<FString, FString> Parts = SplitPackagePath(FixedPathStr);
	FString PossibleMountPoint = Parts.Get<0>() + TEXT("/");
	
	if (!bFullProjectPathExists && FPackageName::MountPointExists(PossibleMountPoint))
	{
		return InPath;
	}

	return FullProjectPath;
}

FString RemoveProjectFromPath(const FString& InPath)
{
	FString DefaultPath = GetDefaultProjectPath();
	if (InPath.StartsWith(TEXT("/Game")) || (!DefaultPath.IsEmpty() && InPath.StartsWith(DefaultPath)))
	{
		TTuple<FString, FString> Parts = SplitPackagePath(InPath);
		return Parts.Get<1>();
	}
	return InPath;
}

}
