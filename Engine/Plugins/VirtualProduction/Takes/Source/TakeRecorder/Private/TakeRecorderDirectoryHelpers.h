// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FString;

namespace UE::TakeRecorder::Private
{
/** Returns if the provided path is valid for the project */
bool IsValidPath(const FString& InPathBase);

/** Return the current default project path. */
FString GetDefaultProjectPath();

/** Resolve a path relative to the current project. */
FString ResolvePathToProject(const FString& InPath);

/** Remove the /Game or /Project part of the path. */
FString RemoveProjectFromPath(const FString& InPath);

}
