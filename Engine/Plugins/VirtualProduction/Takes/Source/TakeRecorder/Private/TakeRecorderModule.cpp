// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderModule.h"

#include "TakesCoreLog.h"
#include "TakeRecorderSettings.h"
#include "Modules/ModuleManager.h"
#include "Recorder/TakeRecorder.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Features/IModularFeatures.h"
#include "ITakeRecorderDropHandler.h"
#include "TakeRecorderSettings.h"
#include "TakePresetSettings.h"

#include "LevelSequence.h"
#include "TakeMetaData.h"
#include "MovieSceneTakeSettings.h"

#include "SerializedRecorder.h"
#include "Misc/Attribute.h"
#include "Textures/SlateIcon.h"

#include "Misc/QualifiedFrameTime.h"
#include "MovieSceneTakeSection.h"
#include "MovieSceneTakeTrack.h"
#include "LevelSequence.h"
#include "MovieSceneSequence.h"
#include "Sections/MovieSceneSubSection.h"
#include "MovieScene.h"
#include "UObject/Class.h"

#include "Algo/RemoveIf.h"
#include "Engine/Font.h"
#include "CanvasTypes.h"

#if WITH_EDITOR
#include "Editor.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "TakeRecorderModule"

FName ITakeRecorderModule::TakeRecorderTabName = "TakeRecorder";
FText ITakeRecorderModule::TakeRecorderTabLabel = LOCTEXT("TakeRecorderTab_Label", "Take Recorder");

FName ITakeRecorderModule::TakesBrowserTabName = "TakesBrowser";
FText ITakeRecorderModule::TakesBrowserTabLabel = LOCTEXT("TakesBrowserTab_Label", "Takes Browser");
FName ITakeRecorderModule::TakesBrowserInstanceName = "TakesBrowser";

IMPLEMENT_MODULE(FTakeRecorderModule, TakeRecorder);

static TAutoConsoleVariable<int32> CVarTakeRecorderSaveRecordedAssetsOverride(
	TEXT("TakeRecorder.SaveRecordedAssetsOverride"),
	0,
	TEXT("0: Save recorded assets is based on user settings\n1: Override save recorded assets to always start on"),
	ECVF_Default);

FName ITakeRecorderDropHandler::ModularFeatureName("ITakeRecorderDropHandler");

TArray<ITakeRecorderDropHandler*> ITakeRecorderDropHandler::GetDropHandlers()
{
	return IModularFeatures::Get().GetModularFeatureImplementations<ITakeRecorderDropHandler>(ModularFeatureName);
}

void FTakeRecorderModule::StartupModule()
{
	RegisterSettings();
	RegisterSerializedRecorder();

#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->OnEditorClose().AddRaw(this, &FTakeRecorderModule::OnEditorClose);
	}
	
#endif
}

void FTakeRecorderModule::ShutdownModule()
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->OnEditorClose().RemoveAll(this);
	}

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
#endif

	UnregisterSerializedRecorder();
}

void FTakeRecorderModule::PopulateSourcesMenu(TSharedRef<FExtender> InExtender, UTakeRecorderSources* InSources) const
{
	SourcesMenuExtenderEvent.Broadcast(InExtender, InSources);
}

UTakePreset* FTakeRecorderModule::GetPendingTake() const
{
	if (UE::IsSavingPackage(nullptr) || IsGarbageCollectingAndLockingUObjectHashTables())
	{
		UE_LOGF(LogTakesCore, Verbose, "Cannot call FTakeRecorderModule::GetPendingTake while saving a package or garbage collecting.");
		return nullptr;
	}
	return FindObject<UTakePreset>(nullptr, TEXT("/Temp/TakeRecorder/PendingTake.PendingTake"));
}

void FTakeRecorderModule::RegisterExternalObject(UObject* InExternalObject)
{
	ExternalObjects.Add(InExternalObject);
	ExternalObjectAddRemoveEvent.Broadcast(InExternalObject, true);
}

void FTakeRecorderModule::UnregisterExternalObject(UObject* InExternalObject)
{
	ExternalObjectAddRemoveEvent.Broadcast(InExternalObject, false);
	ExternalObjects.Remove(InExternalObject);
}

FDelegateHandle FTakeRecorderModule::RegisterSourcesMenuExtension(const FOnExtendSourcesMenu& InExtension)
{
	return SourcesMenuExtenderEvent.Add(InExtension);
}

void FTakeRecorderModule::RegisterSourcesExtension(const FSourceExtensionData& InData)
{
	SourceExtensionData = InData;
}

void FTakeRecorderModule::UnregisterSourcesExtension()
{
	SourceExtensionData = FSourceExtensionData();
}

void FTakeRecorderModule::UnregisterSourcesMenuExtension(FDelegateHandle Handle)
{
	SourcesMenuExtenderEvent.Remove(Handle);
}

void FTakeRecorderModule::RegisterSettingsObject(UObject* InSettingsObject)
{
	GetMutableDefault<UTakeRecorderProjectSettings>()->AdditionalSettings.Add(InSettingsObject);
}

void FTakeRecorderModule::RegisterSettings()
{
	RegisterSettingsObject(GetMutableDefault<UMovieSceneTakeSettings>());
	RegisterSettingsObject(GetMutableDefault<UTakePresetSettings>());

	GetMutableDefault<UTakeRecorderUserSettings>()->LoadConfig();
	const bool bSaveRecordedAssetsOverride = CVarTakeRecorderSaveRecordedAssetsOverride.GetValueOnGameThread() != 0;
	if (bSaveRecordedAssetsOverride)
	{
		GetMutableDefault<UTakeRecorderUserSettings>()->Settings.bSaveRecordedAssets = bSaveRecordedAssetsOverride;
	}
}

void FTakeRecorderModule::RegisterSerializedRecorder()
{
	SerializedRecorder = MakeShared<FSerializedRecorder>();
	IModularFeatures::Get().RegisterModularFeature(FSerializedRecorder::ModularFeatureName, SerializedRecorder.Get());
}

void FTakeRecorderModule::UnregisterSerializedRecorder() const
{
	IModularFeatures::Get().UnregisterModularFeature(FSerializedRecorder::ModularFeatureName, SerializedRecorder.Get());
}

void FTakeRecorderModule::OnEditorClose()
{
	if (UTakeRecorder* ActiveRecorder = UTakeRecorder::GetActiveRecorder())
	{
		ActiveRecorder->Stop();
	}
}

#undef LOCTEXT_NAMESPACE
