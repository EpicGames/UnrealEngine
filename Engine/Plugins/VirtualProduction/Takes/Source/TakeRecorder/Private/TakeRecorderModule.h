// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "ITakeRecorderModule.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FExtender;
class FTakePresetActions;
class UTakePreset;
class FSerializedRecorder;
class UTakeRecorderSources;
class USequencerSettings;

class FTakeRecorderModule : public ITakeRecorderModule
{
public:
	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface
	
	//~ Begin ITakeRecorderModule Interface
	virtual void PopulateSourcesMenu(TSharedRef<FExtender> InExtender, UTakeRecorderSources* InSources) const override;
	virtual UTakePreset* GetPendingTake() const override;
	virtual FOnGenerateWidgetExtensions& GetToolbarExtensionGenerators() override { return ToolbarExtensionGenerators; }
	virtual FOnGenerateWidgetExtensions& GetRecordButtonExtensionGenerators() override { return ButtonExtensionGenerators; }
	virtual FOnExternalObjectAddRemoveEvent& GetExternalObjectAddRemoveEventDelegate() override { return ExternalObjectAddRemoveEvent; }
	virtual FOnRecordErrorCheck&  GetRecordErrorCheckGenerator() override { return RecordErrorCheck; }
	virtual TArray<TWeakObjectPtr<>>& GetExternalObjects() override { return ExternalObjects; }
	virtual FLastRecordedLevelSequenceProvider& GetLastLevelSequenceProvider() override { return LastLevelSequenceProvider; }
	virtual FCanReviewLastRecordedLevelSequence& GetCanReviewLastRecordedLevelSequenceDelegate() override { return CanReviewLastRecordedSequence; };
	virtual void RegisterExternalObject(UObject* InExternalObject) override;
	virtual void UnregisterExternalObject(UObject* InExternalObject) override;
	virtual FOnForceSaveAsPreset& OnForceSaveAsPreset() override { return ForceSaveAsPresetEvent; }
	virtual FDelegateHandle RegisterSourcesMenuExtension(const FOnExtendSourcesMenu& InExtension) override;
	virtual void RegisterSourcesExtension(const FSourceExtensionData& InData) override;
	virtual void UnregisterSourcesExtension() override;
	virtual void UnregisterSourcesMenuExtension(FDelegateHandle Handle) override;
	virtual void RegisterSettingsObject(UObject* InSettingsObject) override;
	virtual FSourceExtensionData& GetSourcesExtensionData() override { return SourceExtensionData; }
	//~ Begin ITakeRecorderModule Interface
	
	
private:
	void RegisterSettings();

	void RegisterSerializedRecorder();
	void UnregisterSerializedRecorder() const;

	void OnEditorClose();

private:

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnExtendSourcesMenuEvent, TSharedRef<FExtender>, UTakeRecorderSources*);

	FOnExtendSourcesMenuEvent SourcesMenuExtenderEvent;
	FOnGenerateWidgetExtensions ToolbarExtensionGenerators;
	FOnGenerateWidgetExtensions ButtonExtensionGenerators;
	FLastRecordedLevelSequenceProvider LastLevelSequenceProvider;
	FCanReviewLastRecordedLevelSequence CanReviewLastRecordedSequence;

	FOnExternalObjectAddRemoveEvent ExternalObjectAddRemoveEvent;
	FOnForceSaveAsPreset ForceSaveAsPresetEvent;
	FOnRecordErrorCheck RecordErrorCheck;

	FDelegateHandle LevelEditorLayoutExtensionHandle;
	FDelegateHandle LevelEditorTabManagerChangedHandle;
	FDelegateHandle ModulesChangedHandle;

	TSharedPtr<FSerializedRecorder> SerializedRecorder;

	/** Extra class default objects that should appear in the Take Recorder panel. */
	TArray<TWeakObjectPtr<>> ExternalObjects;

	/**
	 * Registered extensions for the source module. We only use a single struct rather than container
	 * as only the source module should be utilizing this.
	 */
	FSourceExtensionData SourceExtensionData;
};