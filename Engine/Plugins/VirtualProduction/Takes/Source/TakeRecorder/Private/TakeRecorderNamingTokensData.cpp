// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderNamingTokensData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderNamingTokensData)

TArray<FString>& UTakeRecorderNamingTokensData::FindOrAddTokenKeysForField(const FString& InFieldName)
{
	if (FTakeRecorderNamingTokensFieldMapping* Match = FieldToUndefinedKeys.FindByPredicate([&InFieldName](const FTakeRecorderNamingTokensFieldMapping& Fields)
	{
		return Fields.FieldName == InFieldName;
	}))
	{
		return Match->UndefinedKeys;
	}
	FTakeRecorderNamingTokensFieldMapping Data;
	Data.FieldName = InFieldName;
	return FieldToUndefinedKeys.Add_GetRef(Data).UndefinedKeys;
}

bool UTakeRecorderNamingTokensData::IsTokenKeyUndefined(const FString& InTokenKey) const
{
	for (const FTakeRecorderNamingTokensFieldMapping& TokenFieldMapping : FieldToUndefinedKeys)
	{
		if (TokenFieldMapping.UndefinedKeys.Contains(InTokenKey))
		{
			return true;
		}
	}

	return false;
}
