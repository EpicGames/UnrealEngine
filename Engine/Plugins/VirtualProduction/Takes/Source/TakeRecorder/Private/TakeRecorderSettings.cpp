// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderSettings.h"

#include "Misc/PackageName.h"
#include "TakeRecorderDirectoryHelpers.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderSettings)

UTakeRecorderUserSettings::UTakeRecorderUserSettings()
{
	Settings.bMaximizeViewport  = false;
	Settings.CountdownSeconds   = 3.f;
	Settings.EngineTimeDilation = 1.f;
	Settings.bRemoveRedundantTracks = true;
	Settings.bSaveRecordedAssets = true;
	Settings.bAutoLock = true;
	Settings.bAutoSerialize     = false;
	PresetSaveDir.Path    = TEXT("/Cinematics/Takes/Presets/");
	bIsSequenceOpen       = true;
	bShowUserSettingsOnUI = false;
}

#if WITH_EDITOR
void UTakeRecorderUserSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		SetPresetSaveDir(PresetSaveDir.Path);
		// Dragging spinboxes causes this to be called every frame so we wait until they've finished dragging before saving.
		SaveConfig();
	}
}
#endif

void UTakeRecorderUserSettings::SetPresetSaveDir(const FString& InPath)
{
	PresetSaveDir.Path = UE::TakeRecorder::Private::RemoveProjectFromPath(InPath);
}

FString UTakeRecorderUserSettings::GetResolvedPresetSaveDir() const
{
	return UE::TakeRecorder::Private::ResolvePathToProject(PresetSaveDir.Path);
}

UTakeRecorderProjectSettings::UTakeRecorderProjectSettings()
{
	Settings.RootTakeSaveDir.Path = TEXT("/Cinematics/Takes");
	Settings.TakeSaveDir = TEXT("Takes_{year}{month}{day}/{slate}_{take}");
	Settings.DefaultSlate = TEXT("Scene_1");
	Settings.bRecordSourcesIntoSubSequences = true;
	Settings.bRecordToPossessable = false;
}

#if WITH_EDITOR
void UTakeRecorderProjectSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		Settings.RootTakeSaveDir.Path = UE::TakeRecorder::Private::RemoveProjectFromPath(Settings.RootTakeSaveDir.Path);

		// Dragging spinboxes causes this to be called every frame so we wait until they've finished dragging before saving.
		SaveConfig();
	}
}
#endif
