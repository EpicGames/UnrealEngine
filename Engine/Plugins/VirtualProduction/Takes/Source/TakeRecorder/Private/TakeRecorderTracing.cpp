// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderTracing.h"

UE_TRACE_CHANNEL_DEFINE(TakeRecorderChannel, "Traces take recording operations for virtual production workflows. Used to track recording sessions, \
playback operations, and take management activities.");
