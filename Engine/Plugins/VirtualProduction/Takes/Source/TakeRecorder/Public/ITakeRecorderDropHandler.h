// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Features/IModularFeature.h"

class UTakeRecorderSources;
class FDragDropOperation;

/**
 * Interface registerable through IModularFeatures for extendig drag/drop support for the take recorder sources panel
 */
struct ITakeRecorderDropHandler : public IModularFeature
{
	/** The feature name under which all ITakeRecorderDropHandlers should be registered */
	TAKERECORDER_API static FName ModularFeatureName;

	/**
	 * Get all the currently registered drop handlers
	 */
	TAKERECORDER_API static TArray<ITakeRecorderDropHandler*> GetDropHandlers();

public:

	virtual ~ITakeRecorderDropHandler() = default;


	/**
	 * Handle a drag drop operation for the specified sources.
	 *
	 * @param InOperation      The drag drop operation to be handled
	 * @param Sources          The sources to add any dropped items to
	 */
	virtual void HandleOperation(TSharedPtr<FDragDropOperation> InOperation, UTakeRecorderSources* Sources) = 0;


	/**
	 * Determine whether this drop handler can handle the specified operation
	 *
	 * @param InOperation      The drag drop operation to be handled
	 * @param Sources          The sources to add any dropped items to
	 */
	virtual bool CanHandleOperation(TSharedPtr<FDragDropOperation> InOperation, UTakeRecorderSources* Sources) = 0;
};
