// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class FExtender;
class UTakeRecorderSource;
class UTakeRecorderSources;
class UTakeMetaData;
class UTakePreset;


DECLARE_DELEGATE_TwoParams(FOnExtendSourcesMenu, TSharedRef<FExtender>, UTakeRecorderSources*)

/**
 * Delegate called to add extensions to the take recorder toolbar.
 * Usage: Bind a handler that adds a widget to the out array parameter.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGenerateWidgetExtensions, TArray<TSharedRef<class SWidget>>& /*OutExtensions*/);

/**
 * Delegate called to to validate if recording can take place.
 *
 * Usage: Bind a handler that will supply a FText string when Take Recorder is in an invalid condition. The text will
 * be displayed the user. If there is no error condition then the FText string must be empty.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnRecordErrorCheck, FText &);

/**
 * Delegate to request preset save.  Note this is only a temporary workaround until Concert plugin supports
 * instanced properties.
 */
DECLARE_DELEGATE(FOnForceSaveAsPreset);

class ULevelSequence;
/**
 * Delegate to provide last recorded level sequence.
 */
DECLARE_DELEGATE_OneParam(FLastRecordedLevelSequenceProvider, ULevelSequence*);

/**
 * Delegate to indicate if it is safe to review the last recorded level sequence.
 */
DECLARE_DELEGATE_RetVal(bool, FCanReviewLastRecordedLevelSequence);

/**
 * Public module interface for the Take Recorder module
 */
class ITakeRecorderModule : public IModuleInterface
{
public:
	/**
	 * Delegate called when an external object is registered for the take recorder panel. The boolean
	 * parameter indicates if the object was added. It is true if added and false if removed.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnExternalObjectAddRemoveEvent, UObject *, bool);
	
	/** Relevant data for the sources module. */
	struct FSourceExtensionData
	{
		/**
		 * Delegate for sources module to bind to when adding an actor.
		 */
		DECLARE_DELEGATE_FourParams(FAddActorSource, UTakeRecorderSources* TakeRecorderSources, TArrayView<AActor* const> ActorsToRecord, bool bReduceKeys, bool bShowProgress);                      

		/**
		 * Delegate for sources module to bind to when removing an actor.
		 */
		DECLARE_DELEGATE_TwoParams(FRemoveActorSource, UTakeRecorderSources* TakeRecorderSources, TArrayView<AActor* const> ActorsToRemove);

		/**
		 * Delegate to retrieve the actor of a source.
		 */
		DECLARE_DELEGATE_RetVal_OneParam(AActor*, FGetSourceActor, UTakeRecorderSource* Source);
		
		/** An actor is being added as a source. */
		FAddActorSource OnAddActorSource;

		/** An actor is being removed as a source. */
		FRemoveActorSource OnRemoveActorSource;

		/** The actor belonging to a source should be retrieved. */
		FGetSourceActor OnGetSourceActor;
	};
	
	/** The name under which the take recorder tab is registered and invoked */
	TAKERECORDER_API static FName TakeRecorderTabName;

	/** The default label for the take recorder tab */
	TAKERECORDER_API static FText TakeRecorderTabLabel;

	/** The tab name for the takes browser tab */
	TAKERECORDER_API static FName TakesBrowserTabName;

	/** The default label for the takes browser */
	TAKERECORDER_API static FText TakesBrowserTabLabel;

	/** The Takes Browser Content Browser Instance Name */
	TAKERECORDER_API static FName TakesBrowserInstanceName;

	static inline ITakeRecorderModule& Get() 
	{ 
		return FModuleManager::LoadModuleChecked<ITakeRecorderModule>("TakeRecorder"); 
	}
	
	static inline bool IsAvailable() 
	{ 
		return FModuleManager::Get().IsModuleLoaded( "TakeRecorder" ); 
	}

	virtual void PopulateSourcesMenu(TSharedRef<FExtender> InExtender, UTakeRecorderSources* InSources) const = 0;

	/** Get the current pending take. */
	virtual UTakePreset* GetPendingTake() const = 0;

	/**
	 * Register a new extension callback for the 'Add Source' menu
	 */
	virtual FDelegateHandle RegisterSourcesMenuExtension(const FOnExtendSourcesMenu& InExtension) = 0;

	/**
	 * Register generic extensions for sources. We only allow one extension as this should be handled by the sources module.
	 */
	virtual void RegisterSourcesExtension(const FSourceExtensionData& InData) = 0;

	/**
	 * Unregister the source extension.
	 */
	virtual void UnregisterSourcesExtension() = 0;

	/**
	 * Get the delegate for handling saving of preset data.
	 */
	virtual FOnForceSaveAsPreset& OnForceSaveAsPreset() = 0;

	/**
	 * Get the delegate for providing the current level sequence to the take recorder panel.
	 */
	virtual FLastRecordedLevelSequenceProvider& GetLastLevelSequenceProvider() = 0;

	/**
	 * Provides the take recorder panel state of review last recording.
	 */
	virtual FCanReviewLastRecordedLevelSequence& GetCanReviewLastRecordedLevelSequenceDelegate() = 0;

	/**
	 * Unregister a previously registered extension callback for the 'Add Source' menu
	 */
	virtual void UnregisterSourcesMenuExtension(FDelegateHandle Handle) = 0;


	/**
	 * Register a new class default object that should appear on the take recorder project settings
	 */
	virtual void RegisterSettingsObject(UObject* InSettingsObject) = 0;


	/**
	 * Register a new class default object that should appear on the take recorder panel.
	 */
	virtual void RegisterExternalObject(UObject* InExternalObject) = 0;

	/**
	 * Unregister a new class default object that should appear on the take recorder panel.
	 */
	virtual void UnregisterExternalObject(UObject* InExternalObject) = 0;

	/**
	 * Get the toolbar extension generators.
	 * Usage: Bind a handler that adds a widget to the out array parameter.
	 */
	virtual FOnGenerateWidgetExtensions& GetToolbarExtensionGenerators() = 0;

	/**
	 * Get the toolbar extension generators.
	 * Usage: Bind a handler that adds a widget to the out array parameter.
	 */
	virtual FOnGenerateWidgetExtensions& GetRecordButtonExtensionGenerators() = 0;

	/**
	 * Get the delegate for reporting any error conditions in recording state.
	 */
	virtual FOnRecordErrorCheck& GetRecordErrorCheckGenerator() = 0;

	/**
	 * Get the event notifier when an external object has been added or removed.
	 */
	virtual FOnExternalObjectAddRemoveEvent& GetExternalObjectAddRemoveEventDelegate() = 0;

	/**
	 * Get the take external objects registered to the take recorder.
	 */
	virtual TArray<TWeakObjectPtr<>>& GetExternalObjects() = 0;

	virtual FSourceExtensionData& GetSourcesExtensionData() = 0;
};

