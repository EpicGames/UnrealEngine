// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Internationalization/Text.h"
#include "Recorder/TakeRecorderPanel.h"

class UTakePreset;
class UTakeMetaData;
class ULevelSequence;
class STakeRecorderPanel;
class UTakeRecorderSources;
class STakePresetAssetEditor;

#if WITH_EDITOR
class FTakePresetToolkit;
#endif // WITH_EDITOR

/**
 * Interface for the Take Recorder tab's top-level content widget.
 */
class ITakeRecorderTabContent : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(ITakeRecorderTabContent){}
	SLATE_END_ARGS()

	virtual ~ITakeRecorderTabContent() override = default;

	/** Returns the display title shown in the tab. */
	virtual FText GetTitle() const = 0;

	/** Returns the icon brush shown in the tab. */
	virtual const FSlateBrush* GetIcon() const = 0;

	/**
	 * Returns the current panel mode, or an unset optional when no child widget has been
	 * constructed yet (e.g. during the one-tick deferred initialization).
	 */
	virtual TOptional<ETakeRecorderPanelMode> GetMode() const = 0;

	/**
	 * Replaces the tab content with a new STakeRecorderPanel configured for recording,
	 * using the provided level sequence as the base sequence.
	 */
	virtual void SetupForRecording(ULevelSequence* LevelSequenceAsset) = 0;

	/**
	 * Replaces the tab content with a new STakeRecorderPanel configured for recording,
	 * using the provided take preset as the starting configuration.
	 * Pass nullptr to use the default recording setup.
	 */
	virtual void SetupForRecording(UTakePreset* BasePreset) = 0;

	/**
	 * Replaces the tab content with a new STakeRecorderPanel that will record directly
	 * into the provided existing level sequence, appending new data rather than creating
	 * a fresh take.
	 */
	virtual void SetupForRecordingInto(ULevelSequence* LevelSequenceAsset) = 0;

	/**
	 * Opens the provided take preset for editing in an STakePresetAssetEditor.
	 */
	virtual void SetupForEditing(UTakePreset* Preset) = 0;

#if WITH_EDITOR
	/**
	 * Replaces the tab content with an STakePresetAssetEditor driven by the provided
	 * toolkit.
	 */
	virtual void SetupForEditing(TSharedPtr<FTakePresetToolkit> InToolkit) = 0;
#endif // WITH_EDITOR

	/**
	 * Replaces the tab content with a read-only STakeRecorderPanel showing the provided
	 * level sequence.
	 */
	virtual void SetupForViewing(ULevelSequence* LevelSequence) = 0;

	/**
	 * Returns the level sequence currently active in the panel or asset editor, or nullptr
	 * if neither is valid.
	 */
	virtual ULevelSequence* GetLevelSequence() const = 0;

	/**
	 * Returns the level sequence produced by the most recent completed recording, or
	 * nullptr if no recording has completed or no panel is active.
	 */
	virtual ULevelSequence* GetLastRecordedLevelSequence() const = 0;

	/**
	 * Returns the take metadata for the active level sequence, sourced from either the
	 * asset editor's sequence or the recorder panel. Returns nullptr if no metadata exists
	 * or no child widget is active.
	 */
	virtual UTakeMetaData* GetTakeMetaData() const = 0;

	/**
	 * Returns the current recording frame rate as reported by the cockpit widget.
	 * Returns a default-constructed FFrameRate if no panel or cockpit is available.
	 */
	virtual FFrameRate GetFrameRate() const = 0;

	/**
	 * Sets the recording frame rate on the cockpit widget. Has no effect if no panel or
	 * cockpit is currently active.
	 */
	virtual void SetFrameRate(FFrameRate InFrameRate) = 0;

	/**
	 * Configures the cockpit to derive its frame rate from the current timecode source
	 * when bInFromTimecode is true. Has no effect if no panel or cockpit is currently active.
	 */
	virtual void SetFrameRateFromTimecode(bool bInFromTimecode) = 0;

	/**
	 * Returns the UTakeRecorderSources associated with the active level sequence, looked
	 * up via ITakeRecorderSourcesManager. Returns nullptr if no level sequence is active.
	 */
	virtual UTakeRecorderSources* GetSources() const = 0;

	/**
	 * Starts a recording session via the cockpit widget. Does
	 * nothing if the cockpit is unavailable, a recording is already in progress, the panel
	 * is in review mode, or CanStartRecording returns false.
	 */
	virtual void StartRecording() const = 0;

	/**
	 * Stops the active recording session via the cockpit widget. 
	 * Does nothing if the cockpit is unavailable or no recording is in progress.
	 */
	virtual void StopRecording() const = 0;

	/**
	 * Clears any pending take state from the active recorder panel. Has no
	 * effect if no panel is currently active.
	 */
	virtual void ClearPendingTake() = 0;

	/**
	 * Returns whether a recording can be started, delegating to the cockpit widget.
	 * Populates ErrorText with a human-readable reason when returning false. Returns false
	 * with an appropriate message if no panel or cockpit is available.
	 */
	virtual bool CanStartRecording(FText& ErrorText) const = 0;

	/** Returns a weak pointer to the active STakeRecorderPanel, or an invalid pointer if the asset editor is showing instead. */
	virtual const TWeakPtr<STakeRecorderPanel>& GetTakeRecorderPanel() const = 0;
};
