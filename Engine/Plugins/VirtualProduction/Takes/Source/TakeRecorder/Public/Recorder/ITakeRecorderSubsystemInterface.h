// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "TakeRecorderParameters.h"
#include "UObject/Interface.h"

#include "ITakeRecorderSubsystemInterface.generated.h"

class UTakeRecorderNamingTokensData;
class UTakeRecorderSources;
class UTakeRecorderSource;
class UTakeMetaData;
enum class ETakeRecorderState : uint8;

UINTERFACE(MinimalAPI, NotBlueprintable)
class UTakeRecorderSubsystemInterface : public UInterface
{
	GENERATED_BODY()
};

/** Interface for the public Take Recorder Subsystem and the private implementation. */
class ITakeRecorderSubsystemInterface
{
	GENERATED_BODY()
	
public:
	virtual void SetTargetSequence(const FTakeRecorderSequenceParameters& InData = FTakeRecorderSequenceParameters()) = 0;
	virtual void SetRecordIntoLevelSequence(ULevelSequence* LevelSequence) = 0;
	virtual bool CanReviewLastRecording() const = 0;
	virtual bool ReviewLastRecording() = 0;
	virtual bool StartRecording(bool bOpenSequencer = true, bool bShowErrorMessage = true) = 0;
	virtual void StopRecording() = 0;
	virtual void CancelRecording(bool bShowConfirmMessage = true) = 0;
	virtual void ResetToPendingTake() = 0;
	virtual void ClearPendingTake() = 0;
	virtual UTakePreset* GetPendingTake() const = 0;
	virtual void RevertChanges() = 0;
	virtual bool IsRecordingUsingExistingSequence() const = 0;
	
	// ~Begin Sources
	virtual UTakeRecorderSource* AddSource(const TSubclassOf<UTakeRecorderSource> InSourceClass) = 0;
	virtual void RemoveSource(UTakeRecorderSource* InSource) = 0;
	virtual void ClearSources() = 0;
	virtual UTakeRecorderSources* GetSources() const = 0;
	virtual TArrayView<UTakeRecorderSource* const> GetAllSources() const = 0;
	virtual TArray<UTakeRecorderSource*> GetAllSourcesCopy() const = 0;
	virtual UTakeRecorderSource* GetSourceByClass(const TSubclassOf<UTakeRecorderSource> InSourceClass) const = 0;
	virtual void AddSourceForActor(AActor* InActor, bool bReduceKeys = true, bool bShowProgress = true) = 0;
	virtual void RemoveActorFromSources(AActor* InActor) = 0;
	virtual AActor* GetSourceActor(UTakeRecorderSource* InSource) const = 0;
	// ~End Sources
	
	virtual ETakeRecorderState GetState() const = 0;
	virtual void SetTakeNumber(int32 InNewTakeNumber, bool bEmitChanged = true) = 0;
	virtual int32 GetNextTakeNumber(const FString& InSlate) const = 0;
	virtual void GetNumberOfTakes(const FString& InSlate, int32& OutMaxTake, int32& OutNumTakes) const = 0;
	virtual void IncrementTakeNumber() = 0;
	virtual TArray<FAssetData> GetSlates(FName InPackagePath = NAME_None) const = 0;
	virtual void SetSlateName(const FString& InSlateName, bool bEmitChanged = true) = 0;
	virtual bool MarkFrame() = 0;
	virtual FFrameRate GetFrameRate() const = 0;
	virtual void SetFrameRate(FFrameRate InFrameRate) = 0;
	virtual void SetFrameRateFromTimecode() = 0;
	virtual void ImportPreset(const FAssetData& InPreset) = 0;
	virtual bool IsReviewing() const = 0;
	virtual bool IsRecording() const = 0;
	virtual bool TryGetSequenceCountdown(float& OutValue) const = 0;
	virtual void SetSequenceCountdown(float InSeconds) = 0;
	virtual TArray<UObject*> GetSourceRecordSettings(UTakeRecorderSource* InSource) const = 0;
	virtual FTakeRecorderParameters GetGlobalRecordSettings() const = 0;
	virtual void SetGlobalRecordSettings(const FTakeRecorderParameters& InParameters) = 0;
	virtual UTakeMetaData* GetTakeMetaData() const = 0;
	virtual ULevelSequence* GetLevelSequence() const = 0;
	virtual ULevelSequence* GetSuppliedLevelSequence() const = 0;
	virtual ULevelSequence* GetRecordingLevelSequence() const = 0;
	virtual ULevelSequence* GetRecordIntoLevelSequence() const = 0;
	virtual ULevelSequence* GetLastRecordedLevelSequence() const = 0;
	virtual UTakePreset* GetTransientPreset() const = 0;
	virtual ETakeRecorderMode GetTakeRecorderMode() const = 0;
	virtual UTakeRecorderNamingTokensData* GetNamingTokensData() const = 0;
	virtual bool HasPendingChanges() const = 0;
	virtual UClass* GetPresetTargetRecordClass() const = 0;
	virtual void SetPresetTargetRecordClass(const TSubclassOf<ULevelSequence> InClass) = 0;
};
