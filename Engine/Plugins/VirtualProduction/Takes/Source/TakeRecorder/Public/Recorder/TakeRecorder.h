// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/StrongObjectPtr.h"
#include "Recorder/TakeRecorderParameters.h"
#include "Serializers/MovieSceneManifestSerialization.h"

#if WITH_EDITOR
#include "ISequencer.h"
#endif // WITH_EDITOR

#include "TakeRecorder.generated.h"

#define UE_API TAKERECORDER_API

#if WITH_EDITOR
class ISequencer;
#endif // WITH_EDITOR

class UTakePreset;
class UTakeMetaData;
class UTakeRecorder;
class ULevelSequence;
class SNotificationItem;
class UTakeRecorderSources;
class UTakeRecorderOverlayWidget;
class UMovieSceneSequence;

namespace UE::MovieScene
{
	struct FScopedVolatilityManagerSuppression;
}

UENUM(BlueprintType)
enum class ETakeRecorderState : uint8
{
	PreInitialization,
	CountingDown,
	PreRecord,
	TickingAfterPre,
	Started,
	Stopped,
	Cancelled,
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTakeRecordingPreInitialize, UTakeRecorder*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTakeRecordingInitialized, UTakeRecorder*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTakeRecordingStarted, UTakeRecorder*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTakeRecordingStopped, UTakeRecorder*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTakeRecordingFinished, UTakeRecorder*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTakeRecordingCancelled, UTakeRecorder*);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStartPlayFrameModified, UTakeRecorder*, const FFrameNumber& StartFrame);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTickRecording, UTakeRecorder*, const FQualifiedFrameTime&);
DECLARE_MULTICAST_DELEGATE(FOnPendingTakeCleared);

DECLARE_DELEGATE_RetVal_OneParam(FTakeRecorderParameters, FTakeRecorderParameterDelegate,const FTakeRecorderParameters&);

struct FTakeRecorderParameterOverride
{
	UE_API void RegisterHandler(FName OverrideName, FTakeRecorderParameterDelegate Delegate);
	UE_API void UnregisterHandler(FName OverrideName);

	TMap<FName, FTakeRecorderParameterDelegate> Delegates;
};

DECLARE_LOG_CATEGORY_EXTERN(ManifestSerialization, Verbose, All);

UCLASS(MinimalAPI, BlueprintType)
class UTakeRecorder : public UObject
{
public:

	GENERATED_BODY()

	UE_API UTakeRecorder(const FObjectInitializer& ObjInit);

public:

	/**
	 * Retrieve the currently active take recorder instance
	 */
	static UE_API UTakeRecorder* GetActiveRecorder();

	/**
	 * Retrieve a multi-cast delegate that is triggered when a new recording begins
	 */
	static UE_API FOnTakeRecordingInitialized& OnRecordingInitialized();

	/**
	 * On take initialization overrides provide a mechanism to adjust parameter values when
	 * a new take is initialized.
	 */
	static UE_API FTakeRecorderParameterOverride& TakeInitializeParameterOverride();

public:

	/**
	 * Access the number of seconds remaining before this recording will start
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder")
	float GetCountdownSeconds() const
	{
		return CountdownSeconds;
	}

	/**
	 * Sets the current countdown time in seconds.
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder")
	UE_API void SetCountdown(float InSeconds);

	/**
	 * Access the sequence asset that this recorder is recording into
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder")
	ULevelSequence* GetSequence() const
	{
		return SequenceAsset;
	}

	/**
	 * Get the current state of this recorder
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder")
	ETakeRecorderState GetState() const
	{
		return State;
	}

	/**
	 * Disable saving data on tick event. This make the TakeRecord a no-op.
	 */
	UE_API void SetDisableSaveTick( bool );

public:

	/**
	 * Initialize a new recording with the specified parameters. Fails if another recording is currently in progress.
	 *
	 * @param LevelSequenceBase   A level sequence to use as a base set-up for the take
	 * @param Sources             The sources to record from
	 * @param MetaData            Meta data to store on the recorded level sequence asset for this take
	 * @param InParameters        Configurable parameters for this instance of the recorder
	 * @param OutError            Error string to receive for context
	 * @return True if the recording process was successfully initialized, false otherwise
	 */
	UE_API bool Initialize(ULevelSequence* LevelSequenceBase, UTakeRecorderSources* Sources, UTakeMetaData* MetaData, const FTakeRecorderParameters& InParameters, FText* OutError = nullptr);

	/**
	 * Called to stop the recording
	 */
	UE_API void Stop();

	/**
	 * Called to cancel the recording
	 */
	UE_API void Cancel();

	/**
	 * Retrieve a multi-cast delegate that is triggered before initialization occurs (ie. when the recording button is pressed and before the countdown starts)
	 */
	UE_API FOnTakeRecordingPreInitialize& OnRecordingPreInitialize();

	/**
	 * Retrieve a multi-cast delegate that is triggered when this recording starts
	 */
	UE_API FOnTakeRecordingStarted& OnRecordingStarted();

	/**
	 * Retrieve a multi-cast delegate that is triggered when this recording is stopped
	 */
	UE_API FOnTakeRecordingStopped& OnRecordingStopped();

	/**
	 * Retrieve a multi-cast delegate that is triggered when this recording finishes
	 */
	UE_API FOnTakeRecordingFinished& OnRecordingFinished();

	/**
	 * Retrieve a multi-cast delegate that is triggered when this recording is cancelled
	 */
	UE_API FOnTakeRecordingCancelled& OnRecordingCancelled();

	/**
	 * Retrieve a multi-cast delegate that is triggered when a delta time has been applied to the Movie Scene
	 */
	UE_API FOnStartPlayFrameModified& OnStartPlayFrameModified();

	/**
	 * Retrieve a multi-cast delegate that is triggered when this recording ticks
	 */
	UE_API FOnTickRecording& OnTickRecording();

private:

	/**
	 * Called after the countdown to PreRecord
	 */
	UE_API void PreRecord();

	/**
	 * Called after PreRecord To Start
	 */
	UE_API void Start();

	/*
	 * Stop or cancel
	 */
	UE_API void StopInternal(const bool bCancelled);

	/**
	 * Ticked by a tickable game object to performe any necessary time-sliced logic
	 */
	UE_API void Tick(float DeltaTime);

#if WITH_EDITOR
	/**
	 * Called if we're currently recording a PIE world that has been shut down or if we start PIE in a non-PIE world. Bound in Initialize, and unbound in Stop.
	 */
	UE_API void HandlePIE(bool bIsSimulating);
#endif // WITH_EDITOR

	/**
	 * Create a new destination asset to record into based on the parameters
	 */
	UE_API bool CreateDestinationAsset(const TCHAR* AssetPathFormat, ULevelSequence* LevelSequenceBase, UTakeRecorderSources* Sources, UTakeMetaData* MetaData, FText* OutError, bool bOpenSequencer = true);

	/**
	 * Setup a existing asset to record into based on the parameters
	 */
	UE_API bool SetupDestinationAsset(const FTakeRecorderParameters& InParameters, ULevelSequence* LevelSequenceBase, UTakeRecorderSources* Sources, UTakeMetaData* MetaData, FText* OutError, bool bOpenSequencer = true);

	/**
	 * Discovers the source world to record from, and initializes it for recording
	 */
	UE_API void DiscoverSourceWorld();

	/**
	 * Called to perform any initialization based on the user-provided parameters
	 */
	UE_API void InitializeFromParameters();

	/**
     * Returns true if notification widgets should be shown when recording.
	 * It takes into account TakeRecorder project settings, the command line, and global unattended settings.
     */
	UE_API bool ShouldShowNotifications();

private:

	/* The time at which to record. Taken from the Sequencer global time, otherwise based on timecode */
	UE_API FQualifiedFrameTime GetRecordTime() const;

	/** Called by Tick and Start to make sure we record at start */
	UE_API void InternalTick(float DeltaTime);

	UE_API virtual UWorld* GetWorld() const override;

private:

	/** The number of seconds remaining before Start() should be called */
	float CountdownSeconds;

	/** The state of this recorder instance */
	ETakeRecorderState State;

	/** FFrameTime in MovieScene Resolution we are at*/
	FFrameTime CurrentFrameTime;

	/** Timecode at the start of recording */
	FTimecode TimecodeAtStart;

	/** Optional frame to stop recording at*/
	TOptional<FFrameNumber> StopRecordingFrame;

	/** The asset that we should output recorded data into */
	UPROPERTY(transient)
	TObjectPtr<ULevelSequence> SequenceAsset;

	/** The overlay widget for this recording */
	UPROPERTY(transient)
	TObjectPtr<UTakeRecorderOverlayWidget> OverlayWidget;

	/** The world that we are recording within */
	UPROPERTY(transient)
	TWeakObjectPtr<UWorld> WeakWorld;

	/** Parameters for the recorder - marked up as a uproperty to support reference collection */
	UPROPERTY()
	FTakeRecorderParameters Parameters;

	/** Anonymous array of cleanup functions to perform when a recording has finished */
	TArray<TFunction<void()>> OnStopCleanup;

	/** Triggered before the recorder is initialized */
	FOnTakeRecordingPreInitialize OnRecordingPreInitializeEvent;

	/** Triggered when this recorder starts */
	FOnTakeRecordingStarted OnRecordingStartedEvent;

	/** Triggered when this recorder is stopped */
	FOnTakeRecordingStopped OnRecordingStoppedEvent;

	/** Triggered when this recorder finishes */
	FOnTakeRecordingFinished OnRecordingFinishedEvent;

	/** Triggered when this recorder is cancelled */
	FOnTakeRecordingCancelled OnRecordingCancelledEvent;

	/** Triggered when the movie scene is adjusted. */
	FOnStartPlayFrameModified OnFrameModifiedEvent;

	/** Triggered when this recorder ticks */
	FOnTickRecording OnTickRecordingEvent;

#if WITH_EDITOR
	/** Sequencer ptr that controls playback of the destination asset during the recording */
	TWeakPtr<ISequencer> WeakSequencer;
#endif // WITH_EDITOR

	/** Due a few ticks after the pre so we are set up with asset creation */
	int32 NumberOfTicksAfterPre;

	friend class FTickableTakeRecorder;

private:

	/**
	 * Set the currently active take recorder instance
	 */
	static UE_API bool SetActiveRecorder(UTakeRecorder* NewActiveRecorder);

	/** Event to trigger when a new recording is initialized */
	static UE_API FOnTakeRecordingInitialized OnRecordingInitializedEvent;

private:

	FManifestSerializer ManifestSerializer;

#if WITH_EDITOR
	EAllowEditsMode CachedAllowEditsMode;
	EAutoChangeMode CachedAutoChangeMode;
#endif // WITH_EDITOR

	EUpdateClockSource CachedClockSource;
	
	TRange<FFrameNumber> CachedPlaybackRange;

	TUniquePtr<UE::MovieScene::FScopedVolatilityManagerSuppression> CompileSuppression;
};

#undef UE_API
