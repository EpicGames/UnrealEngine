// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Recorder/TakeRecorder.h"
#include "TakeRecorderBlueprintLibrary.generated.h"

#define UE_API TAKERECORDER_API

class UTakeRecorder;
class UTakeMetaData;
class ULevelSequence;
class UTakeRecorderPanel;

DECLARE_DYNAMIC_DELEGATE( FOnTakeRecorderPanelChanged );
DECLARE_DYNAMIC_DELEGATE( FOnTakeRecorderPreInitialize );
DECLARE_DYNAMIC_DELEGATE( FOnTakeRecorderStarted );
DECLARE_DYNAMIC_DELEGATE( FOnTakeRecorderStopped );
DECLARE_DYNAMIC_DELEGATE_OneParam( FOnTakeRecorderFinished, ULevelSequence*, SequenceAsset );
DECLARE_DYNAMIC_DELEGATE( FOnTakeRecorderCancelled );
DECLARE_DYNAMIC_DELEGATE_OneParam( FOnTakeRecorderMarkedFrameAdded, const FMovieSceneMarkedFrame&, MarkedFrame );

UCLASS(MinimalAPI)
class UTakeRecorderBlueprintLibrary : public UBlueprintFunctionLibrary
{
public:

	GENERATED_BODY()


	/**
	 * Is the Take Recorder enabled in the build
	 */
	UFUNCTION(BlueprintPure, Category="Take Recorder")
	static UE_API bool IsTakeRecorderEnabled();

	/**
	 * Start a new recording using the specified parameters. Will fail if a recording is currently in progress
	 *
	 * @param LevelSequence         The base level sequence to use for the recording. Will be played back during the recording and duplicated to create the starting point for the resulting asset.
	 * @param Sources               The sources to use for the recording
	 * @param MetaData              Meta-data pertaining to this recording, duplicated into the resulting recorded sequence
	 * @param Parameters            Configurable parameters for this recorder instance
	 * @return The recorder responsible for the recording, or None if a a recording could not be started
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder")
	static UE_API UTakeRecorder* StartRecording(ULevelSequence* LevelSequence, UTakeRecorderSources* Sources, UTakeMetaData* MetaData, const FTakeRecorderParameters& Parameters);


	/**
	 * Get the default recorder parameters according to the project and user settings
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder")
	static UE_API FTakeRecorderParameters GetDefaultParameters();


	/**
	 * Set the default recorder parameters
	 */
	UFUNCTION(BlueprintCallable, Category = "Take Recorder")
	static UE_API void SetDefaultParameters(const FTakeRecorderParameters& DefaultParameters);

	/**
	 * Check whether a recording is currently active
	 */
	UFUNCTION(BlueprintPure, Category="Take Recorder")
	static UE_API bool IsRecording();


	/**
	 * Retrieve the currently active recorder, or None if there none are active
	 */
	UFUNCTION(BlueprintPure, Category="Take Recorder")
	static UE_API UTakeRecorder* GetActiveRecorder();


	/**
	 * Stop recording if there is a recorder currently active
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder")
	static UE_API void StopRecording();


	/**
	 * Cancel recording if there is a recorder currently active
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder")
	static UE_API void CancelRecording();


	/**
	 * Get the currently open take recorder panel, if one is open
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Take Recorder")
	static UE_API UTakeRecorderPanel* GetTakeRecorderPanel();


	/**
	 * Get the currently open take recorder panel, if one is open, opening a new one if not
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder")
	static UE_API UTakeRecorderPanel* OpenTakeRecorderPanel();


	/** Called when a Take Panel is constructed or destroyed. */
	UFUNCTION(BlueprintCallable, Category="Take Recorder", meta=(DisplayName="Set On Take Recorder Panel Changed"))
	static UE_API void SetOnTakeRecorderPanelChanged(FOnTakeRecorderPanelChanged OnTakeRecorderPanelChanged);

	UE_DEPRECATED(5.4, "Please use TakeRecorderSubsystem::TakeRecorderPreInitialize")
	UFUNCTION(BlueprintCallable, Category = "Take Recorder", meta = (DeprecatedFunction, DeprecationMessage = "Please use TakeRecorderSubsystem::TakeRecorderPreInitialize"))
	static UE_API void SetOnTakeRecorderPreInitialize(FOnTakeRecorderPreInitialize OnTakeRecorderPreInitialize);

	UE_DEPRECATED(5.4, "Please use TakeRecorderSubsystem::TakeRecorderStarted")
	UFUNCTION(BlueprintCallable, Category = "Take Recorder", meta = (DeprecatedFunction, DeprecationMessage = "Please use TakeRecorderSubsystem::TakeRecorderStarted"))
	static UE_API void SetOnTakeRecorderStarted(FOnTakeRecorderStarted OnTakeRecorderStarted);

	UE_DEPRECATED(5.4, "Please use TakeRecorderSubsystem::TakeRecorderStopped")
	UFUNCTION(BlueprintCallable, Category = "Take Recorder", meta = (DeprecatedFunction, DeprecationMessage = "Please use TakeRecorderSubsystem::TakeRecorderStopped"))
	static UE_API void SetOnTakeRecorderStopped(FOnTakeRecorderStopped OnTakeRecorderStopped);

	UE_DEPRECATED(5.4, "Please use TakeRecorderSubsystem::TakeRecorderFinished")
	UFUNCTION(BlueprintCallable, Category = "Take Recorder", meta = (DeprecatedFunction, DeprecationMessage = "Please use TakeRecorderSubsystem::TakeRecorderFinished"))
	static UE_API void SetOnTakeRecorderFinished(FOnTakeRecorderFinished OnTakeRecorderFinished);

	UE_DEPRECATED(5.4, "Please use TakeRecorderSubsystem::TakeRecorderCancelled")
	UFUNCTION(BlueprintCallable, Category = "Take Recorder", meta = (DeprecatedFunction, DeprecationMessage = "Please use TakeRecorderSubsystem::TakeRecorderCancelled"))
	static UE_API void SetOnTakeRecorderCancelled(FOnTakeRecorderCancelled OnTakeRecorderCancelled);

	UE_DEPRECATED(5.4, "Please use TakeRecorderSubsystem::TakeRecorderMarkedFrameAdded")
	UFUNCTION(BlueprintCallable, Category = "Take Recorder", meta = (DeprecatedFunction, DeprecationMessage = "Please use TakeRecorderSubsystem::TakeRecorderMarkedFrameAdded"))
	static UE_API void SetOnTakeRecorderMarkedFrameAdded(FOnTakeRecorderMarkedFrameAdded OnTakeRecorderMarkedFrameAdded);

	static UE_API void OnTakeRecorderPreInitialize();
	static UE_API void OnTakeRecorderStarted();
	static UE_API void OnTakeRecorderStopped();
	static UE_API void OnTakeRecorderFinished(ULevelSequence* InSequenceAsset);
	static UE_API void OnTakeRecorderCancelled();
	static UE_API void OnTakeRecorderMarkedFrameAdded(const FMovieSceneMarkedFrame& InMarkedFrame);

	/**
	 * Internal function to assign a new take recorder panel singleton.
	 * NOTE: Only to be called by STakeRecorderTabContent::Construct.
	 */
	static UE_API void SetTakeRecorderPanel(UTakeRecorderPanel* InNewPanel);
};

#undef UE_API
