// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CatchupFixedRateCustomTimeStep.h"
#include "Engine/EngineCustomTimeStep.h"
#include "TakeRecorderHitchProtectionParameters.generated.h"

USTRUCT(BlueprintType)
struct FTakeRecorderHitchProtectionParameters
{
	GENERATED_BODY()

	/**
	 * Attempt to prevent hitches in recorded data. If a frame were to be dropped, Take Recorder will instead slow its evaluation and attempt to
	 * catch up. This is achieved by using a fixed delta time time-step, which will use the FPS value set in Take Recorder.
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	bool bEnableHitchProtection = true;

	/**
	 * Determines the number of frames in the buffer used by linear regression.
	 * The final number of frames is FPS * RegressionBufferSizeInSeconds.
	 * 
	 * Without hitch protection, each frame is marked with the timecode reported by UTimecodeProvider; when a hitch occurs, the timecode hitches by
	 * the same amount.
	 * 
	 * With hitch protection is enabled, the engine is set up with a fixed delta time step. Every frame, the timecode is sampled from the timecode
	 * provider, stamped with the current physical time, and placed in the buffer. When frame n takes longer to evaluate than expected ("hitches"), the
	 * phyiscal time will have moved by a certain amount but the world's simulation time will have remained the same (as simulation time advances only
	 * when the engine ticks). Then, frame n+1 uses the world's simulation time to perform linear regression on what the frame's timecode should be.
	 */
	UPROPERTY(Config, AdvancedDisplay, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder", meta = (EditCondition = "bEnableHitchProtection", EditConditionHides, ClampMin = 0.01666, ClampMax = 10))
	float RegressionBufferSizeInSeconds = 1.f;

	/** The custom engine timestep class to use. Allows you to customize the behavior. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder", meta=(EditCondition = "bEnableHitchProtection", EditConditionHides, AllowAbstract = "false"))
	TSoftClassPtr<UCatchupFixedRateCustomTimeStep> CustomTimestep = UCatchupFixedRateCustomTimeStep::StaticClass();

	/**
	 * Maximum allowed delay when Take Recorder is attempting to catch up during protection.
	 * 
	 * If Take Recorder falls behind further than this value, it will not accrue additional delay and will drop frames. The result is that the next
	 * frame immediately jumps to the target timecode in order to attempt to catch up.
	 *
	 * Larger values will allow for jumps in timecode in the final data but will also result in possible larger delays in evaluation.
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder", meta = (EditCondition = "bEnableHitchProtection", EditConditionHides))
	double MaxCatchupSeconds = 8.f;
};