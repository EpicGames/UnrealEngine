// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "TakeRecorderPanel.generated.h"

#define UE_API TAKERECORDER_API

class UTakePreset;
class ULevelSequence;
class UTakeMetaData;
class UTakeRecorderSources;

class ITakeRecorderTabContent;

UENUM()
enum class ETakeRecorderPanelMode : uint8
{
	/** The panel is setting up a new recording */
	NewRecording,
	/** The panel is setting up recording into an existing level sequence */
	RecordingInto,
	/** The panel is editing a Take Preset asset */
	EditingPreset,
	/** The panel is reviewing a previously recorded take */
	ReviewingRecording,
};

/**
 * Take recorder UI panel interop object
 */
UCLASS(MinimalAPI)
class UTakeRecorderPanel : public UObject
{
public:

	GENERATED_BODY()

	/**
	 * Get the mode that the panel is currently in
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel")
	UE_API ETakeRecorderPanelMode GetMode() const;

	/**
	 * Setup this panel such that it is ready to start recording using the specified
	 * take preset as a template for the recording.
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel", DisplayName="Set Mode (Recording w/ Take Preset)")
	UE_API void SetupForRecording_TakePreset(UTakePreset* TakePresetAsset);

	/**
	 * Setup this panel such that it is ready to start recording using the specified
	 * level sequence asset as a template for the recording.
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel", DisplayName="Set Mode (Recording w/ Level Sequence)")
	UE_API void SetupForRecording_LevelSequence(ULevelSequence* LevelSequenceAsset);

	/**
	 * Setup this panel such that it is ready to start recording using the specified
	 * level sequence asset to record into.
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel", DisplayName="Set Mode (Recording into Level Sequence)")
	UE_API void SetupForRecordingInto_LevelSequence(ULevelSequence* LevelSequenceAsset);

	/**
	 * Setup this panel as an editor for the specified take preset asset.
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel", DisplayName="Set Mode (Editing Take Preset)")
	UE_API void SetupForEditing(UTakePreset* TakePreset);

	/**
	 * Setup this panel as a viewer for a previously recorded take.
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel", DisplayName="Set Mode (Read-Only Level Sequence)")
	UE_API void SetupForViewing(ULevelSequence* LevelSequenceAsset);

	/*
	 * Clear the pending take level sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Take Recorder|Panel", DisplayName = "Clear Pending Take")
	UE_API void ClearPendingTake();

	/**
	 * Access the level sequence for this take
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel")
	UE_API ULevelSequence* GetLevelSequence() const;


	/**
	 * Access the last level sequence that was recorded
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel")
	UE_API ULevelSequence* GetLastRecordedLevelSequence() const;


	/**
	 * Access take meta data for this take
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel")
	UE_API UTakeMetaData* GetTakeMetaData() const;


	/**
	 * Access the frame rate for this take
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel")
	UE_API FFrameRate GetFrameRate() const;


	/**
	* Set the frame rate for this take
	*/
	UFUNCTION(BlueprintCallable, Category = "Take Recorder|Panel")
	UE_API void SetFrameRate(FFrameRate InFrameRate);


	/**
	* Set if the frame rate is set from the Timecode frame rate
	*/
	UFUNCTION(BlueprintCallable, Category = "Take Recorder|Panel")
	UE_API void SetFrameRateFromTimecode(bool  bInFromTimecode);


	/**
	 * Access the sources that are to be (or were) used for recording this take
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel")
	UE_API UTakeRecorderSources* GetSources() const;


	/**
	 * Start recording with the current take
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel")
	UE_API void StartRecording() const;


	/**
	 * Stop recording with the current take
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel")
	UE_API void StopRecording() const;

	/**
	 * Whether the panel is ready to start recording
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder|Panel")
	UE_API bool CanStartRecording(FText& OutErrorText) const;

public:

	/*~ Native interface ~*/

	/**
	 * Initialize this object with a weak pointer to the tab content from which all the UI state can be retrieved
	 */
	UE_API void InitializePanel(TWeakPtr<ITakeRecorderTabContent> InTabContent);

	/**
	 * Check whether this panel is still open or not
	 */
	UE_API bool IsPanelOpen() const;

	/**
	 * Invalidate this object by reporting that it is no longer open. Any subsequent scripting interactions will result in an error
	 */
	UE_API void ClosePanel();

private:

	UE_API bool ValidateTabContent() const;

	TWeakPtr<ITakeRecorderTabContent> WeakTabContent;
};

#undef UE_API
