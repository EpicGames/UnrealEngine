// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "MovieSceneFwd.h"
#include "Recorder/TakeRecorderHitchProtectionParameters.h"
#include "TakeRecorderSpawnableBehavior.h"
#include "TrackRecorders/IMovieSceneTrackRecorderHost.h"
#include "TakeRecorderParameters.generated.h"

#define UE_API TAKERECORDER_API

class UTakePreset;

USTRUCT(BlueprintType)
struct FTakeRecorderUserParameters
{
	GENERATED_BODY()

	TAKERECORDER_API FTakeRecorderUserParameters();

	/** Whether to maximize the viewport (enter Immersive Mode) when recording */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category="User Settings")
	bool bMaximizeViewport;

	/** Delay that we will use before starting recording */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category="User Settings", DisplayName="Countdown", meta=(Units=s, ClampMin="0.0", UIMin="0.0", ClampMax="60.0", UIMax="60.0"))
	float CountdownSeconds;

	/** The engine time dilation to apply during the recording */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category="User Settings", meta=(Units=Multiplier, ClampMin="0.00001", UIMin="0.00001"))
	float EngineTimeDilation;

	/** Reset playhead to beginning of the playback range when starting recording */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "User Settings")
	bool bResetPlayhead;

	/** Automatically stop recording when reaching the end of the playback range */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "User Settings")
	bool bStopAtPlaybackEnd;

	/**
	 * Automatically stop recording when the current time code value loops over to before the start timecode value,
	 * e.g. when a rollover happens (23:59:59 -> 00:00:00:01), or if you started recording at 18:39:12:01 and timecode later jumps to 18:35:00:01.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "User Settings")
	bool bStopOnRollover;

	/** Recommended for use with recorded spawnables. Beware that changes to actor instances in the map after recording may alter the recording when played back */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category="User Settings")
	bool bRemoveRedundantTracks;

	/** Tolerance to use when reducing keys */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category="User Settings")
	float ReduceKeysTolerance;

	/** Whether to save recorded level sequences and assets when done recording */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "User Settings")
	bool bSaveRecordedAssets;

	/** Whether to lock the level sequence when done recording */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "User Settings")
	bool bAutoLock;

	/** Whether to incrementally serialize and store some data while recording*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "User Settings")
	bool bAutoSerialize;
};

USTRUCT(BlueprintType)
struct FTakeRecorderProjectParameters
{
	GENERATED_BODY()

	TAKERECORDER_API FTakeRecorderProjectParameters();

	/** Get the resolved root take save dir. The is the path that considers both the /Game and /Plugin based directories.  */
	TAKERECORDER_API FString GetResolvedRootTakeSaveDir() const;


	/** The take asset path, composed of the TakeRootSaveDir and the TakeSaveDir */
	TAKERECORDER_API FString GetTakeAssetPath() const;

	/** The root of the directory in which to save recorded takes. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder", meta = (ContentDir, NamingTokens, ForceShowPluginContent))
	FDirectoryPath RootTakeSaveDir;

	/**
	 * The name of the directory in which to save recorded takes. Supports any of the following format specifiers that will be substituted when a take is recorded:
	 * {day}       - The day of the timestamp for the start of the recording.
	 * {month}     - The month of the timestamp for the start of the recording.
	 * {year}      - The year of the timestamp for the start of the recording.
	 * {hour}      - The hour of the timestamp for the start of the recording.
	 * {minute}    - The minute of the timestamp for the start of the recording.
	 * {second}    - The second of the timestamp for the start of the recording.
	 * {take}      - The take number.
	 * {slate}     - The slate string.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder", meta = (NamingTokens))
	FString TakeSaveDir;

	/**
	 * The directory to record subscenes into.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder", meta = (NamingTokens))
	FString SubSequenceDirectory;
	
	/**
	 * The default name to use for the Slate information
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	FString DefaultSlate;

	/**
	 * The clock source to use when recording
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	EUpdateClockSource RecordingClockSource;

	/**
	 * If enabled, track sections will start at the current timecode. Otherwise, 0.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	bool bStartAtCurrentTimecode;

	/**
	 * If enabled, timecode will be recorded into each actor track
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	bool bRecordTimecode;

	/**
	* If enabled, each Source will be recorded into a separate Sequence and embedded in the Root Sequence will link to them via Subscenes track.
	* If disabled, all Sources will be recorded into the Root Sequence, and you will not be able to swap between various takes of specific source
	* using the Sequencer Take ui. This can still be done via copying and pasting between sequences if needed.
	*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	bool bRecordSourcesIntoSubSequences;

	/*
	 * If enabled, all recorded actors will be recorded to possessable object bindings in Sequencer. If disabled, all recorded actors will be 
	 * recorded to spawnable object bindings in Sequencer. This can be overridden per actor source.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	bool bRecordToPossessable;

	/** If spawnables should be overwritten during record. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder", meta = (ValidEnumValues = "OverwriteLegacy, CreateNew"))
	ETakeRecorderSpawnableOverwriteBehavior SpawnableOverwriteBehavior;

	/** The record behaviour of attachments and hierarchies. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder", meta = (ValidEnumValues = "None, Attachments, Hierarchy"))
	ETakeRecorderAttachRecordBehaviour AttachRecordBehaviour;
	
	/** List of property names for which movie scene tracks will always record. */
	UPROPERTY(config, EditAnywhere, Category = "Take Recorder")
	TArray<FTakeRecorderTrackSettings> DefaultTracks;

	/** Whether to show notification windows or not when recording */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Take Recorder")
	bool bShowNotifications;

	/** Whether to switch to the simple sequencer view whilst recording. */
	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category = "Take Recorder")
	bool bRecordWithSimpleSequencerView = false;
};

UENUM(BlueprintType)
enum class ETakeRecorderMode : uint8
{
	/* Record into a new sequence */
	RecordNewSequence,

	/* Record into an existing sequence */
	RecordIntoSequence
};

/**
 * Structure housing all configurable parameters for a take recorder instance
 */
USTRUCT(BlueprintType)
struct FTakeRecorderParameters
{
	GENERATED_BODY()

	TAKERECORDER_API FTakeRecorderParameters();

	UPROPERTY(BlueprintReadWrite, Category="Take Recorder")
	FTakeRecorderUserParameters User;

	UPROPERTY(BlueprintReadWrite, Category="Take Recorder")
	FTakeRecorderProjectParameters Project;

	UPROPERTY(BlueprintReadWrite, Category="Take Recorder")
	FTakeRecorderHitchProtectionParameters HitchProtection;

	UPROPERTY(BlueprintReadWrite, Category="Take Recorder")
	ETakeRecorderMode TakeRecorderMode;

	UPROPERTY(BlueprintReadWrite, Category = "Take Recorder")
	FFrameNumber StartFrame;

	/**
	 * Option to disable recording and saving of data. This can be used in a scenario where multiple clients are running
	 * take recorder, but only certain ones are set to process and save the data.
	 */
	UPROPERTY(Transient)
	bool bDisableRecordingAndSave = false;

	/** If sequencer should open when starting a recording. */
	UPROPERTY(BlueprintReadWrite, Category = "Take Recorder")
	bool bOpenSequencer = true;
};

/**
 * Parameters for initializing a new sequence.
 */
USTRUCT(Blueprintable)
struct FTakeRecorderSequenceParameters
{
	GENERATED_BODY()
	
	/** A preset asset to base the recording off */
	UPROPERTY(BlueprintReadWrite, Category = TakeRecorder)
	TObjectPtr<UTakePreset> BasePreset;

	/** A level sequence asset to base the recording off */
	UPROPERTY(BlueprintReadWrite, Category = TakeRecorder)
	TObjectPtr<ULevelSequence> BaseSequence;

	/** A level sequence asset to record into */
	UPROPERTY(BlueprintReadWrite, Category = TakeRecorder)
	TObjectPtr<ULevelSequence> RecordIntoSequence;

	/** A sequence that should be shown directly on the take recorder UI */
	UPROPERTY(BlueprintReadWrite, Category = TakeRecorder)
	TObjectPtr<ULevelSequence> SequenceToView;

	/** Checks whether we have any valid data present. */
	UE_API bool IsValid() const;
	
	UE_API bool operator==(const FTakeRecorderSequenceParameters& Other) const;
	UE_API bool operator!=(const FTakeRecorderSequenceParameters& Other) const;
};

#undef UE_API
