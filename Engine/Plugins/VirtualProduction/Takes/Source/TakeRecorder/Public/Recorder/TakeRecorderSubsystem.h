// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITakeRecorderSubsystemInterface.h"
#include "Subsystems/EngineSubsystem.h"
#include "TakeRecorder.h"
#include "TakeRecorderSources.h"

#include "TakeRecorderSubsystem.generated.h"

#define UE_API TAKERECORDER_API

struct FNamingTokensEvaluationData;
class ULevelSequence;
class UTakeRecorderNamingTokensData;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FTakeRecorderPreInitialize);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FTakeRecorderInitialized);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FTakeRecorderStarted);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FTakeRecorderStopped);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTakeRecorderFinished, ULevelSequence*, SequenceAsset);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FTakeRecorderCancelled);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTakeRecorderMarkedFrameAdded, const FMovieSceneMarkedFrame&, MarkedFrame);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FPendingTakeCleared);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FTakeRecorderSlateChanged, const FString&, Slate, UTakeMetaData*, TakeMetaData);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FTakeRecorderTakeNumberChanged, int32, TakeNumber, UTakeMetaData*, TakeMetaData);

/**
 * Dynamic delegates for source modification. These are reproductions of static ones declared under UTakeRecorderSource.
 * We do this so we can successfully leverage static delegates within the TakesCore module, firing when any change is made for
 * any source owner. We define new ones here so they can be blueprint assignable.
 */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnTakeRecorderSourceAddedDynamic, UTakeRecorderSource*, Source);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnTakeRecorderSourceRemovedDynamic, UTakeRecorderSource*, Source);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnTakeRecorderSourceModifiedDynamic, UTakeRecorderSource*, Source);

/**
* UTakeRecorderSubsystem Subsystem for Take Recorder.
* SetTargetSequence() needs to be called at least once prior to use. This will perform additional initialization
* and register the subsystem as tickable currently required to cache level metadata.
*/
UCLASS(MinimalAPI)
class UTakeRecorderSubsystem final : public UEngineSubsystem, public ITakeRecorderSubsystemInterface
{
	GENERATED_BODY()

public:
	
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	// ~Begin ITakeRecorderSubsystem interface
	
	/**
	 * Provide sequence data for this take recorder. This will also perform initialization of the subsystem.
	 * This must be called prior to any usage.
	 *
	 * @param InData The sequence parameters, which are mutually exclusive. They can all be null.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder, meta=(AutoCreateRefTerm="InData"))
	UE_API virtual void SetTargetSequence(const FTakeRecorderSequenceParameters& InData = FTakeRecorderSequenceParameters()) override;
	
	/** Set the record into level sequence. */
	UE_API virtual void SetRecordIntoLevelSequence(ULevelSequence* LevelSequence) override;

	/** Can we review the last recording? */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual bool CanReviewLastRecording() const override;
	
	/**
	 * Supply the last recording if it exists.
	 * @return true if it can be reviewed, false if it there isn't a recording to review.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual bool ReviewLastRecording() override;

	/**
	 * Begin a new recording.
	 * @param bOpenSequencer If sequencer should open when starting the recording.
	 * @param bShowErrorMessage If an error message should be displayed on failure.
	 * @return true if the recording was started successfully.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual bool StartRecording(bool bOpenSequencer = true, bool bShowErrorMessage = true) override;

	/**
	 * Stop an existing recording.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void StopRecording() override;

	/**
	 * Cancel an in-progress recording.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void CancelRecording(bool bShowConfirmMessage = true) override;

	/**
	 * Reset to the pending take.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void ResetToPendingTake() override;

	/**
	 * Clear the pending take.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void ClearPendingTake() override;

	/** Retrieve the pending take. This may be null. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual UTakePreset* GetPendingTake() const override;

	/** Revert any changes restoring the preset origin. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void RevertChanges() override;
	
	/** If we are using an existing sequence as a base or recording into an existing one. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual bool IsRecordingUsingExistingSequence() const override;
	
	// ~Begin Sources
	/** Add a source by a source class. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder, meta = (DeterminesOutputType = "InSourceClass"))
	UE_API virtual UTakeRecorderSource* AddSource(const TSubclassOf<UTakeRecorderSource> InSourceClass) override;
	
	/** Remove a given source. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void RemoveSource(UTakeRecorderSource* InSource) override;

	/** Remove all sources from the current sequence. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void ClearSources() override;

	/** Retrieve the sources. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual UTakeRecorderSources* GetSources() const override;
	
	/** Retrieve all sources for the current sequence. */
	UE_API virtual TArrayView<UTakeRecorderSource* const> GetAllSources() const override;

	/**
	 * Retrieves a copy of the list of sources that are being recorded. This is intended for Blueprint usages which cannot
	 * use TArrayView.
	 * DO NOT MODIFY THIS ARRAY, modifications will be lost.
	 */
	UFUNCTION(BlueprintPure, DisplayName = "Get Sources (Copy)", Category = TakeRecorder)
	UE_API virtual TArray<UTakeRecorderSource*> GetAllSourcesCopy() const override;

	/** Retrieve the first source of the given class. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder, meta = (DeterminesOutputType = "InSourceClass"))
	UE_API virtual UTakeRecorderSource* GetSourceByClass(const TSubclassOf<UTakeRecorderSource> InSourceClass) const override;

	/*
	 * Add an actor as a source.
	 * 
	 * @param InActor The actor that should be added to Sources. Note that this can include ALevelSequenceActors.
	 * @param bReduceKeys Enable/disable key reduction on all the sources registered
	 * @param bShowProgress Enable/disable the dialog box showing progress for the potentially slow parts of finalizing the take
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void AddSourceForActor(AActor* InActor, bool bReduceKeys = true, bool bShowProgress = true) override;

	/**
	 * Remove an actor from available sources.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void RemoveActorFromSources(AActor* InActor) override;

	/** Retrieve the actor from a source, if applicable. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual AActor* GetSourceActor(UTakeRecorderSource* InSource) const override;
	// ~End Sources
	
	/** Retrieve the current take recorder state. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual ETakeRecorderState GetState() const override;

	/**
	 * Directly set the take number.
	 * @param InNewTakeNumber The new take number to set.
	 * @param bEmitChanged Whether to broadcast events signaling the take number has changed.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void SetTakeNumber(int32 InNewTakeNumber, bool bEmitChanged = true) override;

	/** Compute the next take number given a slate. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual int32 GetNextTakeNumber(const FString& InSlate) const override;

	/** Find both the current maximum take value and the total number of takes for a given slate. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void GetNumberOfTakes(const FString& InSlate, int32& OutMaxTake, int32& OutNumTakes) const override;

	/** Calculate and apply the next take number. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void IncrementTakeNumber() override;
	
	/**
	 * Retrieve all slates.
	 * @param InPackagePath [Optional] Scope the search to a specific folder, recursively.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual TArray<FAssetData> GetSlates(FName InPackagePath = NAME_None) const override;
	
	/**
	 * Directly set the slate name.
	 * @param InSlateName The new slate name to set.
	 * @param bEmitChanged Whether to broadcast events signaling the slate name has changed.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void SetSlateName(const FString& InSlateName, bool bEmitChanged = true) override;

	/** Mark the current frame. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual bool MarkFrame() override;

	/**
	 * Access the frame rate for this take
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual FFrameRate GetFrameRate() const override;
	
	/**
	 * Set the frame rate for this take
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void SetFrameRate(FFrameRate InFrameRate) override;
	
	/**
	 * Set if the frame rate is set from the Timecode frame rate
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void SetFrameRateFromTimecode() override;

	/** Import a preset to the transient preset. */
	UE_API virtual void ImportPreset(const FAssetData& InPreset) override;

	/** If Take Recorder is currently reviewing. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual bool IsReviewing() const override;

	/** If Take Recorder is currently recording. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual bool IsRecording() const override;

	/**
	 * Retrieve the current sequence's countdown.
	 * @param OutValue The current value of the countdown.
	 * @return True if we are in a countdown sequence, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual bool TryGetSequenceCountdown(float& OutValue) const override;

	/**
	 * Sets the current sequence's countdown.
	 * @param InSeconds Time in seconds.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void SetSequenceCountdown(float InSeconds) override;

	/** Retrieve additional settings objects from a source. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual TArray<UObject*> GetSourceRecordSettings(UTakeRecorderSource* InSource) const override;

	/** Retrieve the global take recorder settings. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual FTakeRecorderParameters GetGlobalRecordSettings() const override;

	/** Set the global take recorder settings. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void SetGlobalRecordSettings(const FTakeRecorderParameters& InParameters) override;
	
	/** Retrieve the current meta data. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual UTakeMetaData* GetTakeMetaData() const override;

	/** Return the level sequence we are using. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual ULevelSequence* GetLevelSequence() const override;

	/** Retrieve the Supplied Level Sequence if it exists. */
	UE_API virtual ULevelSequence* GetSuppliedLevelSequence() const override;

	/** Retrieve the Recording Level Sequence if it exists. */
	UE_API virtual ULevelSequence* GetRecordingLevelSequence() const override;

	/** Retrieve the Record Into Level Sequence if it exists. */
	UE_API virtual ULevelSequence* GetRecordIntoLevelSequence() const override;

	/** Retrieve the Last Recorded Level Sequence if it exists. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual ULevelSequence* GetLastRecordedLevelSequence() const override;

	/** Retrieve the transient preset the subsystem is using. */
	UE_API virtual UTakePreset* GetTransientPreset() const override;
	
	/** The current take recorder mode. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual ETakeRecorderMode GetTakeRecorderMode() const override;

	/** Retrieve relevant Naming Tokens data for Take Recorder. */
	UE_API virtual UTakeRecorderNamingTokensData* GetNamingTokensData() const override;

	/** @return Whether there are any pending changes, which ClearPendingTake would discard. */
	UE_API virtual bool HasPendingChanges() const override;
	
	/** Retrieve the target record class of the Take Preset. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual UClass* GetPresetTargetRecordClass() const override;
	
	/** Set the target record class of the Take Preset. */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder)
	UE_API virtual void SetPresetTargetRecordClass(const TSubclassOf<ULevelSequence> InClass) override;
	// ~End ITakeRecorderSubsystem interface

	/**
	 * Retrieve Additional Settings Objects - eg. The Audio Input Device to access Microphone Recorder settings.
	 */
	UFUNCTION(BlueprintCallable, Category = TakeRecorder, meta = (DeterminesOutputType="SettingsType"))
	UE_API UObject* GetAdditionalSettings(TSubclassOf<UObject> SettingsType);
	
	/**
	 * Retrieve a multi-cast delegate that is triggered when a recording pre-initializes
	 */
	UE_API FOnTakeRecordingInitialized& GetOnRecordingPreInitializedEvent();
	/**
	 * Retrieve a multi-cast delegate that is triggered when a recording initializes
	 */
	UE_API FOnTakeRecordingInitialized& GetOnRecordingInitializedEvent();
	/**
	 * Retrieve a multi-cast delegate that is triggered when a recording starts
	 */
	UE_API FOnTakeRecordingStarted& GetOnRecordingStartedEvent();
	/**
	 * Retrieve a multi-cast delegate that is triggered when a recording is stopped
	 */
	UE_API FOnTakeRecordingStopped& GetOnRecordingStoppedEvent();
	/**
	 * Retrieve a multi-cast delegate that is triggered when a recording finishes
	 */
	UE_API FOnTakeRecordingFinished& GetOnRecordingFinishedEvent();
	/**
	 * Retrieve a multi-cast delegate that is triggered when a recording is cancelled.
	 */
	UE_API FOnTakeRecordingCancelled& GetOnRecordingCancelledEvent();
	/**
	 * Retrieve a multi-cast delegate that is triggered when a source is added.
	 */
	UE_API UTakeRecorderSources::FOnSourceAdded& GetOnRecordingSourceAddedEvent();
	/**
	 * Retrieve a multi-cast delegate that is triggered when a source is removed.
	 */
	UE_API UTakeRecorderSources::FOnSourceRemoved& GetOnRecordingSourceRemovedEvent();
	/**
	 * Retrieve a multi-cast delegate that is triggered when a source is modified.
	 */
	UE_API UTakeRecorderSources::FOnSourceModified& GetOnRecordingSourceModifiedEvent();

	/**
	 * Retrieve a multi-cast delegate that is triggered when the pending take is cleared.
	 */
	UE_API FOnPendingTakeCleared& GetOnPendingTakeClearedEvent();

private:
	// Friend class declared as a precaution in case the implementation needs to access private members.
	friend class UTakeRecorderSubsystemImplementation;
	
	/** The actual Take Recorder Subsystem implementation. */
	UPROPERTY()
	TScriptInterface<ITakeRecorderSubsystemInterface> Implementation;
	
	// These native delegates are declared elsewhere throughout Take Recorder and we reuse them here. However, it's important
	// the editor subsystem manage its own variants because the order they fire is important for listeners, such as the panel. For example, the
	// editor subsystem may store a strong reference to a sequence which needs to happen before the panel run its logic, otherwise the panel
	// may end up clearing the only strong reference prematurely. Therefor the panel can't directly listen to UTakeRecorder::OnRecordingStartedEvent,
	// but can listen to the editor subsystem's version.
	
	/** Multicast delegate when recording is pre-initialized. For native binding. */
	FOnTakeRecordingPreInitialize OnRecordingPreInitializeEvent;
	/** Multicast delegate when recording is initialized. For native binding. */
	FOnTakeRecordingInitialized ORecordingInitializedEvent;
	/** Multicast delegate for when a recording is started. For native binding. */
	FOnTakeRecordingStarted OnRecordingStartedEvent;
	/** Multicast delegate for when a recording has stopped. For native binding. */
	FOnTakeRecordingStopped OnRecordingStoppedEvent;
	/** Multicast delegate for when a recording has finished. For native binding. */
	FOnTakeRecordingFinished OnRecordingFinishedEvent;
	/** Multicast delegate for when a recording has been cancelled. For native binding. */
	FOnTakeRecordingCancelled OnRecordingCancelledEvent;
	/** Multicast delegate when any source has been added. For native binding. */
	UTakeRecorderSources::FOnSourceAdded OnRecordingSourceAddedEvent;
	/** Multicast delegate when any source has been added. For native binding. */
	UTakeRecorderSources::FOnSourceRemoved OnRecordingSourceRemovedEvent;
	/** Multicast delegate when any source has been modified. For native binding. */
	UTakeRecorderSources::FOnSourceModified OnRecordingSourceModifiedEvent;
	/** Multi-cast delegate when the pending take is cleared. For native binding. */
	FOnPendingTakeCleared OnPendingTakeClearedEvent;

public:
	/** Called before initialization occurs (ie. when the recording button is pressed and before the countdown starts) */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FTakeRecorderPreInitialize TakeRecorderPreInitialize;

	/** called when take recorder is initializing. */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FTakeRecorderInitialized TakeRecorderInitialized;
	
	/** Called when take recorder is started */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FTakeRecorderStarted TakeRecorderStarted;

	/** Called when take recorder is stopped */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FTakeRecorderStopped TakeRecorderStopped;

	/** Called when take recorder has finished */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FTakeRecorderFinished TakeRecorderFinished;

	/** Called when take recorder is cancelled */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FTakeRecorderCancelled TakeRecorderCancelled;

	/** Called when a marked frame is added to take recorder */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FTakeRecorderMarkedFrameAdded TakeRecorderMarkedFrameAdded;

	/** Called when a take recorder slate changes. */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FTakeRecorderSlateChanged TakeRecorderSlateChanged;

	/** Called when a take recorder take number changes. */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FTakeRecorderTakeNumberChanged TakeRecorderTakeNumberChanged;
	
	/** Multicast delegate when any source has been added. */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FOnTakeRecorderSourceAddedDynamic TakeRecorderSourceAdded;

	/** Multicast delegate when any source has been removed. */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FOnTakeRecorderSourceRemovedDynamic TakeRecorderSourceRemoved;
	
	/** Multicast delegate when any source has been modified. */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FOnTakeRecorderSourceModifiedDynamic TakeRecorderSourceModified;

	/** Multicast delegate when the pending take is cleared. */
	UPROPERTY(BlueprintAssignable, Category=TakeRecorder)
	FPendingTakeCleared PendingTakeCleared;
};

#undef UE_API
