// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISerializedRecorder.h"
#include "LevelSequence.h"

#define UE_API TAKERECORDER_API

class UWorld;
class AActor;
struct FActorFileHeader;
struct FActorProperty;
//Implementation of ISerializedRecorder that's defined in the SerializedRecorderInterface Module

class FSerializedRecorder:  public ISerializedRecorder
{
public:
	FSerializedRecorder() : bLoadSequenceFile(false) {}
	~FSerializedRecorder() = default;

	UE_API virtual bool LoadRecordedSequencerFile(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FString& InFileName, TFunction<void()> InCompletionCallback) override;

	//this is only valid during LoadSequenceFile or LoadActorFile
	UE_API AActor GetActorFromGuid(const FGuid& InGuid);

private:
	UE_API bool LoadSequenceFile(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FString& InFileName, TFunction<void()> InCompletionCallback);
	UE_API bool LoadActorFile(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FString& InFileName, TFunction<void()> InCompletionCallback);
	UE_API bool LoadPropertyFile(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FString& InFileName, TFunction<void()> InCompletionCallback);
	UE_API bool LoadSubSequenceFile(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FString& InFileName, TFunction<void()> InCompletionCallback);

	UE_API AActor* SetActorPossesableOrSpawnable(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FActorFileHeader& ActorHeader);
	UE_API void SetComponentPossessable(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, AActor* Actor, const FActorFileHeader& ActorHeader, const FActorProperty& ActorProperty);


private:
	TMap<FGuid, AActor*> ActorGuidToActorMap;
	bool bLoadSequenceFile;
	
};

#undef UE_API
