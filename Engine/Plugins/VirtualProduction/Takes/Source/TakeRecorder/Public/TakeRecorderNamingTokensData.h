// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/LocKeyFuncs.h"
#include "NamingTokenData.h"

#include "TakeRecorderNamingTokensData.generated.h"

class UNamingTokens;

USTRUCT()
struct FTakeRecorderNamingTokensFieldMapping
{
	GENERATED_BODY()

	/** Name of the property field. */
	UPROPERTY()
	FString FieldName;
	
	/** All undefined keys with this field. */
	UPROPERTY()
	TArray<FString> UndefinedKeys;
};

/**
 * Data container object for Take Recorder Naming Tokens. Kept as UObject to assist with transactions.
 */
UCLASS()
class UTakeRecorderNamingTokensData : public UObject
{
	GENERATED_BODY()
public:
	/** Guid to our managed tokens. */
	FGuid NamingTokensExternalGuid;
	
	/**
	 * Custom tokens entered in by the user, mapped to a user defined value.
	 * We use FNamingTokenData rather than an FString for the key so we can support case sensitivity in our map.
	 */
	UPROPERTY()
	TMap<FNamingTokenData, FText> UserDefinedTokens;

	/**
	 * User tokens that are currently visible. Kept as a separate property from UsedDefinedTokens we can persist user values
	 * between selected sources that have different token entries in their fields.
	 */
	TSet<FString, FLocKeySetFuncs> VisibleUserTokens;
	
	/** Ordered array of FieldName to undefined token keys. These are present if a token evaluation failed to identify them. */
	UPROPERTY(NonTransactional)
	TArray<FTakeRecorderNamingTokensFieldMapping> FieldToUndefinedKeys;

	/** Evaluated text to serve as an example. */
	UPROPERTY()
	FText EvaluatedTextValue;
	
	/** Pointer to our naming tokens object. */
	TWeakObjectPtr<UNamingTokens> TakeRecorderNamingTokens;

	/** Find or add an array given a field name. The array will be a list of token keys for this specific field. */
	TAKERECORDER_API TArray<FString>& FindOrAddTokenKeysForField(const FString& InFieldName);

	/** Checks if a token key is in our undefined list. */
	TAKERECORDER_API bool IsTokenKeyUndefined(const FString& InTokenKey) const;
};