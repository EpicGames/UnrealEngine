// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidget.h"
#include "TakeRecorderOverlayWidget.generated.h"

#define UE_API TAKERECORDER_API

class UTakeRecorder;

UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UTakeRecorderOverlayWidget : public UUserWidget
{
public:

	GENERATED_BODY()

	UE_API UTakeRecorderOverlayWidget(const FObjectInitializer& ObjectInitializer);

	/**
	 * Set the recorder that this overlay is reflecting
	 */
	void SetRecorder(UTakeRecorder* InRecorder)
	{
		Recorder = InRecorder;
	}

protected:

	/** The recorder that this overlay is reflecting */
	UPROPERTY(BlueprintReadOnly, Category="Take Recorder")
	TObjectPtr<UTakeRecorder> Recorder;
};

#undef UE_API
