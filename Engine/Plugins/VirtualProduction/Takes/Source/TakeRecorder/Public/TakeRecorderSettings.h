// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Recorder/TakeRecorderHitchProtectionParameters.h"
#include "Recorder/TakeRecorderParameters.h"
#include "TakeRecorderSettings.generated.h"

class UTakePreset;

/**
 * Universal take recorder settings that apply to a whole take
 */
UCLASS(config=EditorPerProjectUserSettings, MinimalAPI)
class UTakeRecorderUserSettings : public UObject
{
public:
	GENERATED_BODY()

	TAKERECORDER_API UTakeRecorderUserSettings();

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** Return the PresetSaveDir path relative to the project. */
	TAKERECORDER_API FString GetResolvedPresetSaveDir() const;

	/** Sets the default location for the present saved directory. */
	TAKERECORDER_API void SetPresetSaveDir(const FString& InPath);

	/** User settings that should be passed to a recorder instance */
	UPROPERTY(config, EditAnywhere, BlueprintReadOnly, Category="User Settings", meta=(ShowOnlyInnerProperties))
	FTakeRecorderUserParameters Settings;

	/** The default location in which to save take presets */
	UPROPERTY(config, EditAnywhere, Category="User Settings", DisplayName="Preset Save Location", meta = (ContentDir, NamingTokens))
	FDirectoryPath PresetSaveDir;

	/** Soft reference to the preset last opened on the take recording UI */
	UPROPERTY(config)
	TSoftObjectPtr<UTakePreset> LastOpenedPreset;

	/** Whether the sequence editor is open for the take recorder */
	UPROPERTY(config)
	bool bIsSequenceOpen;

	/** Whether the sequence editor is open for the take recorder */
	UPROPERTY(config)
	bool bShowUserSettingsOnUI;
};

/**
 * Universal take recorder settings that apply to a whole take
 */
UCLASS(config=EditorSettings, MinimalAPI)
class UTakeRecorderProjectSettings : public UObject
{
public:
	GENERATED_BODY()

	TAKERECORDER_API UTakeRecorderProjectSettings();

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** General take recorder settings */
	UPROPERTY(config, EditAnywhere, BlueprintReadOnly, Category="Take Recorder", meta=(ShowOnlyInnerProperties))
	FTakeRecorderProjectParameters Settings;

	/**
	 * Controls the behaviour for hitch protection.
	 *
	 * On a high level, it achieves even key spacing in recorded data even when the engine freezes on a frame ("hitches").
	 * When a frames takes longer, the simulation continues to advance at the same fixed time step and assigned to the timecode the frame would have
	 * had, if they hitch hadn't occured.
	 * 
	 * Example: frame n at 17:56:12:02 freezes for 1s. In physical time, once the hitches finishes, timecode will have advanced by 1s to 17:56:13:02.
	 * However, hitch protection annotates frame n+1 with timecode 17:56:12:03. The subsequent frames continue to be ticked at the same rate, e.g. 1/24s;
	 * as long as it takes less CPU time to run the simulation (i.e. CPU time to simulate is less than 1/24s), the simulation time eventually catches
	 * up with physical time.
	 */
	UPROPERTY(config, EditAnywhere, Category="Hitch Protection")
	FTakeRecorderHitchProtectionParameters HitchProtectionSettings;

	/** Array of externally supplied CDOs that should be displayed on the take recorder project settings */
	TArray<TWeakObjectPtr<UObject>> AdditionalSettings;
};
