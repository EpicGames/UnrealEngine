// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Trace/Trace.h"

UE_TRACE_CHANNEL_EXTERN(TakeRecorderChannel, TAKERECORDER_API);
#define SCOPED_TAKERECORDER_TRACE(TraceName) TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(TraceName, TakeRecorderChannel)

