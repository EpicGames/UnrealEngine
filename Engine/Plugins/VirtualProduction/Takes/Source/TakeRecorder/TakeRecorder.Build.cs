// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TakeRecorder : ModuleRules
{
	public TakeRecorder(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AppFramework",
				"AssetRegistry",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"LevelSequence",
				"MovieScene",
				"TakesCore",
				"TakeMovieScene",
				"TimeManagement",
				"Slate",
				"SlateCore",
				"Analytics",
				"NamingTokens"
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"UMG",
                "TakeTrackRecorders",
                "SerializedRecorderInterface",
				"TraceLog"
            }
        );

		PublicIncludePathModuleNames.AddRange(
			new string[]
			{
				"TakeRecorderNamingTokens"
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AssetDefinition",
					"ContentBrowser",
					"ClassViewer",
					"EditorStyle",
					"EditorWidgets",
					"LevelEditor",
					"MovieSceneTools",
					"PropertyEditor",
					"EditorFramework",
					"UnrealEd",
					"WorkspaceMenuStructure",
					"SequencerCore",
					"NamingTokensUI",
					"ToolMenus",
					"ToolWidgets",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"EditorSubsystem",
					"Sequencer"
				}
			);
		}
	}
}
