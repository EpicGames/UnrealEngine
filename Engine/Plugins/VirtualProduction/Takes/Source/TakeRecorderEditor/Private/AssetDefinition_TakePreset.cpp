// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_TakePreset.h"
#include "TakePreset.h"
#include "TakePresetToolkit.h"
#include "ITakeRecorderModule.h"
#include "Widgets/STakeRecorderTabContent.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Docking/TabManager.h"
#include "Toolkits/ToolkitManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_TakePreset)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FAssetOpenSupport UAssetDefinition_TakePreset::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(OpenSupportArgs.OpenMethod,OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit, EToolkitMode::WorldCentric); 
}

EAssetCommandResult UAssetDefinition_TakePreset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	UWorld* WorldContext = nullptr;
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::Editor)
		{
			WorldContext = Context.World();
			break;
		}
	}

	if (!ensure(WorldContext))
	{
		return EAssetCommandResult::Handled;
	}

	for (UTakePreset* TakePreset : OpenArgs.LoadObjects<UTakePreset>())
	{
		TSharedPtr<FTakePresetToolkit> Toolkit = MakeShared<FTakePresetToolkit>();
		Toolkit->Initialize(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, TakePreset);

		TSharedPtr<SDockTab> DockTab = OpenArgs.ToolkitHost->GetTabManager()->TryInvokeTab(ITakeRecorderModule::TakeRecorderTabName);
		if (DockTab.IsValid())
		{
			TSharedRef<STakeRecorderTabContent> TabContent = StaticCastSharedRef<STakeRecorderTabContent>(DockTab->GetContent());
			TabContent->SetupForEditing(Toolkit);
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
