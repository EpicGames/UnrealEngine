// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakePreset.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_TakePreset.generated.h"

UCLASS()
class UAssetDefinition_TakePreset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TakePreset", "Take Recorder Preset"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(226, 155, 72)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTakePreset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Cinematics };
		return Categories;
	}
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
