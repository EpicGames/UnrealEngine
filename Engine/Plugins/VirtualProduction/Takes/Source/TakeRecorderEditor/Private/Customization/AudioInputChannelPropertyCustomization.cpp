// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioInputChannelPropertyCustomization.h"

#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "TakeRecorderAudioSettingsCustomization.h"
#include "TakeRecorderSourceProperty.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::TakeRecorder
{
void FAudioInputChannelPropertyCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils
	)
{
	InputDeviceChannelHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAudioInputDeviceChannelProperty, AudioInputDeviceChannel));

	if (InputDeviceChannelHandle.IsValid())
	{
		IDetailPropertyRow& InputDeviceChannelPropertyRow = ChildBuilder.AddProperty(PropertyHandle);
		InputDeviceChannelPropertyRow.CustomWidget()
		.NameContent()
		[
			InputDeviceChannelHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			MakeInputChannelSelectorWidget()
		];
	}

	// Register delegate so UI can update when the audio input device changes
	if (UTakeRecorderAudioInputSettings* AudioInputSettings = TakeRecorderAudioSettingsUtils::GetTakeRecorderAudioInputSettings())
	{
		AudioInputSettings->GetOnAudioInputDeviceChanged().Add(FSimpleDelegate::CreateSP(this, &FAudioInputChannelPropertyCustomization::RebuildInputChannelArray));
	}
}

void FAudioInputChannelPropertyCustomization::BuildInputChannelArray()
{
	if (UTakeRecorderAudioInputSettings* AudioInputSettings = TakeRecorderAudioSettingsUtils::GetTakeRecorderAudioInputSettings())
	{
		int32 DeviceChannelCount = AudioInputSettings->GetDeviceChannelCount();

		if (DeviceChannelCount > 0)
		{
			for (int32 ChannelIndex = 1; ChannelIndex <= DeviceChannelCount; ++ChannelIndex)
			{
				TSharedPtr<int32> ChannelIndexPtr = MakeShared<int32>(ChannelIndex);
				InputDeviceChannelArray.Add(ChannelIndexPtr);
			}
		}

		int32 CurrentValue;
		InputDeviceChannelHandle->GetValue(CurrentValue);
		if (CurrentValue > DeviceChannelCount)
		{
			InputDeviceChannelHandle->SetValue(0);
		}
	}
}

void FAudioInputChannelPropertyCustomization::RebuildInputChannelArray()
{
	InputDeviceChannelArray.Empty();
	BuildInputChannelArray();
	ChannelComboBox->RefreshOptions();

	UTakeRecorderAudioInputSettings* AudioInputSettings = TakeRecorderAudioSettingsUtils::GetTakeRecorderAudioInputSettings();

	if (AudioInputSettings && InputDeviceChannelHandle.IsValid())
	{
		int32 CurrentValue;
		InputDeviceChannelHandle->GetValue(CurrentValue);
		int32 DeviceChannelCount = AudioInputSettings->GetDeviceChannelCount();

		if (CurrentValue < 1 || CurrentValue > DeviceChannelCount)
		{
			InputChannelTitleBlock->SetText(FText());
		}
	}
}

TSharedRef<SWidget> FAudioInputChannelPropertyCustomization::MakeInputChannelSelectorWidget()
{
	BuildInputChannelArray();

	// Combo box component:
	SAssignNew(ChannelComboBox, SComboBox<TSharedPtr<int32>>)
		.OptionsSource(&InputDeviceChannelArray)
		.OnGenerateWidget_Lambda([](TSharedPtr<int32> InValue)
		{
			return SNew(STextBlock)
				.Text(FText::AsNumber(*InValue))
				.Font(FAppStyle::Get().GetFontStyle("SmallFont"));
		})
		.OnSelectionChanged_Lambda([this](TSharedPtr<int32> InSelection, ESelectInfo::Type InSelectInfo)
		{
			if (InSelection.IsValid())
			{
				InputDeviceChannelHandle->SetValue(*InSelection);
				InputChannelTitleBlock->SetText(FText::AsNumber(*InSelection));
			}
		})
		[
			SAssignNew(InputChannelTitleBlock, STextBlock)
			.Text_Lambda([this]()
			{
				int32 ChannelNumber;
				InputDeviceChannelHandle->GetValue(ChannelNumber);

				if (ChannelNumber > 0)
				{
					return FText::AsNumber(ChannelNumber);
				}
				return FText();
			})
			.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
		];

	return ChannelComboBox.ToSharedRef();
}
}
