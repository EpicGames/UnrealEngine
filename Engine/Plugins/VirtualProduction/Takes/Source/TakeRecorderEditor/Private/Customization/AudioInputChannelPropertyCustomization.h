// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
class SWidget;
class STextBlock;
template<typename OptionType> class SComboBox;

namespace UE::TakeRecorder
{
class FAudioInputChannelPropertyCustomization : public IPropertyTypeCustomization
{
public:

	//~ Begin IPropertyTypeCustomization Interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& ) {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);
	//~ End IPropertyTypeCustomization Interface

private:

	TSharedPtr<IPropertyHandle> InputDeviceChannelHandle;
	TArray<TSharedPtr<int32>> InputDeviceChannelArray;
	TSharedPtr<STextBlock> InputChannelTitleBlock;
	TSharedPtr<SComboBox<TSharedPtr<int32>>> ChannelComboBox;
		
	void BuildInputChannelArray();
	void RebuildInputChannelArray();

	TSharedRef<SWidget> MakeInputChannelSelectorWidget();
};
}