// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokenDirectoryPathCustomization.h"

#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "DesktopPlatformModule.h"
#include "DetailWidgetRow.h"
#include "EditorDirectories.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "IContentBrowserSingleton.h"
#include "IDesktopPlatform.h"
#include "Misc/MessageDialog.h"
#include "SNamingTokensEditableTextBox.h"
#include "TakeRecorderNamingTokenCustomizationUtilities.h"

#define LOCTEXT_NAMESPACE "FNamingTokenDirectoryPathStructCustomization"

TSharedRef<IPropertyTypeCustomization> FNamingTokenDirectoryPathStructCustomization::MakeInstance()
{
	return MakeShared<FNamingTokenDirectoryPathStructCustomization>();
}

void FNamingTokenDirectoryPathStructCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	const TSharedPtr<IPropertyHandle> PathProperty = StructPropertyHandle->GetChildHandle("Path");

	const bool bRelativeToGameContentDir = StructPropertyHandle->HasMetaData( TEXT("RelativeToGameContentDir") );
	const bool bUseRelativePath = StructPropertyHandle->HasMetaData( TEXT("RelativePath") );
	const bool bContentDir = StructPropertyHandle->HasMetaData( TEXT("ContentDir") ) || StructPropertyHandle->HasMetaData(TEXT("LongPackageName"));
	const bool bForceShowPluginContent = StructPropertyHandle->HasMetaData(TEXT("ForceShowPluginContent"));
	const bool bForceShowEngineContent = StructPropertyHandle->HasMetaData(TEXT("ForceShowEngineContent"));

	AbsoluteGameContentDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());

	if (PathProperty.IsValid())
	{
		TSharedPtr<SWidget> PickerWidget = nullptr;
		
		NamingTokensEditableTextBox = UE::TakeRecorder::NamingTokens::CreateNamingTokensWidgetForProperty(PathProperty);
		PathProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this]
		{
			if (NamingTokensEditableTextBox.IsValid())
			{
				NamingTokensEditableTextBox->EvaluateNamingTokens();
			}
		}));
		
		if(bContentDir)
		{
			PickerWidget = SAssignNew(PickerButton, SButton)
			.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
			.ToolTipText( LOCTEXT("FolderComboToolTipText", "Choose a content directory") )
			.OnClicked( this, &FNamingTokenDirectoryPathStructCustomization::OnPickContent, PathProperty.ToSharedRef(), bForceShowPluginContent, bForceShowEngineContent )
			.ContentPadding(2.0f)
			.ForegroundColor( FSlateColor::UseForeground() )
			.IsFocusable(false)
			.IsEnabled(this, &FNamingTokenDirectoryPathStructCustomization::IsBrowseEnabled, StructPropertyHandle)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			];

		}
		else
		{
			PickerWidget = SAssignNew(BrowseButton, SButton)
			.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
			.ToolTipText( LOCTEXT( "FolderButtonToolTipText", "Choose a directory from this computer") )
			.OnClicked( this, &FNamingTokenDirectoryPathStructCustomization::OnPickDirectory, PathProperty.ToSharedRef(), bRelativeToGameContentDir, bUseRelativePath )
			.ContentPadding( 2.0f )
			.ForegroundColor( FSlateColor::UseForeground() )
			.IsFocusable( false )
			.IsEnabled( this, &FNamingTokenDirectoryPathStructCustomization::IsBrowseEnabled, StructPropertyHandle )
			[
				SNew( SImage )
				.Image( FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis") )
				.ColorAndOpacity( FSlateColor::UseForeground() )
			];
		}
		
		HeaderRow.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(600.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				NamingTokensEditableTextBox.ToSharedRef()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				PickerWidget.ToSharedRef()
			]
		]
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
	}
}

void FNamingTokenDirectoryPathStructCustomization::CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
}

FReply FNamingTokenDirectoryPathStructCustomization::OnPickContent(TSharedRef<IPropertyHandle> PropertyHandle, const bool bForceShowPluginContent, const bool bForceShowEngineContent) 
{
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FPathPickerConfig PathPickerConfig;
	PropertyHandle->GetValue(PathPickerConfig.DefaultPath);
	PathPickerConfig.bAllowContextMenu = false;
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &FNamingTokenDirectoryPathStructCustomization::OnPathPicked, PropertyHandle);
	PathPickerConfig.bForceShowEngineContent = bForceShowEngineContent;
	PathPickerConfig.bForceShowPluginContent = bForceShowPluginContent;
	
	FMenuBuilder MenuBuilder(true, nullptr);
	MenuBuilder.AddWidget(SNew(SBox)
		.WidthOverride(300.0f)
		.HeightOverride(300.0f)
		[
			ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
		], FText());
	
	PickerMenu = FSlateApplication::Get().PushMenu(PickerButton.ToSharedRef(),
		FWidgetPath(),
		MenuBuilder.MakeWidget(),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
	
	return FReply::Handled();
}

FReply FNamingTokenDirectoryPathStructCustomization::OnPickDirectory(TSharedRef<IPropertyHandle> PropertyHandle, const bool bRelativeToGameContentDir, const bool bUseRelativePath) const
{
	FString Directory;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{

		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(BrowseButton.ToSharedRef());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		FString StartDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT);
		if (bRelativeToGameContentDir && !IsValidPath(StartDirectory, bRelativeToGameContentDir))
		{
			StartDirectory = AbsoluteGameContentDir;
		}

		// Loop until; a) the user cancels (OpenDirectoryDialog returns false), or, b) the chosen path is valid (IsValidPath returns true)
		for (;;)
		{
			if (DesktopPlatform->OpenDirectoryDialog(ParentWindowHandle, LOCTEXT("FolderDialogTitle", "Choose a directory").ToString(), StartDirectory, Directory))
			{
				FText FailureReason;
				if (IsValidPath(Directory, bRelativeToGameContentDir, &FailureReason))
				{
					FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_IMPORT, Directory);

					if (bRelativeToGameContentDir)
					{
						Directory.RightChopInline(AbsoluteGameContentDir.Len(), EAllowShrinking::No);
					}
					else if (bUseRelativePath)
					{
						Directory = IFileManager::Get().ConvertToRelativePath(*Directory);
					}

					PropertyHandle->SetValue(Directory);
				}
				else
				{
					StartDirectory = Directory;
					FMessageDialog::Open(EAppMsgType::Ok, FailureReason);
					continue;
				}
			}
			break;
		}
	}

	return FReply::Handled();
}

bool FNamingTokenDirectoryPathStructCustomization::IsValidPath(const FString& AbsolutePath, const bool bRelativeToGameContentDir, FText* const OutReason) const
{
	if(bRelativeToGameContentDir)
	{
		if(!AbsolutePath.StartsWith(AbsoluteGameContentDir))
		{
			if(OutReason)
			{
				*OutReason = FText::Format(LOCTEXT("Error_InvalidRootPath", "The chosen directory must be within {0}"), FText::FromString(AbsoluteGameContentDir));
			}
			return false;
		}
	}

	return true;
}

void FNamingTokenDirectoryPathStructCustomization::OnPathPicked(const FString& Path, TSharedRef<IPropertyHandle> PropertyHandle)
{
	if (PickerMenu.IsValid())
	{
		PickerMenu->Dismiss();
		PickerMenu.Reset();
	}

	PropertyHandle->SetValue(Path);
}

bool FNamingTokenDirectoryPathStructCustomization::IsBrowseEnabled(TSharedRef<IPropertyHandle> PropertyHandle) const
{
	return PropertyHandle->IsEditable();
}

#undef LOCTEXT_NAMESPACE
