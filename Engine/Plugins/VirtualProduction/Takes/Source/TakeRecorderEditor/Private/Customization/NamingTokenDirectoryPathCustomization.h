// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

#include "Containers/UnrealString.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
class IDetailPropertyRow;

class IMenu;
class SNamingTokensEditableTextBox;
class SButton;

/**
 * A Naming Tokens version of the FDirectoryPathStructCustomization, mostly identical apart from using the
 * NamingTokens box instead of a normal text box.
 *
 * We may want to move this under the NamingTokens plugin for use with other plugins.
 */
class FNamingTokenDirectoryPathStructCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;              

private:

	/** Delegate used to display a directory picker */
	FReply OnPickContent(TSharedRef<IPropertyHandle> PropertyHandle, const bool bForceShowPluginContent, const bool bForceShowEngineContent);

	/** Delegate used to display a directory picker */
	FReply OnPickDirectory(TSharedRef<IPropertyHandle> PropertyHandle, const bool bRelativeToGameContentDir, const bool bUseRelativePaths) const;

	/** Check whether that the chosen path is valid */
	bool IsValidPath(const FString& AbsolutePath, const bool bRelativeToGameContentDir, FText* const OutReason = nullptr) const;

	/** Called when a path is picked from the path picker */
	void OnPathPicked(const FString& Path, TSharedRef<IPropertyHandle> PropertyHandle);

	/** Delegate to determine whether the browse button should be enabled */
	bool IsBrowseEnabled(TSharedRef<IPropertyHandle> PropertyHandle) const;

private:
	/** The browse button widget */
	TSharedPtr<SButton> BrowseButton;

	/** The pick button widget */
	TSharedPtr<SButton> PickerButton;

	/** The pick button popup menu */
	TSharedPtr<IMenu> PickerMenu;

	/** The naming tokens text box */
	TSharedPtr<SNamingTokensEditableTextBox> NamingTokensEditableTextBox;
	
	/** Absolute path to the game content directory */
	FString AbsoluteGameContentDir;
};
