// Copyright Epic Games, Inc. All Rights Reserved.

#include "RecordedPropertyCustomization.h"

#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "TakeRecorderSourceProperty.h"

namespace UE::TakeRecorder
{
const FString FRecordedPropertyCustomization::PropertyPathDelimiter = FString(TEXT("."));
	
void FRecordedPropertyCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils
	)
{
	if (PropertyHandle->IsValidHandle())
	{
		TSharedPtr<IPropertyHandle> PropertyNameHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FActorRecordedProperty, PropertyName)); 
		TSharedPtr<IPropertyHandle> bEnabledHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FActorRecordedProperty, bEnabled)); 
		TSharedPtr<IPropertyHandle> RecorderNameHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FActorRecordedProperty, RecorderName));

		FString PropertyNameValue;
		PropertyNameHandle->GetValueAsDisplayString(PropertyNameValue);

		FString ParentGroups; 
		FString PropertyName;
		FText DisplayString = FText::FromString(PropertyNameValue);
		if (PropertyNameValue.Split(PropertyPathDelimiter, &ParentGroups, &PropertyName, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
		{
			DisplayString = FText::FromString(PropertyName);
		}

		HeaderRow 
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				bEnabledHandle->CreatePropertyValueWidget(false)
			]

			+SHorizontalBox::Slot()
			.Padding(8, 0, 0, 0)
			[
				PropertyNameHandle->CreatePropertyNameWidget(DisplayString)
			]

		];
	}
}
}

