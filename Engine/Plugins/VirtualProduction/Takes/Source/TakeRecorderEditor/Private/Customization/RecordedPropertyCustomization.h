// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Containers/UnrealString.h"

namespace UE::TakeRecorder
{
class FRecordedPropertyCustomization : public IPropertyTypeCustomization
{
	static const FString PropertyPathDelimiter;

	//~ Begin IPropertyTypeCustomization Interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils );
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils )
	{
		// Intentionally Left Blank, Child Customization was handled in the Header Row
	}
	//~ End IPropertyTypeCustomization Interface
};
}