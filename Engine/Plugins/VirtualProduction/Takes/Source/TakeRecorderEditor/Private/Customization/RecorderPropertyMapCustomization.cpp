// Copyright Epic Games, Inc. All Rights Reserved.

#include "RecorderPropertyMapCustomization.h"

#include "ClassIconFinder.h"
#include "DetailWidgetRow.h"
#include "GameFramework/Actor.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "TakeRecorderSourceProperty.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SLevelSequenceTakeEditor"

namespace UE::TakeRecorder
{
const FString FRecorderPropertyMapCustomization::PropertyPathDelimiter = FString(TEXT("."));
	
void FRecorderPropertyMapCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils
	)
{
	TSharedPtr<IPropertyHandle> RecordedObjectHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UActorRecorderPropertyMap, RecordedObject)); 

	UObject* Object = nullptr;
	FText ActorOrComponentName = LOCTEXT("MissingActorOrComponentName", "MissingActorOrComponentName");

	const FSlateBrush* Icon = nullptr;

	if ( RecordedObjectHandle->IsValidHandle() && RecordedObjectHandle->GetValue(Object) == FPropertyAccess::Success && Object != nullptr )
	{
		if (AActor* Actor = Cast<AActor>(Object))
		{
			ActorOrComponentName = FText::AsCultureInvariant(Actor->GetActorLabel());
			Icon = FClassIconFinder::FindIconForActor(Actor);
		}
		else
		{
			ActorOrComponentName = FText::AsCultureInvariant(Object->GetName());
			Icon = FSlateIconFinder::FindIconBrushForClass(Object->GetClass());
		}
	}

	HeaderRow
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew( SCheckBox )
			.OnCheckStateChanged( this, &FRecorderPropertyMapCustomization::OnCheckStateChanged, PropertyHandle )	
			.IsChecked( this, &FRecorderPropertyMapCustomization::OnGetCheckState, PropertyHandle )	
			.Padding(0.0)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(8, 0, 0, 0)
		[
			SNew(SImage)
			.Image(Icon)
		]

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(2.0, 0.0)
		[
			SNew(STextBlock)
			.Text(ActorOrComponentName)
			.Font( FAppStyle::GetFontStyle( "PropertyWindow.BoldFont" ))
		]

		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.Padding(2.0, 0.0)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("TakeRecorderRecordedPropertiesTitle", "Recorded Properties"))
			.Font( FAppStyle::GetFontStyle( "PropertyWindow.NormalFont" ))
		]
	];
}

IDetailGroup& FRecorderPropertyMapCustomization::GetOrCreateDetailGroup(
	IDetailChildrenBuilder& ChildBuilder, TMap<FString, IDetailGroup*>& GroupMap, TSharedPtr<IPropertyHandleArray> PropertiesArray, FString& GroupName
	)
{
	if (GroupMap.Contains(GroupName))
	{
		return *GroupMap[GroupName];
	}

	FString ParentGroups; 
	FString PropertyName;
	FText DisplayName;

	if (GroupName.Split(PropertyPathDelimiter, &ParentGroups, &PropertyName, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
	{
		IDetailGroup& ParentGroup = GetOrCreateDetailGroup(ChildBuilder, GroupMap, PropertiesArray, ParentGroups);
		DisplayName = FText::FromString(PropertyName);
		GroupMap.Add(GroupName, &ParentGroup.AddGroup(FName(*PropertyName), DisplayName));
	}
	else 
	{
		DisplayName = FText::FromString(GroupName);
		GroupMap.Add(GroupName, &ChildBuilder.AddGroup(FName(*GroupName), DisplayName));
	}

	GroupMap[GroupName]->HeaderRow()
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew( SCheckBox )
			.OnCheckStateChanged( this, &FRecorderPropertyMapCustomization::OnGroupCheckStateChanged, PropertiesArray, GroupName )	
			.IsChecked( this, &FRecorderPropertyMapCustomization::OnGroupGetCheckState, PropertiesArray, GroupName )	
		]

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(6.0, 0.0)
		[
			SNew(STextBlock)
			.Text(DisplayName)
			.Font( FAppStyle::GetFontStyle( "PropertyWindow.NormalFont" ))
		]
	];

	return *GroupMap[GroupName];
}

void FRecorderPropertyMapCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils
	)
{
	TMap<FString, IDetailGroup*> DetailGroupMap;

	TSharedPtr<IPropertyHandleArray> RecordedPropertiesArrayHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UActorRecorderPropertyMap, Properties))->AsArray();

	uint32 NumProperties;
	RecordedPropertiesArrayHandle->GetNumElements(NumProperties);


	for(uint32 i = 0; i < NumProperties; ++i)
	{
		TSharedRef<IPropertyHandle> RecordedPropertyTemp = RecordedPropertiesArrayHandle->GetElement(i);
		if (RecordedPropertyTemp->IsValidHandle())
		{

			TSharedPtr<IPropertyHandle> PropertyNameHandle = RecordedPropertyTemp->GetChildHandle(GET_MEMBER_NAME_CHECKED(FActorRecordedProperty, PropertyName)); 
			if (PropertyNameHandle->IsValidHandle())
			{
				FString PropertyNameValue;
				PropertyNameHandle->GetValueAsDisplayString(PropertyNameValue);

				FString ParentGroups; 
				FString PropertyName;
				if (PropertyNameValue.Split(PropertyPathDelimiter, &ParentGroups, &PropertyName, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
				{
					IDetailGroup& ParentGroup = GetOrCreateDetailGroup(ChildBuilder, DetailGroupMap, RecordedPropertiesArrayHandle, ParentGroups);
					ParentGroup.AddPropertyRow(RecordedPropertyTemp);
				}
				else 
				{
					ChildBuilder.AddProperty(RecordedPropertyTemp);
				}
			}
		}
	}

	TSharedPtr<IPropertyHandleArray> RecordedComponentsArrayHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UActorRecorderPropertyMap, Children))->AsArray();

	uint32 NumCompProperties;
	RecordedComponentsArrayHandle->GetNumElements(NumCompProperties);

	for(uint32 i = 0; i < NumCompProperties; ++i)
	{
		TSharedRef<IPropertyHandle> RecordedCompTemp = RecordedComponentsArrayHandle->GetElement(i);
		if (RecordedCompTemp->IsValidHandle())
		{
			ChildBuilder.AddProperty(RecordedCompTemp);
		}
	}
}

void FRecorderPropertyMapCustomization::OnGroupCheckStateChanged(
	ECheckBoxState InNewState, TSharedPtr<IPropertyHandleArray> RecordedPropertiesArrayHandle, FString GroupName
	) const
{
	uint32 NumProperties;
	RecordedPropertiesArrayHandle->GetNumElements(NumProperties);

	for(uint32 i = 0; i < NumProperties; ++i)
	{
		TSharedRef<IPropertyHandle> RecordedPropertyTemp = RecordedPropertiesArrayHandle->GetElement(i);
		if (RecordedPropertyTemp->IsValidHandle())
		{
			TSharedPtr<IPropertyHandle> PropertyNameHandle = RecordedPropertyTemp->GetChildHandle(GET_MEMBER_NAME_CHECKED(FActorRecordedProperty, PropertyName)); 
			if (PropertyNameHandle->IsValidHandle())
			{
				FString PropertyNameValue;
				PropertyNameHandle->GetValueAsDisplayString(PropertyNameValue);

				if (PropertyNameValue.StartsWith(GroupName))
				{
					TSharedPtr<IPropertyHandle> EnabledRecordedPropertyTemp = RecordedPropertyTemp->GetChildHandle(GET_MEMBER_NAME_CHECKED(FActorRecordedProperty, bEnabled));
					if (EnabledRecordedPropertyTemp->IsValidHandle())
					{
						EnabledRecordedPropertyTemp->SetValue( InNewState == ECheckBoxState::Checked ? true : false );
					}
				}
			}
		}
	}
}

ECheckBoxState FRecorderPropertyMapCustomization::OnGroupGetCheckState(
	TSharedPtr<IPropertyHandleArray> RecordedPropertiesArrayHandle, FString GroupName
	) const
{
	bool SetFirst = false;
	bool FinalCheckedValue = false;

	uint32 NumProperties;
	RecordedPropertiesArrayHandle->GetNumElements(NumProperties);

	for(uint32 i = 0; i < NumProperties; ++i)
	{
		TSharedRef<IPropertyHandle> RecordedPropertyTemp = RecordedPropertiesArrayHandle->GetElement(i);
		if (RecordedPropertyTemp->IsValidHandle())
		{
			TSharedPtr<IPropertyHandle> PropertyNameHandle = RecordedPropertyTemp->GetChildHandle(GET_MEMBER_NAME_CHECKED(FActorRecordedProperty, PropertyName)); 
			if (PropertyNameHandle->IsValidHandle())
			{
				FString PropertyNameValue;
				PropertyNameHandle->GetValueAsDisplayString(PropertyNameValue);

				if (PropertyNameValue.StartsWith(GroupName))
				{
					TSharedPtr<IPropertyHandle> EnabledRecordedPropertyTemp = RecordedPropertyTemp->GetChildHandle(GET_MEMBER_NAME_CHECKED(FActorRecordedProperty, bEnabled));
					if (EnabledRecordedPropertyTemp->IsValidHandle())
					{
						bool EnabledValue;
						if ( EnabledRecordedPropertyTemp->GetValue(EnabledValue) == FPropertyAccess::Success )
						{
							if (!SetFirst)
							{
								FinalCheckedValue = EnabledValue;
								SetFirst = true;
							}
							else if ( EnabledValue != FinalCheckedValue )
							{
								return ECheckBoxState::Undetermined;
							}
						}
					}
				}
			}
		}
	}

	return FinalCheckedValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FRecorderPropertyMapCustomization::OnCheckStateChanged(ECheckBoxState InNewState, TSharedRef<IPropertyHandle> PropertyHandle)
{
	TSharedPtr<IPropertyHandleArray> RecordedPropertiesArrayHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UActorRecorderPropertyMap, Properties))->AsArray();

	uint32 NumProperties;
	RecordedPropertiesArrayHandle->GetNumElements(NumProperties);

	for(uint32 i = 0; i < NumProperties; ++i)
	{
		TSharedRef<IPropertyHandle> RecordedPropertyTemp = RecordedPropertiesArrayHandle->GetElement(i);
		if (RecordedPropertyTemp->IsValidHandle())
		{
			TSharedPtr<IPropertyHandle> EnabledRecordedPropertyTemp = RecordedPropertyTemp->GetChildHandle(GET_MEMBER_NAME_CHECKED(FActorRecordedProperty, bEnabled));
			if (EnabledRecordedPropertyTemp->IsValidHandle())
			{
				EnabledRecordedPropertyTemp->SetValue( InNewState == ECheckBoxState::Checked );
			}
		}
	}

	TSharedPtr<IPropertyHandleArray> RecordedComponentsArrayHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UActorRecorderPropertyMap, Children))->AsArray();

	uint32 NumCompProperties;
	RecordedComponentsArrayHandle->GetNumElements(NumCompProperties);

	for(uint32 i = 0; i < NumCompProperties; ++i)
	{
		TSharedPtr<IPropertyHandle> RecordedCompTemp = RecordedComponentsArrayHandle->GetElement(i);
		if (RecordedCompTemp->IsValidHandle())
		{
			OnCheckStateChanged( InNewState, RecordedCompTemp.ToSharedRef());
		}
	}
}

ECheckBoxState FRecorderPropertyMapCustomization::OnGetCheckState(TSharedRef<IPropertyHandle> PropertyHandle) const
{
	bool SetFirst = false;
		bool FinalCheckedValue = false;

		TSharedPtr<IPropertyHandleArray> RecordedPropertiesArrayHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UActorRecorderPropertyMap, Properties))->AsArray();

		uint32 NumProperties;
		RecordedPropertiesArrayHandle->GetNumElements(NumProperties);

		for(uint32 i = 0; i < NumProperties; ++i)
		{
			TSharedRef<IPropertyHandle> RecordedPropertyTemp = RecordedPropertiesArrayHandle->GetElement(i);
			if (RecordedPropertyTemp->IsValidHandle())
			{

				TSharedPtr<IPropertyHandle> EnabledRecordedPropertyTemp = RecordedPropertyTemp->GetChildHandle(GET_MEMBER_NAME_CHECKED(FActorRecordedProperty, bEnabled));
				if (EnabledRecordedPropertyTemp->IsValidHandle())
				{
					bool EnabledValue;
					if ( EnabledRecordedPropertyTemp->GetValue(EnabledValue) == FPropertyAccess::Success )
					{
						if (!SetFirst)
						{
							FinalCheckedValue = EnabledValue;
							SetFirst = true;
						}
						else if ( EnabledValue != FinalCheckedValue )
						{
							return ECheckBoxState::Undetermined;
						}
					}
				}
			}
		}

		TSharedPtr<IPropertyHandleArray> RecordedComponentsArrayHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UActorRecorderPropertyMap, Children))->AsArray();

		uint32 NumCompProperties;
		RecordedComponentsArrayHandle->GetNumElements(NumCompProperties);

		for(uint32 i = 0; i < NumCompProperties; ++i)
		{
			TSharedPtr<IPropertyHandle> RecordedCompTemp = RecordedComponentsArrayHandle->GetElement(i);
			if (RecordedCompTemp->IsValidHandle())
			{
				ECheckBoxState CompEnabledState = OnGetCheckState( RecordedCompTemp.ToSharedRef() );
				if (CompEnabledState == ECheckBoxState::Undetermined)
				{
					return ECheckBoxState::Undetermined;
				}
				else 
				{
					bool IsCompChecked = CompEnabledState == ECheckBoxState::Checked ? true : false;

					if ( !SetFirst )
					{
						FinalCheckedValue = IsCompChecked;	
						SetFirst = true;
					}
					else if ( IsCompChecked != FinalCheckedValue )
					{
						return ECheckBoxState::Undetermined;
					}
				}
			}
		}

		return FinalCheckedValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}
}

#undef LOCTEXT_NAMESPACE