// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointerFwd.h"

class FString;
class IDetailGroup;
class IPropertyHandle;
class IPropertyHandleArray;
enum class ECheckBoxState : uint8;

namespace UE::TakeRecorder
{
class FRecorderPropertyMapCustomization : public IPropertyTypeCustomization
{
public:
	
	static const FString PropertyPathDelimiter; 

	//~ Begin IPropertyTypeCustomization Interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils);
	virtual IDetailGroup& GetOrCreateDetailGroup(IDetailChildrenBuilder& ChildBuilder, TMap<FString, IDetailGroup*>& GroupMap, TSharedPtr<IPropertyHandleArray> PropertiesArray, FString& GroupName);
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);
	//~ End IPropertyTypeCustomization Interface

private:
	
	void OnGroupCheckStateChanged( ECheckBoxState InNewState, TSharedPtr<IPropertyHandleArray> RecordedPropertiesArrayHandle, FString GroupName) const;
	ECheckBoxState OnGroupGetCheckState( TSharedPtr<IPropertyHandleArray> RecordedPropertiesArrayHandle, FString GroupName) const;
	
	void OnCheckStateChanged( ECheckBoxState InNewState, TSharedRef<IPropertyHandle> PropertyHandle );
	ECheckBoxState OnGetCheckState( TSharedRef<IPropertyHandle> PropertyHandle ) const;
};
}

