// Copyright Epic Games, Inc. All Rights Reserved.

#include "RecorderSourceObjectCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "TakeRecorderSource.h"

#define LOCTEXT_NAMESPACE "SLevelSequenceTakeEditor"

namespace UE::TakeRecorder
{
void FRecorderSourceObjectCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FText NewTitle = ComputeTitle(DetailBuilder.GetDetailsViewSharedPtr());
	if (!NewTitle.IsEmpty())
	{
		// Edit the category and add *all* properties for the object to it
		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory("CustomCategory", NewTitle);

		UClass* BaseClass = DetailBuilder.GetBaseClass();
		while (BaseClass)
		{
			for (FProperty* Property : TFieldRange<FProperty>(BaseClass, EFieldIteratorFlags::ExcludeSuper))
			{
				CategoryBuilder.AddProperty(Property->GetFName(), BaseClass);
			}

			BaseClass = BaseClass->GetSuperClass();
		}
	}
}

FText FRecorderSourceObjectCustomization::ComputeTitle(const TSharedPtr<const IDetailsView>& DetailsView) const
{
	if (!DetailsView)
	{
		return FText();
	}

	// Compute the title for all the sources that this details panel is editing
	static const FName CategoryName = "Category";

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = DetailsView->GetSelectedObjects();
	UObject* FirstObject = SelectedObjects.Num() > 0 ? SelectedObjects[0].Get() : nullptr;

	if (FirstObject)
	{
		if (SelectedObjects.Num() == 1)
		{
			UTakeRecorderSource* Source = Cast<UTakeRecorderSource>(FirstObject);
			return Source ? Source->GetDisplayText() : FText::FromString(FirstObject->GetName());
		}
		else if (SelectedObjects.Num() > 1)
		{
			const FString& Category = FirstObject->GetClass()->GetMetaData(CategoryName);
			return FText::Format(LOCTEXT("CategoryFormatString", "{0} ({1})"), FText::FromString(Category), SelectedObjects.Num());
		}
	}

	return FText();
}
}

#undef LOCTEXT_NAMESPACE