// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IDetailsView.h"

namespace UE::TakeRecorder
{
class FRecorderSourceObjectCustomization : public IDetailCustomization
{
public:

	//~ Begin IDetailCustomization Interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface

private:
	
	FText ComputeTitle(const TSharedPtr<const IDetailsView>& DetailsView) const;
};
}
