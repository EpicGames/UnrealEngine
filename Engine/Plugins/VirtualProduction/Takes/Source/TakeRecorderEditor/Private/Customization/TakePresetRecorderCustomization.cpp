// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakePresetRecorderCustomization.h"

#include "ClassViewerFilter.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "LevelSequence.h"
#include "PropertyCustomizationHelpers.h"
#include "TakePresetSettings.h"

#define LOCTEXT_NAMESPACE "FTakePresetRecorderCustomization"

namespace UE::TakeRecorder
{
class FAssetClassParentFilter : public IClassViewerFilter
{
public:
	/** All children of these classes will be included unless filtered out by another setting. */
	TSet< const UClass* > AllowedChildrenOfClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};

void FTakePresetRecorderCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils
	)
{
	const TSharedPtr<IPropertyHandle> TargetRecordClass = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTakeRecorderTargetRecordClassProperty, TargetRecordClass));
	if (!ensure(TargetRecordClass))
	{
		return;
	}
	
	TSharedRef<FAssetClassParentFilter> ClassFilter = MakeShared<FAssetClassParentFilter>();
	ClassFilter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
	ClassFilter->AllowedChildrenOfClasses.Add(ULevelSequence::StaticClass());

	// The point of this is to customize the TargetRecordClass and only trigger a PostEditChange if the user clicked "Ok" in the dialogue.
	// A dialogue will only be shown if the user has made changes that would be discarded. If there's none, the property is silently changed.
	HeaderRow
		.NameContent() [ TargetRecordClass->CreatePropertyNameWidget() ]
		.ValueContent()
		[
			SNew(SClassPropertyEntryBox)
			.MetaClass(UObject::StaticClass())
			.AllowNone(false)
			.AllowAbstract(false)
			.SelectedClass_Lambda([this, TargetRecordClass]
			{
				UObject* ClassObject = nullptr;
				TargetRecordClass->GetValue(ClassObject);
				return Cast<UClass>(ClassObject);
			})
			.ClassViewerFilters({ ClassFilter })
			.OnSetClass_Lambda([this, TargetRecordClass](const UClass* Class)
			{
				if (Class && PromptChangeTargetRecordClassDelegate.Execute(Class))
				{
					TargetRecordClass->SetValue(Class);
				}
			})
		];
}
}

#undef LOCTEXT_NAMESPACE