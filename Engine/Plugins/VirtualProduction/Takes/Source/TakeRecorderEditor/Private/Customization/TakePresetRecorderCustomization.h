// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "IPropertyTypeCustomization.h"

namespace UE::TakeRecorder
{
/** Possibly asks the user whether they are sure they want to change TargetRecordClass. Returns whether the change can be made. */
DECLARE_DELEGATE_RetVal_OneParam(bool, FPromptChangeTargetRecordClass, const UClass* /*NewClass*/);
	
/** Asks the user whether they're sure they want to change the UTakePresetSettings::TargetRecordClass, as that usually requires clearing the pending change. */
class FTakePresetRecorderCustomization : public IPropertyTypeCustomization
{
public:

	explicit FTakePresetRecorderCustomization(FPromptChangeTargetRecordClass PromptUserDelegate)
		: PromptChangeTargetRecordClassDelegate(MoveTemp(PromptUserDelegate))
	{}

	//~ Begin IDetailCustomization Interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
	//~ End IDetailCustomization Interface

private:
	
	/** Possibly asks the user whether they are sure they want to change TargetRecordClass. Returns whether the change can be made. */
	FPromptChangeTargetRecordClass PromptChangeTargetRecordClassDelegate;
};

}
