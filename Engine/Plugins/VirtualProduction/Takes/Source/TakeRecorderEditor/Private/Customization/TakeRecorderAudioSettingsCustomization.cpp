// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderAudioSettingsCustomization.h"
#include "IDetailChildrenBuilder.h"
#include "TakeRecorderSettings.h"
#include "TakeRecorderSourceProperty.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "FAudioInputDevicePropertyCustomization"

void FAudioInputDevicePropertyCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	UseSystemDefaultHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAudioInputDeviceProperty, bUseSystemDefaultAudioDevice));
	InputDeviceHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAudioInputDeviceProperty, DeviceId));
	BufferSizeHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAudioInputDeviceProperty, AudioInputBufferSize));
	DeviceInfoArrayHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAudioInputDeviceProperty, DeviceInfoArray))->AsArray();

	// Create check box which indicates system default audio device should be used (disables device combo button)
	if (UseSystemDefaultHandle.IsValid())
	{
		IDetailPropertyRow& UseSystemDefaultRow = ChildBuilder.AddProperty(PropertyHandle);
		UseSystemDefaultRow.CustomWidget()
		.NameContent()
		[
			UseSystemDefaultHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([&]() -> ECheckBoxState
			{
				bool bUseSystemDefault;
				UseSystemDefaultHandle->GetValue(bUseSystemDefault);

				return bUseSystemDefault ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([&](ECheckBoxState NewState) -> void
			{
				bool bUseSystemDefault = (NewState == ECheckBoxState::Checked);
				UseSystemDefaultHandle->SetValue(bUseSystemDefault);

				// Refresh the device list
				RequestsDeviceListRefresh();
			})
		];
	}

	// Create audio input device combo button
	if (InputDeviceHandle.IsValid() && DeviceInfoArrayHandle.IsValid())
	{
		IDetailPropertyRow& InputDevicePropertyRow = ChildBuilder.AddProperty(PropertyHandle);

		DeviceInfoArrayHandle->SetOnNumElementsChanged(FSimpleDelegate::CreateSP(this, &FAudioInputDevicePropertyCustomization::OnDeviceListChanged));
		BuildAudioInputDeviceList();

		InputDevicePropertyRow.CustomWidget()
		.NameContent()
		[
			InputDeviceHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			CreateDeviceListComboButton()
		];
	}

	// Create audio callback buffer size widget
	if (BufferSizeHandle.IsValid())
	{
		IDetailPropertyRow& BufferSizePropertyRow = ChildBuilder.AddProperty(PropertyHandle);
		BufferSizePropertyRow.CustomWidget()
		.NameContent()
		[
			BufferSizeHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			BufferSizeHandle->CreatePropertyValueWidget()
		];
	}

	SynchronizeWidgetStates();

	// Register delegate so UI can update when the audio input device changes
	// This is needed because the device menu lives in both the project settings window and the take recorder panel
	if (UTakeRecorderAudioInputSettings* AudioInputSettings = TakeRecorderAudioSettingsUtils::GetTakeRecorderAudioInputSettings())
	{
		AudioInputSettings->GetOnAudioInputDeviceChanged().Add(FSimpleDelegate::CreateSP(this, &FAudioInputDevicePropertyCustomization::SynchronizeWidgetStates));
	}
}

FString FAudioInputDevicePropertyCustomization::GetDeviceNameFromInfoProperty(const TSharedPtr<IPropertyHandle>& InDeviceInfoHandle)
{
	FString DeviceNameStr;
	if (InDeviceInfoHandle && InDeviceInfoHandle.IsValid())
	{
		TSharedPtr<IPropertyHandle> DeviceNameHandle = InDeviceInfoHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAudioInputDeviceInfoProperty, DeviceName));
		if (DeviceNameHandle.IsValid())
		{
			DeviceNameHandle->GetValue(DeviceNameStr);
		}
	}

	return DeviceNameStr;
}

FString FAudioInputDevicePropertyCustomization::GetDeviceIdFromInfoProperty(const TSharedPtr<IPropertyHandle>& InDeviceInfoHandle)
{
	FString DeviceIdStr;
	if (InDeviceInfoHandle.IsValid())
	{
		TSharedPtr<IPropertyHandle> DeviceIdHandle = InDeviceInfoHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAudioInputDeviceInfoProperty, DeviceId));
		if (DeviceIdHandle.IsValid())
		{
			DeviceIdHandle->GetValue(DeviceIdStr);
		}
	}

	return DeviceIdStr;
}

int32 FAudioInputDevicePropertyCustomization::GetChannelCountFromInfoProperty(const TSharedPtr<IPropertyHandle>& InDeviceInfoHandle)
{
	int32 ChannelCount = 0;
	if (InDeviceInfoHandle.IsValid())
	{
		TSharedPtr<IPropertyHandle> ChannelCountHandle = InDeviceInfoHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAudioInputDeviceInfoProperty, InputChannels));
		if (ChannelCountHandle.IsValid())
		{
			ChannelCountHandle->GetValue(ChannelCount);
		}
	}

	return ChannelCount;
}

bool FAudioInputDevicePropertyCustomization::GetIsDefaultFromInfoProperty(const TSharedPtr<IPropertyHandle>& InDeviceInfoHandle)
{
	bool bIsDefaultDevice = false;
	if (InDeviceInfoHandle.IsValid())
	{
		TSharedPtr<IPropertyHandle> IsDefaultHandle = InDeviceInfoHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAudioInputDeviceInfoProperty, bIsDefaultDevice));
		if (IsDefaultHandle.IsValid())
		{
			IsDefaultHandle->GetValue(bIsDefaultDevice);
		}
	}

	return bIsDefaultDevice;
}

void FAudioInputDevicePropertyCustomization::BuildAudioInputDeviceList()
{
	FString CurrentDeviceId;
	InputDeviceHandle->GetValue(CurrentDeviceId);

	bool bHaveCurrentDevice = CurrentDeviceId.Len() != 0;
	uint32 NumDevices;
	DeviceInfoArrayHandle->GetNumElements(NumDevices);

	AudioInputDevices.Empty();

	for (uint32 DeviceIndex = 0; DeviceIndex < NumDevices; ++DeviceIndex)
	{
		TSharedRef<IPropertyHandle> DeviceInfoHandle = DeviceInfoArrayHandle->GetElement(DeviceIndex);
		if (DeviceInfoHandle->IsValidHandle())
		{
			bool bIsDefaultDevice = GetIsDefaultFromInfoProperty(DeviceInfoHandle);
			FString DeviceId = GetDeviceIdFromInfoProperty(DeviceInfoHandle);

			if (bIsDefaultDevice)
			{
				DefaultDeviceId = DeviceId;
			}

			if (bIsDefaultDevice && !bHaveCurrentDevice)
			{
				CurrentDeviceId = DeviceId;
				InputDeviceHandle->SetValue(CurrentDeviceId);
				SelectedAudioInputDevice = DeviceInfoHandle;
			}
			else if (bHaveCurrentDevice)
			{
				if (CurrentDeviceId == DeviceId)
				{
					SelectedAudioInputDevice = DeviceInfoHandle;
				}
			}

			AudioInputDevices.Add(DeviceInfoHandle);
		}
	}
}

void FAudioInputDevicePropertyCustomization::RequestsDeviceListRefresh()
{
	TakeRecorderAudioSettingsUtils::RefreshAudioInputSettings();

	BuildAudioInputDeviceList();
	SynchronizeWidgetStates();
}

void FAudioInputDevicePropertyCustomization::OnDeviceListChanged()
{
	BuildAudioInputDeviceList();
	SynchronizeWidgetStates();
}

void FAudioInputDevicePropertyCustomization::MenuItemRefreshSelected()
{
	RequestsDeviceListRefresh();
}

void FAudioInputDevicePropertyCustomization::MenuItemDeviceSelected(TSharedPtr<IPropertyHandle> InDeviceInfoHandle)
{
	SelectedAudioInputDevice = InDeviceInfoHandle;
	InputDeviceHandle->SetValue(GetDeviceIdFromInfoProperty(InDeviceInfoHandle));
}

TSharedRef<SWidget> FAudioInputDevicePropertyCustomization::OnGenerateDeviceMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("RefreshDeviceList", "Refresh List"),
		LOCTEXT("RefreshDeviceListToolTip", "Refresh contents of this menu's Audio Input Device List"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FAudioInputDevicePropertyCustomization::MenuItemRefreshSelected)
		)
	);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("AudioInputDeviceMenuSection", "Audio Input Devices"));
	{
		for (TSharedPtr<IPropertyHandle> DeviceInfoHandle : AudioInputDevices)
		{
			FString DeviceName = GetDeviceNameFromInfoProperty(DeviceInfoHandle);

			if (!DeviceName.IsEmpty())
			{
				MenuBuilder.AddMenuEntry(
					FText::FromString(DeviceName),
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &FAudioInputDevicePropertyCustomization::MenuItemDeviceSelected, DeviceInfoHandle)
					)
				);
			}
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FAudioInputDevicePropertyCustomization::CreateDeviceListComboButton()
{
	TSharedRef<SSimpleComboButton> NewWidget = SNew(SSimpleComboButton)
		.OnGetMenuContent(this, &FAudioInputDevicePropertyCustomization::OnGenerateDeviceMenu)
		.ToolTipText(LOCTEXT("AudioInputDeviceMenuToolTip", "Available Audio Input Devices"))
		.HasDownArrow(true)
		.UsesSmallText(true)
		.Text_Lambda([this]()
		{
			return FText::AsCultureInvariant(GetDeviceNameFromInfoProperty(SelectedAudioInputDevice));
		});

	DeviceListComboButton = NewWidget;

	return NewWidget;
}

void FAudioInputDevicePropertyCustomization::UpdateDeviceCombobxEnabled()
{
	// Get current data model value
	bool bUseSystemDefault;
	UseSystemDefaultHandle->GetValue(bUseSystemDefault);

	// We want the widget enabled if bUseSystemDefault is false
	bool bEnableWidget = !bUseSystemDefault;

	// Get the current widget state
	bool bIsWidgetEnabled = DeviceListComboButton->IsEnabled();

	// If the UI doesn't match the data model, refresh the UI
	if (bEnableWidget != bIsWidgetEnabled)
	{
		DeviceListComboButton->SetEnabled(bEnableWidget);
        
        // If disabling, set the current menu item to default
        if (!bEnableWidget)
        {
            InputDeviceHandle->SetValue(DefaultDeviceId);
        }
	}
}

void FAudioInputDevicePropertyCustomization::SynchronizeWidgetStates()
{
	UpdateDeviceCombobxEnabled();

	uint32 NumDevices;
	DeviceInfoArrayHandle->GetNumElements(NumDevices);

	// Get current data model value
	FString CurrentDeviceId;
	InputDeviceHandle->GetValue(CurrentDeviceId);

	// If the UI doesn't match the data model, refresh the UI
	if (CurrentDeviceId != GetDeviceIdFromInfoProperty(SelectedAudioInputDevice))
	{
		for (uint32 DeviceIndex = 0; DeviceIndex < NumDevices; ++DeviceIndex)
		{
			TSharedRef<IPropertyHandle> DeviceInfoHandle = DeviceInfoArrayHandle->GetElement(DeviceIndex);
			if (DeviceInfoHandle->IsValidHandle())
			{
				FString DeviceId = GetDeviceIdFromInfoProperty(DeviceInfoHandle);

				if (DeviceId == CurrentDeviceId)
				{
					SelectedAudioInputDevice = DeviceInfoHandle;
					DeviceListComboButton->Invalidate(EInvalidateWidgetReason::Paint);
					break;
				}
			}
		}
	}
}

UTakeRecorderAudioInputSettings* TakeRecorderAudioSettingsUtils::GetTakeRecorderAudioInputSettings()
{
	const UTakeRecorderProjectSettings* ProjectSettings = GetDefault<UTakeRecorderProjectSettings>();
	for (TWeakObjectPtr<UObject> WeakObject : ProjectSettings->AdditionalSettings)
	{
		UObject* Object = WeakObject.Get();
		if (Object)
		{
			if (Object->IsA<UTakeRecorderAudioInputSettings>())
			{
				return Cast<UTakeRecorderAudioInputSettings>(Object);
			}
		}
	}
	return nullptr;
}

void TakeRecorderAudioSettingsUtils::RefreshAudioInputSettings()
{
	// Enumerate audio devices which will rebuild the device list. Note, this can be expensive depending on the hardware
	// attached to the machine.
	if (UTakeRecorderAudioInputSettings* AudioInputSettings = TakeRecorderAudioSettingsUtils::GetTakeRecorderAudioInputSettings())
	{
		AudioInputSettings->EnumerateAudioDevices(true);
	}
}

#undef LOCTEXT_NAMESPACE // "FAudioInputDevicePropertyCustomization"
