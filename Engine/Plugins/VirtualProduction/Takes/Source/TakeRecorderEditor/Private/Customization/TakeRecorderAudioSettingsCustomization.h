// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailWidgetRow.h"
#include "IPropertyTypeCustomization.h"
#include "TakeRecorderSourceProperty.h"

#include "SSimpleComboButton.h"
#include "Widgets/Text/STextBlock.h"

class FAudioInputDevicePropertyCustomization : public IPropertyTypeCustomization
{
public:

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:

	static FString GetDeviceNameFromInfoProperty(const TSharedPtr<IPropertyHandle>& InDeviceInfoHandle);

	FString GetDeviceIdFromInfoProperty(const TSharedPtr<IPropertyHandle>& InDeviceInfoHandle);
	int32 GetChannelCountFromInfoProperty(const TSharedPtr<IPropertyHandle>& InDeviceInfoHandle);

	bool GetIsDefaultFromInfoProperty(const TSharedPtr<IPropertyHandle>& InDeviceInfoHandle);
	void BuildAudioInputDeviceList();
	TSharedRef<SWidget> CreateDeviceListComboButton();

	void RequestsDeviceListRefresh();
	TSharedRef<SWidget> OnGenerateDeviceMenu();
	void MenuItemRefreshSelected();
	void MenuItemDeviceSelected(TSharedPtr<IPropertyHandle> InDeviceInfoHandle);

	void OnDeviceListChanged();
	void UpdateDeviceCombobxEnabled();
	void SynchronizeWidgetStates();

	TSharedPtr<IPropertyHandle> UseSystemDefaultHandle;
	TSharedPtr<IPropertyHandle> InputDeviceHandle;
	TSharedPtr<IPropertyHandle> BufferSizeHandle;
	TSharedPtr<IPropertyHandleArray> DeviceInfoArrayHandle;

	TSharedPtr<IPropertyHandle> SelectedAudioInputDevice;
	TArray<TSharedPtr<IPropertyHandle>> AudioInputDevices;
	TSharedPtr<SSimpleComboButton> DeviceListComboButton;
	FString DefaultDeviceId;
};

class TakeRecorderAudioSettingsUtils
{
public:

	static UTakeRecorderAudioInputSettings* GetTakeRecorderAudioInputSettings();
	static void RefreshAudioInputSettings();
};