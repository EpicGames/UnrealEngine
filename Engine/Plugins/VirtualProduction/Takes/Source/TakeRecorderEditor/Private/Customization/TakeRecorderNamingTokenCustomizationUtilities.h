// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "ITakeRecorderNamingTokensModule.h"
#include "NamingTokensSpecifiers.h"
#include "SNamingTokensEditableTextBox.h"
#include "UObject/TextProperty.h"

/**
 * Helps customize Naming Tokens in the details panel.
 * We may want to move this to be public under the NamingTokens plugin for use with other plugins.
 */
namespace UE::TakeRecorder::NamingTokens
{
	/** Scopes our directory path customization to naming token directory paths only. */
	class FDirectoryPathPropertyTypeIdentifier : public IPropertyTypeIdentifier
	{
		virtual bool IsPropertyTypeCustomized(const IPropertyHandle& PropertyHandle) const override
		{
			return PropertyHandle.HasMetaData(*UE::NamingTokens::Specifiers::UseNamingTokens);
		}
	};

	/** Creates a naming tokens widget which will respond to a property handle. */
	inline TSharedRef<SNamingTokensEditableTextBox> CreateNamingTokensWidgetForProperty(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		const FString TakeRecorderNamespace = ITakeRecorderNamingTokensModule::GetTakeRecorderNamespace();
		FNamingTokenFilterArgs FilterArgs;
		FilterArgs.AdditionalNamespacesToInclude.Add(TakeRecorderNamespace);
		
		return SNew(SNamingTokensEditableTextBox)
			.ShouldEvaluateTokens(false)
			.FilterArgs(FilterArgs)
			.NamespaceSuggestionPriority({TakeRecorderNamespace})
			.Text_Lambda([InPropertyHandle]() -> FText
			{
				FText Value;
				if (InPropertyHandle.IsValid())
				{
					if (const FProperty* Property = InPropertyHandle->GetProperty())
					{
						if (Property->IsA<FStrProperty>())
						{
							FString Result;
							InPropertyHandle->GetValue(Result);
							Value = FText::FromString(Result);
						}
						else if (Property->IsA<FNameProperty>())
						{
							FName Result;
							InPropertyHandle->GetValue(Result);
							Value = FText::FromName(Result);
						}
						else if (Property->IsA<FTextProperty>())
						{
							InPropertyHandle->GetValue(Value);
						}
					}
				}
				return Value;
			})
			.OnTextChanged_Lambda([InPropertyHandle](const FText& InText)
			{
				if (InPropertyHandle.IsValid())
				{
					if (const FProperty* Property = InPropertyHandle->GetProperty())
					{
						if (Property->IsA<FStrProperty>())
						{
							InPropertyHandle->SetValue(InText.ToString());
						}
						else if (Property->IsA<FNameProperty>())
						{
							InPropertyHandle->SetValue(*InText.ToString());
						}
						else if (Property->IsA<FTextProperty>())
						{
							InPropertyHandle->SetValue(InText);
						}
					}
				}
			});
	}

	/** Given a detail property row create a custom naming tokens widget. */
	inline void HandleNamingTokensRow(IDetailPropertyRow& InPropertyRow)
	{
		const TSharedPtr<IPropertyHandle> PropertyHandle = InPropertyRow.GetPropertyHandle();
		InPropertyRow.CustomWidget()
	   .NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			CreateNamingTokensWidgetForProperty(PropertyHandle)
		];
	}
}
