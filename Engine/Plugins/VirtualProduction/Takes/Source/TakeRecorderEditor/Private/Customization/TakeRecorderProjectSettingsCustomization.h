// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customization/TakeRecorderAudioSettingsCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailCustomization.h"
#include "Modules/ModuleManager.h"
#include "NamingTokensSpecifiers.h"
#include "TakeRecorderNamingTokenCustomizationUtilities.h"
#include "TakeRecorderSettings.h"
#include "Timecode/HitchProtectionModel.h"

class FTakeRecorderProjectSettingsCustomization : public IDetailCustomization
{
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// Register audio device details customization for the project settings window
		PropertyEditorModule.RegisterCustomPropertyTypeLayout(FName(TEXT("AudioInputDeviceProperty")), FOnGetPropertyTypeCustomizationInstance::CreateLambda(&MakeShared<FAudioInputDevicePropertyCustomization>));
		
		// Pop the take recorder category to the top
		DetailLayout.EditCategory("Take Recorder");
		// Show the Hitch Protection category after "Take Recorder"
		CustomizeHitchProtection(DetailLayout);

		TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
		DetailLayout.GetObjectsBeingCustomized(CustomizedObjects);
		
		TArray<IDetailPropertyRow*> DetailRows;
		
		for (TWeakObjectPtr<UObject> EditObject : CustomizedObjects)
		{
			UTakeRecorderProjectSettings* Settings = Cast<UTakeRecorderProjectSettings>(EditObject.Get());
			if (!Settings)
			{
				continue;
			}

			// Get main properties
			{
				const TSharedRef<IPropertyHandle> SettingsProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UTakeRecorderProjectSettings,
					Settings));

				IDetailCategoryBuilder& SettingCategoryBuilder = DetailLayout.EditCategory(*SettingsProperty->GetDefaultCategoryName().ToString(),
					FText::GetEmpty(), ECategoryPriority::Important);

				TArray<TSharedRef<IPropertyHandle>> StructProperties;
				uint32 NumChildren = 0;
				SettingsProperty->GetNumChildren(NumChildren);

				for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
				{
					TSharedPtr<IPropertyHandle> ChildHandle = SettingsProperty->GetChildHandle(ChildIndex);
					if (ChildHandle.IsValid() && ChildHandle->IsValidHandle())
					{
						// Always add the property so we maintain proper order. This property has ShowOnlyInnerProperties set so this
						// respects that behavior.
						IDetailPropertyRow& DetailRow = SettingCategoryBuilder.AddProperty(ChildHandle);

						const FStructProperty* StructProp = CastField<FStructProperty>(ChildHandle->GetProperty());
						if (ChildHandle->HasMetaData(*UE::NamingTokens::Specifiers::UseNamingTokens)
						&& (!StructProp || StructProp->Struct != FDirectoryPath::StaticStruct())) // Paths customized separately
						{
							DetailRows.Add(&DetailRow);
						}
					}
				}
			}

			for (TWeakObjectPtr<UObject> WeakAdditionalSettings : Settings->AdditionalSettings)
			{
				UObject* AdditionalSettings = WeakAdditionalSettings.Get();
				if (!AdditionalSettings)
				{
					continue;
				}

				UClass* Class = AdditionalSettings->GetClass();
				TArray<FProperty*> EditProperties;
				for (FProperty* Property : TFieldRange<FProperty>(Class))
				{
					if (Property && Property->HasAllPropertyFlags(CPF_Edit | CPF_Config))
					{
						EditProperties.Add(Property);
					}
				}

				if (EditProperties.Num() > 0)
				{
					IDetailCategoryBuilder& Category = DetailLayout.EditCategory(*Class->GetDisplayNameText().ToString());

					TArray<UObject*> SettingAsArray = { AdditionalSettings };
					for (const FProperty* Property : EditProperties)
					{
						if (IDetailPropertyRow* DetailRow = Category.AddExternalObjectProperty(SettingAsArray, Property->GetFName()))
						{
							if (Property->HasMetaData(*UE::NamingTokens::Specifiers::UseNamingTokens))
							{
								DetailRows.Add(DetailRow);
							}
						}
					}
				}
			}
		}

		for (IDetailPropertyRow* DetailRow : DetailRows)
		{
			UE::TakeRecorder::NamingTokens::HandleNamingTokensRow(*DetailRow);
		}
	}

private:

	void CustomizeHitchProtection(IDetailLayoutBuilder& DetailLayout)
	{
		// Create a separate "Hitch Protection" category right under.
		// ShowOnlyInnerProperties does not work correctly... we need to manually add the children ... otherwise they end up under the Take Recorder category.
		IDetailCategoryBuilder& HitchProtectionCategory = DetailLayout.EditCategory("Hitch Protection");
		const TSharedRef<IPropertyHandle> HitchProtectionSettings = DetailLayout.GetProperty(
			GET_MEMBER_NAME_CHECKED(UTakeRecorderProjectSettings, HitchProtectionSettings)
		);
		
		// Structs have a root node, under which their properties are nested. Since we're promoting them up, we need to hide this node.
		HitchProtectionCategory.AddProperty(HitchProtectionSettings).EditCondition(false, {}).EditConditionHides(true);

		// Enabled property show should first...
		AddHitchProtectionEnabledProperty(HitchProtectionCategory, HitchProtectionSettings);
		
		// ... then all the others.
		const auto AddProperty = [&HitchProtectionCategory, &HitchProtectionSettings](FName PropertyName)
		{
			HitchProtectionCategory.AddProperty(HitchProtectionSettings->GetChildHandle(PropertyName))
				.EditCondition(TAttribute<bool>::CreateLambda([]()
				{
					return UE::TakeRecorder::HitchProtectionModel::CanInitializeHitchProtection();
				}),
				// Hide the property either if !bEnableHitchProtection or when !CanInitializeHitchProtection
				{}, ECustomEditConditionMode::Merge)
				.EditConditionHides(true);
		};
		AddProperty(GET_MEMBER_NAME_CHECKED(FTakeRecorderHitchProtectionParameters, RegressionBufferSizeInSeconds));
		AddProperty(GET_MEMBER_NAME_CHECKED(FTakeRecorderHitchProtectionParameters, CustomTimestep));
		AddProperty(GET_MEMBER_NAME_CHECKED(FTakeRecorderHitchProtectionParameters, MaxCatchupSeconds));
	}

	void AddHitchProtectionEnabledProperty(IDetailCategoryBuilder& HitchProtectionCategory, const TSharedRef<IPropertyHandle> HitchProtectionSettings)
	{
		const TSharedPtr<IPropertyHandle> EnableHitchProtectionProperty = HitchProtectionSettings->GetChildHandle(
			GET_MEMBER_NAME_CHECKED(FTakeRecorderHitchProtectionParameters, bEnableHitchProtection)
			);
		
		FDetailWidgetRow& EnableRow = HitchProtectionCategory.AddProperty(EnableHitchProtectionProperty)
			.EditCondition(TAttribute<bool>::CreateLambda([]()
			{
				const bool bCanEdit = UE::TakeRecorder::HitchProtectionModel::CanInitializeHitchProtection(); 
				return bCanEdit;
			}), {}).CustomWidget();

		const TSharedRef<SWidget> NameWidget = EnableHitchProtectionProperty->CreatePropertyNameWidget();
		ForEachChild(*NameWidget, [DefaultText = EnableHitchProtectionProperty->GetToolTipText()](SWidget& Widget)
		{
			Widget.SetToolTipText(TAttribute<FText>::CreateLambda([DefaultText]
			{
				return UE::TakeRecorder::HitchProtectionModel::CanInitializeHitchProtection()
					? DefaultText
					: UE::TakeRecorder::HitchProtectionModel::GetHitchProtectionDisabledReasonText();
			}));
		});
		
		EnableRow.NameContent() [ NameWidget ];
		EnableRow.ValueContent() [ EnableHitchProtectionProperty->CreatePropertyValueWidget() ];
	}

	template<typename TCallback>
	static void ForEachChild(SWidget& Widget, TCallback&& Callback)
	{
		Callback(Widget);

		FChildren* Children = Widget.GetChildren();
		if (!Children)
		{
			return;
		}

		Children->ForEachWidget([&Callback](SWidget& Child)
		{
			ForEachChild(Child, Callback);
		});
	}
};
