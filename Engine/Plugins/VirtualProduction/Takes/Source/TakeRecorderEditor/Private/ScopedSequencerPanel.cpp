// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScopedSequencerPanel.h"
#include "Widgets/TakeRecorderWidgetConstants.h"
#include "TakeRecorderSettings.h"
#include "TakeRecorderStyle.h"
#include "LevelSequence.h"
#include "TakeRecorderStyle.h"

// Sequencer includes
#include "ISequencer.h"
#include "SequencerSettings.h"

// Slate includes
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Styling/SlateIconFinder.h"

// Style includes
#include "Styling/AppStyle.h"

// UnrealEd includes
#include "Subsystems/AssetEditorSubsystem.h"

// LevelSequenceEditor includes
#include "ILevelSequenceEditorToolkit.h"


FScopedSequencerPanel::FScopedSequencerPanel(const TAttribute<ULevelSequence*>& InLevelSequenceAttribute)
	: LevelSequenceAttribute(InLevelSequenceAttribute)
{
	if (GetDefault<UTakeRecorderUserSettings>()->bIsSequenceOpen)
	{
		Open();
	}
}

FScopedSequencerPanel::~FScopedSequencerPanel()
{
	Close();
}

bool FScopedSequencerPanel::IsOpen(ULevelSequence* InSequence)
{
	return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(InSequence, false) != nullptr;
}

void FScopedSequencerPanel::Open(ULevelSequence* InSequence)
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InSequence);

	IAssetEditorInstance*        AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(InSequence, false);
	ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);

	if (LevelSequenceEditor->GetSequencer())
	{
		USequencerSettings* SequencerSettings = USequencerSettingsContainer::GetOrCreate<USequencerSettings>(TEXT("TakeRecorderSequenceEditor"));

		LevelSequenceEditor->GetSequencer()->SetSequencerSettings(SequencerSettings);
	}
}

void FScopedSequencerPanel::Close(ULevelSequence* InSequence)
{
	if (UAssetEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
	{
		EditorSubsystem->CloseAllEditorsForAsset(InSequence);
	}
}

ECheckBoxState FScopedSequencerPanel::GetToggleCheckState() const
{
	ULevelSequence* LevelSequence = LevelSequenceAttribute.Get();

	return LevelSequence && IsOpen(LevelSequence) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FScopedSequencerPanel::Toggle(ECheckBoxState CheckState)
{
	UTakeRecorderUserSettings* TakeRecorderUserSettings = GetMutableDefault<UTakeRecorderUserSettings>();

	ULevelSequence* LevelSequence = LevelSequenceAttribute.Get();
	if (LevelSequence)
	{
		if (IsOpen(LevelSequence))
		{
			Close(LevelSequence);
			TakeRecorderUserSettings->bIsSequenceOpen = false;
		}
		else
		{
			Open(LevelSequence);
			TakeRecorderUserSettings->bIsSequenceOpen = true;
		}
	}

	TakeRecorderUserSettings->SaveConfig();
}

bool FScopedSequencerPanel::IsOpen() const
{
	ULevelSequence* LevelSequence = LevelSequenceAttribute.Get();

	return LevelSequence ? IsOpen(LevelSequence) : false;
}

void FScopedSequencerPanel::Open()
{
	ULevelSequence* LevelSequence = LevelSequenceAttribute.Get();
	if (LevelSequence)
	{
		Open(LevelSequence);
	}
}

void FScopedSequencerPanel::Close()
{
	ULevelSequence* LevelSequence = LevelSequenceAttribute.Get();
	if (IsValid(LevelSequence))
	{
		Close(LevelSequence);
	}
}

TSharedRef<SWidget> FScopedSequencerPanel::MakeToggleButton()
{
	return SNew(SCheckBox)
	.Padding(TakeRecorder::ButtonPadding)
	.ToolTipText(NSLOCTEXT("TakeRecorder", "ToggleSequencer_Tip", "Show/Hide the Level Sequence that is used for setting up this take"))
	.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
	.IsChecked(this, &FScopedSequencerPanel::GetToggleCheckState)
	.OnCheckStateChanged(this, &FScopedSequencerPanel::Toggle)
	[
		SNew(SImage)
		.Image(FTakeRecorderStyle::Get().GetBrush("TakeRecorder.SequencerButton"))
	];
}