// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Misc/Attribute.h"

enum class ECheckBoxState : uint8;

class SWidget;
class ULevelSequence;

/**
 * Scoped object that manages opening and closing an externally supplied level sequence based on user-settings.
 * This class will invoke the sequencer tab for the sequence on construction if the user settings state that it should be open.
 * It also closes the sequence on destruction, and keeps track of whether the sequence should or should not be open when toggled.
 */
struct FScopedSequencerPanel : public TSharedFromThis<FScopedSequencerPanel>
{
	/** Constructor that opens the level sequence if UTakeRecorderUserSettings::bIsSequencerOpen is true */
	FScopedSequencerPanel(const TAttribute<ULevelSequence*>& InLevelSequenceAttribute);

	/** Destructor that closes the level sequence if it's currently open */
	~FScopedSequencerPanel();

	/**
	 * Check whether the sequence is currently open
	 */
	bool IsOpen() const;

	/**
	 * Open the level sequence in sequencer without changing bIsSequencerOpen
	 */
	void Open();

	/**
	 * Close the level sequence in sequencer if it's open, without changing bIsSequencerOpen
	 */
	void Close();

	/**
	 * Make a standard button for toggling the sequence
	 */
	TSharedRef<SWidget> MakeToggleButton();

private:

	void Toggle(ECheckBoxState CheckState);
	ECheckBoxState GetToggleCheckState() const;

	static bool IsOpen(ULevelSequence* InSequence);
	static void Open(ULevelSequence* InSequence);
	static void Close(ULevelSequence* InSequence);

	TAttribute<ULevelSequence*> LevelSequenceAttribute;
};