// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakePresetToolkit.h"
#include "TakePreset.h"
#include "TakeRecorderStyle.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "TakePresetToolkit"

const FName FTakePresetToolkit::TabId("TakePresetEditor");

void FTakePresetToolkit::Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UTakePreset* InTakePreset)
{
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_TakePresetEditor")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->Split
			(
				FTabManager::NewStack()->AddTab(TabId, ETabState::OpenedTab)
			)
		);

	TakePreset = InTakePreset;

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = false;

	const FName TakePresetEditor = "TakePresetEditor";
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, TakePresetEditor, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, TakePreset);
}

void FTakePresetToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(TakePreset);
}

FText FTakePresetToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Take Preset Editor");
}


FName FTakePresetToolkit::GetToolkitFName() const
{
	static FName TakePresetEditor("TakePresetEditor");
	return TakePresetEditor;
}


FLinearColor FTakePresetToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.7f, 0.0f, 0.0f, 0.5f);
}


FString FTakePresetToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Take ").ToString();
}

void FTakePresetToolkit::OnClose()
{
	OnClosedEvent.Broadcast();
}

#undef LOCTEXT_NAMESPACE
