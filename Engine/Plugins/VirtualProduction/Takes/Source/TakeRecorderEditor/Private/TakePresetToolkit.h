// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"

class UTakePreset;

class FTakePresetToolkit
	: public FAssetEditorToolkit
	, public FGCObject
{ 
public:

	/**
	 * Initialize this asset editor.
	 *
	 * @param Mode Asset editing mode for this editor (standalone or world-centric).
	 * @param InitToolkitHost When Mode is WorldCentric, this is the level editor instance to spawn this editor within.
	 * @param LevelSequence The animation to edit.
	 * @param TrackEditorDelegates Delegates to call to create auto-key handlers for this sequencer.
	 */
	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UTakePreset* TakePreset);

	UTakePreset* GetTakePreset() const
	{
		return TakePreset;
	}

	const FSlateBrush* GetTabIcon() const
	{
		return GetDefaultTabIcon();
	}

	/** Called when the tab manager is changed */
	DECLARE_EVENT(FTakePresetToolkit, FTakePresetToolkitClosed);
	FTakePresetToolkitClosed& OnClosed() { return OnClosedEvent; }

private:

	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override {}
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override {}
	virtual void OnClose() override;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FTakePresetToolkit");
	}

private:

	/**  */
	TObjectPtr<UTakePreset> TakePreset;

	static const FName TabId;

	/** Event that is called when this toolkit is closed */
	FTakePresetToolkitClosed OnClosedEvent;
};
