// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderCommands.h"
#include "TakeRecorderStyle.h"

#define LOCTEXT_NAMESPACE "TakeRecorderCommands"

FTakeRecorderCommands::FTakeRecorderCommands()
	: TCommands<FTakeRecorderCommands>("TakeRecorder", LOCTEXT("TakeRecorderCommandLabel", "Take Recorder"), NAME_None, FTakeRecorderStyle::StyleName)
{
}

void FTakeRecorderCommands::RegisterCommands()
{
	UI_COMMAND(StartRecording, "StartRecording", "Start recording", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::R));
	UI_COMMAND(StopRecording, "StopRecording", "Stop recording", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::S));
	UI_COMMAND(ClearRecordingIntegrityData, "Clear Recording Integrity Markers and Ranges", "Clear Recording Integrity Markers and Ranges", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
