// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

class FTakeRecorderCommands : public TCommands<FTakeRecorderCommands>
{
public:
	FTakeRecorderCommands();

	TSharedPtr<FUICommandInfo> StartRecording;
	TSharedPtr<FUICommandInfo> StopRecording;

	TSharedPtr<FUICommandInfo> ClearRecordingIntegrityData;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
