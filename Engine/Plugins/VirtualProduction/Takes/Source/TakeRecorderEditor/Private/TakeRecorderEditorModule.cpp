// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderEditorModule.h"

#include "ITakeRecorderModule.h"
#include "TakesCoreLog.h"
#include "TakeRecorderSettings.h"
#include "SequencerUtilities.h"
#include "Customization/NamingTokenDirectoryPathCustomization.h"
#include "Customization/TakeRecorderProjectSettingsCustomization.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "LevelEditor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ILevelSequenceEditorToolkit.h"
#include "TakeRecorderCommands.h"
#include "TakeRecorderStyle.h"
#include "TakeRecorderSettings.h"
#include "Recorder/TakeRecorder.h"
#include "Recorder/TakeRecorderSubsystem.h"

#include "WorkspaceMenuStructureModule.h"
#include "WorkspaceMenuStructure.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "ContentBrowserMenuContexts.h"
#include "Features/IModularFeatures.h"
#include "ITakeRecorderDropHandler.h"
#include "ISettingsModule.h"
#include "TakeRecorderSettings.h"
#include "TakePresetSettings.h"

#include "Widgets/STakeRecorderTabContent.h"
#include "Widgets/STakeRecorderCockpit.h"
#include "Widgets/STakeRecorderPanel.h"

#include "ISequencer.h"
#include "ISequencerModule.h"
#include "LevelSequence.h"
#include "LevelSequenceEditorModule.h"
#include "SequencerSettings.h"
#include "TakeMetaData.h"
#include "MovieSceneTakeSettings.h"
#include "FileHelpers.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "SerializedRecorder.h"
#include "Misc/Attribute.h"
#include "Textures/SlateIcon.h"

#include "Misc/QualifiedFrameTime.h"
#include "MovieSceneTakeSection.h"
#include "MovieSceneTakeTrack.h"
#include "ISequencer.h"
#include "LevelSequence.h"
#include "MovieSceneSequence.h"
#include "Sections/MovieSceneSubSection.h"
#include "MovieScene.h"
#include "UObject/Class.h"

#include "Algo/RemoveIf.h"
#include "Engine/Font.h"
#include "CanvasTypes.h"

#define LOCTEXT_NAMESPACE "TakeRecorderModule"

IMPLEMENT_MODULE(FTakeRecorderEditorModule, TakeRecorderEditor);

namespace UE::TakeRecorder::Private
{
    template<typename T>
    concept CInvocableWithMovieSceneTakeSection = std::is_invocable_v<T, UMovieSceneTakeSection*>;

	/** Render the timecode data with slate and rate to the canvas. Flag invalid TC values with a red color to indicate a problem to the user. */
	int32 RenderTimecode(FCanvas* Canvas, int32 X, int32 Y, const FTimecode& Timecode, const float& Rate, const FString& SequenceName)
	{
		UFont* Font = FPlatformProperties::SupportsWindowedMode() ? GEngine->GetSmallFont() : GEngine->GetMediumFont();
		const int32 RowHeight = FMath::TruncToInt(Font->GetMaxCharHeight());

		const bool bForceSignDisplay = false;
		const bool bAlwaysDisplaySubframe = true;

		const FString TimecodeStr = Timecode.ToString(bForceSignDisplay, bAlwaysDisplaySubframe);

		const FString TakeSection = TEXT("Take Section -- ");
		const FString FPS = TEXT("(00.00)");
		float CharWidth, CharHeight;
		Font->GetCharSize(TEXT(' '), CharWidth, CharHeight);

		int32 TakeSectionWidth = Font->GetStringSize(*TakeSection);
		int32 NewX = X - Font->GetStringSize(*SequenceName) - Font->GetStringSize(*FPS) - TakeSectionWidth - (int32)CharWidth;
		int32 SectionX = NewX + TakeSectionWidth;
		FColor Color = Rate > 6000 ? FColor::Red : FColor::Green;
		float DisplayRate = Rate > 6000 ? 0 : Rate;
		Canvas->DrawShadowedString(NewX, Y, *TakeSection, Font, FColor::Cyan);
		Canvas->DrawShadowedString(SectionX, Y, *FString::Printf(TEXT("%s TC: %s (%.2f)"), *SequenceName, *TimecodeStr, DisplayRate), Font, Color);
		Y += RowHeight;
		return Y;
	};

	template <typename CInvocableWithMovieSceneTakeSection>
	void IterateOverMovieSceneForSections(UMovieScene* MovieScene, CInvocableWithMovieSceneTakeSection&& SectionFunction)
	{
		if (!MovieScene)
		{
			return;
		}

		// Lambda to iterate over tracks.  This will either get called via a FMovieSceneBinding or directly froma UMovieScene
		auto ForEachTrack = [&SectionFunction](const TArray<UMovieSceneTrack*>& Tracks) -> bool
		{
			bool bDidRenderSection = false;
			for (UMovieSceneTrack* Track : Tracks)
			{
				for (UMovieSceneSection* Section : Track->GetAllSections())
				{
					if (UMovieSceneTakeSection* TakeSection = Cast<UMovieSceneTakeSection>(Section))
					{
						SectionFunction(TakeSection);
						bDidRenderSection = true;
					}
					else if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
					{
						if (SubSection->GetSequence())
						{
							IterateOverMovieSceneForSections(SubSection->GetSequence()->GetMovieScene(), SectionFunction);
						}
					}
				}
			}
			return bDidRenderSection;
		};

		const TArray<FMovieSceneBinding>& Bindings = ((const UMovieScene*)MovieScene)->GetBindings();
		bool bDidRenderForMovieScene = false;
		for (const FMovieSceneBinding& Binding : Bindings)
		{
			bDidRenderForMovieScene = ForEachTrack(Binding.GetTracks());
		}

		// If we didn't render anything with FMovieSceneBinding try the MovieScene object.
		if (!bDidRenderForMovieScene)
		{
			ForEachTrack(MovieScene->GetTracks());
		}
	}

	int32 RenderTakeSectionsInSequencer(FCanvas* Canvas, int32 X, int32 Y, TSharedPtr<ISequencer>& InSequencer)
	{
		const FFrameRate RootTickRate = InSequencer->GetRootTickResolution();
		const FFrameTime CurrentTime = InSequencer->GetGlobalTime().ConvertTo(RootTickRate);
		auto RenderOneTakeSubSection = [CurrentTime, Canvas, X, &Y](UMovieSceneTakeSection* TakeSection) mutable
		{
			TOptional<UMovieSceneTakeSection::FSectionData> TakeData = TakeSection->Evaluate(CurrentTime);
			if (TakeData)
			{
				Y = RenderTimecode(Canvas, X, Y, TakeData->Timecode, TakeData->Rate, TakeData->Slate);
			}
		};

		const TArray<FMovieSceneSequenceID>& SubSequenceHierarchy = InSequencer->GetSubSequenceHierarchy();
		if (SubSequenceHierarchy.Num()>0)
		{
			UMovieSceneSequence* Sequence = FSequencerUtilities::GetMovieSceneSequence(InSequencer, SubSequenceHierarchy.Last());
			check(Sequence);

			UMovieScene* MovieScene = Sequence->GetMovieScene();
			IterateOverMovieSceneForSections(MovieScene, RenderOneTakeSubSection);
		}
		return Y;
	}

	static FOpenSequencerWatcher SequencerWatcher;

	/** For the given sequencer, iterate over all sections and find a take section then evaluate and show timecode, rate, and slate info in the HUD. */
	int32 RenderTakeSectionTime(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
	{
		for (const FOpenSequencerWatcher::FOpenSequencerData& OpenSequencer : SequencerWatcher.OpenSequencers)
		{
			if (TSharedPtr<ISequencer> Sequencer = OpenSequencer.WeakSequencer.Pin())
			{
				Y = RenderTakeSectionsInSequencer(Canvas, X, Y, Sequencer);
			}
		}
		return Y;
	}

    void InitStatCommands()
    {
		auto StartupComplete = []()
		{
			check(GEngine);
			if (GIsEditor)
			{
                const bool bIsRHS = true;
				GEngine->AddEngineStat(TEXT("STAT_TakeTimecode"), TEXT("STATCAT_Sequencer"),
									   LOCTEXT("TakeTimecodeDisplay", "Displays current sequencer time value in NDF timecode format."),
									   UEngine::FEngineStatRender::CreateStatic(&RenderTakeSectionTime), nullptr, bIsRHS);
			}
		};

		SequencerWatcher.DoStartup(StartupComplete);
    }

	/** Get the sequencer for the given TakeRecorders sequence. Can return null. */
	TSharedPtr<ISequencer> GetSequencerForRecorder(UTakeRecorder* InRecorder)
	{
		if (!GIsEditor || !GEditor || InRecorder == nullptr)
		{
			return nullptr;
		}

		ULevelSequence* TargetSequence = InRecorder->GetSequence();
		if (TargetSequence == nullptr)
		{
			return nullptr;
		}

		TSharedPtr<ISequencer> TargetSequencer = nullptr;

		if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			if (IAssetEditorInstance* Editor = AssetEditorSubsystem->FindEditorForAsset(TargetSequence, /*bFocusIfOpen*/ false);
				Editor != nullptr && Editor->GetEditorName() == TEXT("LevelSequenceEditor"))
			{
				ILevelSequenceEditorToolkit* Toolkit = static_cast<ILevelSequenceEditorToolkit*>(Editor);
				TargetSequencer = Toolkit->GetSequencer();
			}
		}

		return TargetSequencer;
	}
} // namespace UE::TakeRecorder::Private

namespace
{
	static TSharedRef<SDockTab> SpawnTakesBrowserTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		TSharedRef<SDockTab> NewTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

		FContentBrowserConfig ContentBrowserConfig;
		{
			ContentBrowserConfig.ThumbnailLabel =  EThumbnailLabel::ClassName ;
			ContentBrowserConfig.ThumbnailScale = 0.1f;
			ContentBrowserConfig.InitialAssetViewType = EAssetViewType::Column;
			ContentBrowserConfig.bShowBottomToolbar = true;
			ContentBrowserConfig.bCanShowClasses = true;
			ContentBrowserConfig.bUseSourcesView = true;
			ContentBrowserConfig.bExpandSourcesView = true;
			ContentBrowserConfig.bUsePathPicker = true;
			ContentBrowserConfig.bCanShowFilters = true;
			ContentBrowserConfig.bCanShowAssetSearch = true;
			ContentBrowserConfig.bCanShowFolders = true;
			ContentBrowserConfig.bCanShowRealTimeThumbnails = true;
			ContentBrowserConfig.bCanShowDevelopersFolder = true;
			ContentBrowserConfig.bCanShowLockButton = true;
			ContentBrowserConfig.bCanSetAsPrimaryBrowser = false;
		}

		IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
		TSharedRef<SWidget> NewBrowser = ContentBrowser.CreateContentBrowser( ITakeRecorderModule::TakesBrowserInstanceName, NewTab, nullptr );

		NewTab->SetContent( NewBrowser );

		FString TakesDir = FPaths::GetPath(FPaths::GetPath(GetDefault<UTakeRecorderProjectSettings>()->Settings.GetTakeAssetPath()));
		TArray<FString> TakesFolders;
		TakesFolders.Push(TakesDir);
		ContentBrowser.SyncBrowserToFolders(TakesFolders, true, false, ITakeRecorderModule::TakesBrowserInstanceName);

		return NewTab;
	}

	static TSharedRef<SDockTab> SpawnTakeRecorderTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		TSharedRef<STakeRecorderTabContent> Content = SNew(STakeRecorderTabContent);
		TSharedPtr<SDockTab> ContentTab = SNew(SDockTab)
			.Label(Content, &STakeRecorderTabContent::GetTitle)
			.TabRole(ETabRole::NomadTab)
			[
				Content
			];
		const TAttribute<const FSlateBrush*> TabIcon = TAttribute<const FSlateBrush*>::CreateLambda([Content]()
			{
				return Content->GetIcon();
			});
		ContentTab->SetTabIcon(TabIcon);
		return ContentTab.ToSharedRef();
	}

	static void RegisterLevelEditorLayout(FLayoutExtender& Extender)
	{
		Extender.ExtendArea("TopLevelArea",
			[](TSharedRef<FTabManager::FArea> InArea)
			{
				InArea->SplitAt(1, 
					FTabManager::NewStack()
					->SetSizeCoefficient( 0.3f )
					->AddTab(ITakeRecorderModule::TakeRecorderTabName, ETabState::ClosedTab)
				);
			}
		);
	}

	static void RegisterTabImpl()
	{
		FTabSpawnerEntry& TabSpawner = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ITakeRecorderModule::TakeRecorderTabName, FOnSpawnTab::CreateStatic(SpawnTakeRecorderTab));

		TabSpawner
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCinematicsCategory())
			.SetDisplayName(ITakeRecorderModule::TakeRecorderTabLabel)
			.SetTooltipText(LOCTEXT("TakeRecorderTab_Tooltip", "Open the main Take Recorder UI."))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "SequenceRecorder.TabIcon"));

		FTabSpawnerEntry& TBTabSpawner = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ITakeRecorderModule::TakesBrowserTabName, FOnSpawnTab::CreateStatic(SpawnTakesBrowserTab));

		TBTabSpawner
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCinematicsCategory())
			.SetDisplayName(ITakeRecorderModule::TakesBrowserTabLabel)
			.SetTooltipText(LOCTEXT("TakeBrowserTab_Tooltip", "Open the Take Browser UI"))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.TabIcon"));
	}
	
	static void ModulesChangedCallback(FName ModuleName, EModuleChangeReason ReasonForChange)
	{
		static const FName LevelEditorModuleName(TEXT("LevelEditor"));
		if (ReasonForChange == EModuleChangeReason::ModuleLoaded && ModuleName == LevelEditorModuleName)
		{
			RegisterTabImpl();
		}
	}
}

FTakeRecorderEditorModule::FTakeRecorderEditorModule()
	: SequencerSettings(nullptr)
{
}

void FTakeRecorderEditorModule::StartupModule()
{
	FTakeRecorderStyle::Get();
	FTakeRecorderCommands::Register();

	RegisterDetailCustomizations();
	RegisterLevelEditorExtensions();
	RegisterSettings();
	RegisterMessageLogListing();
	TimecodeManagement = MakeUnique<UE::TakeRecorder::FHitchlessProtectionRootLogic>();
	if (GIsEditor)
	{
		SequencerHitchVisualizer = MakeUnique<UE::TakeRecorder::FSequencerHitchVisualizer>();
	}

	if (GIsEditor)
	{
		if (UToolMenus::TryGet())
		{
			RegisterMenus();
			bMenusRegisteredAtModuleStartup = true;
		}
		UE::TakeRecorder::Private::InitStatCommands();
	}

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FTakeRecorderEditorModule::HandleOnPostEngineInit);

	if (GEditor)
	{
		GEditor->OnEditorClose().AddRaw(this, &FTakeRecorderEditorModule::OnEditorClose);
	}
}

void FTakeRecorderEditorModule::ShutdownModule()
{
	// If we have an active recorder, ensure we unhook our callbacks from it.
	if (TStrongObjectPtr<UTakeRecorder> StrongActiveRecorder = WeakActiveRecorder.Pin())
	{
		StrongActiveRecorder->OnRecordingCancelled().RemoveAll(this);
		StrongActiveRecorder->OnRecordingFinished().RemoveAll(this);
		StrongActiveRecorder.Reset();
	}

	if (GEngine)
	{
		if (UTakeRecorderSubsystem* TRSubSystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>())
		{
			TRSubSystem->GetOnRecordingInitializedEvent().RemoveAll(this);
		}
	}

	if (GEditor)
	{
		GEditor->OnEditorClose().RemoveAll(this);
	}

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	FTakeRecorderCommands::Unregister();

	UnregisterDetailCustomizations();
	UnregisterLevelEditorExtensions();
	UnregisterSettings();
	UnregisterMessageLogListing();

	bMenusRegisteredAtModuleStartup = false;

	TimecodeManagement.Reset();
	SequencerHitchVisualizer.Reset();
}

void FTakeRecorderEditorModule::RegisterMenus()
{
	if (!UToolMenus::IsToolMenuUIEnabled())
	{
		return;
	}

	FToolMenuOwnerScoped MenuOwner("TakeRecorder");
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* Menu = ToolMenus->ExtendMenu("ContentBrowser.AssetContextMenu.LevelSequence");
	if (!Menu)
	{
		return;
	}

	FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
	Section.AddDynamicEntry("TakeRecorderActions", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
		if (!Context)
		{
			return;
		}

		if (Context->SelectedAssets.Num() == 1 && Context->SelectedAssets[0].IsInstanceOf(ULevelSequence::StaticClass()))
		{
			const FAssetData LevelSequenceAsset = Context->SelectedAssets[0];

			InSection.AddMenuEntry(
				"OpenInTakeRecorder_Label",
				LOCTEXT("OpenInTakeRecorder_Label", "Open in Take Recorder"),
				LOCTEXT("OpenInTakeRecorder_Tooltip", "Opens this level sequence asset in Take Recorder by copying its contents into the pending take"),
				FSlateIcon(FTakeRecorderStyle::StyleName, "TakeRecorder.TabIcon"),
				FExecuteAction::CreateLambda(
					[LevelSequenceAsset]
					{
						if (ULevelSequence* LevelSequence = Cast<ULevelSequence>(LevelSequenceAsset.GetAsset()))
						{
							FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
							TSharedPtr<SDockTab> DockTab = LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(ITakeRecorderModule::TakeRecorderTabName);
							if (DockTab.IsValid())
							{
								TSharedRef<STakeRecorderTabContent> TabContent = StaticCastSharedRef<STakeRecorderTabContent>(DockTab->GetContent());

								// If this sequence has already been recorded, set it up for viewing, otherwise start recording from it.
								UTakeMetaData* TakeMetaData = LevelSequence->FindMetaData<UTakeMetaData>();
								if (!TakeMetaData || !TakeMetaData->Recorded())
								{
									TabContent->SetupForRecording(LevelSequence);
								}
								else
								{
									TabContent->SetupForViewing(LevelSequence);
								}
							}
						}
					}
				)
			);
			InSection.AddMenuEntry(
				"RecordIntoTakeRecorder_Label",
				LOCTEXT("RecordWithTakeRecorder_Label", "Record with Take Recorder"),
				LOCTEXT("RecordWithTakeRecorder_Tooltip", "Opens this level sequence asset for recording into with Take Recorder"),
				FSlateIcon(FTakeRecorderStyle::StyleName, "TakeRecorder.TabIcon"),
				FExecuteAction::CreateLambda(
					[LevelSequenceAsset]
					{
						if (ULevelSequence* LevelSequence = Cast<ULevelSequence>(LevelSequenceAsset.GetAsset()))
						{
							FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
							TSharedPtr<SDockTab> DockTab = LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(ITakeRecorderModule::TakeRecorderTabName);
							if (DockTab.IsValid())
							{
								TSharedRef<STakeRecorderTabContent> TabContent = StaticCastSharedRef<STakeRecorderTabContent>(DockTab->GetContent());

								TabContent->SetupForRecordingInto(LevelSequence);
							}
						}
					}
				)
			);
		}
	}));
}

void FTakeRecorderEditorModule::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (SequencerSettings)
	{
		Collector.AddReferencedObject(SequencerSettings);
	}
}

void FTakeRecorderEditorModule::RegisterDetailCustomizations()
{
	if (GIsEditor)
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.RegisterCustomClassLayout(
			UTakeRecorderProjectSettings::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateLambda(&MakeShared<FTakeRecorderProjectSettingsCustomization>)
		);

		PropertyEditorModule.RegisterCustomPropertyTypeLayout(FName(TEXT("DirectoryPath")),
			FOnGetPropertyTypeCustomizationInstance::CreateLambda(&FNamingTokenDirectoryPathStructCustomization::MakeInstance),
			MakeShared<UE::TakeRecorder::NamingTokens::FDirectoryPathPropertyTypeIdentifier>());
	}
}

void FTakeRecorderEditorModule::UnregisterDetailCustomizations()
{
	if (UObjectInitialized() && !IsEngineExitRequested())
	{
		if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			PropertyEditorModule->UnregisterCustomClassLayout(UTakeRecorderProjectSettings::StaticClass()->GetFName());
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(FName(TEXT("DirectoryPath")),
				MakeShared<UE::TakeRecorder::NamingTokens::FDirectoryPathPropertyTypeIdentifier>());
		}
	}
}

void FTakeRecorderEditorModule::RegisterLevelEditorExtensions()
{
	if (GIsEditor)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

		LevelEditorLayoutExtensionHandle = LevelEditorModule.OnRegisterLayoutExtensions().AddStatic(RegisterLevelEditorLayout);

		if (LevelEditorModule.GetLevelEditorTabManager())
		{
			RegisterTabImpl();
		}
		else
		{
			LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddStatic(RegisterTabImpl);
		}

		if (!FModuleManager::Get().IsModuleLoaded("LevelEditor"))
		{
			ModulesChangedHandle = FModuleManager::Get().OnModulesChanged().AddStatic(ModulesChangedCallback);
		}
	}
}

void FTakeRecorderEditorModule::UnregisterLevelEditorExtensions() const
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ITakeRecorderModule::TakeRecorderTabName);
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ITakeRecorderModule::TakesBrowserTabName);
	}

	if (FLevelEditorModule* LevelEditorModulePtr = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditorModulePtr->OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);
	}

	FModuleManager::Get().OnModulesChanged().Remove(ModulesChangedHandle);
}

void FTakeRecorderEditorModule::RegisterSettings()
{
	ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");

	SequencerSettings = USequencerSettingsContainer::GetOrCreate<USequencerSettings>(TEXT("TakeRecorderSequenceEditor"));

	SettingsModule.RegisterSettings("Project", "Plugins", "Take Recorder",
		LOCTEXT("ProjectSettings_Label", "Take Recorder"),
		LOCTEXT("ProjectSettings_Description", "Configure project-wide defaults for take recorder."),
		GetMutableDefault<UTakeRecorderProjectSettings>()
	);

	SettingsModule.RegisterSettings("Editor", "ContentEditors", "Take Recorder",
		LOCTEXT("UserSettings_Label", "Take Recorder"),
		LOCTEXT("UserSettings_Description", "Configure user-specific settings for take recorder."),
		GetMutableDefault<UTakeRecorderUserSettings>()
	);

	SettingsModule.RegisterSettings("Editor", "ContentEditors", "TakeRecorderSequenceEditor",
		LOCTEXT("TakeRecorderSequenceEditorSettingsName", "Take Recorder Sequence Editor"),
		LOCTEXT("TakeRecorderSequenceEditorSettingsDescription", "Configure the look and feel of the Take Recorder Sequence Editor."),
		SequencerSettings);
}

void FTakeRecorderEditorModule::UnregisterSettings()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Take Recorder");
		SettingsModule->UnregisterSettings("Editor", "ContentEditors", "Take Recorder");
		SettingsModule->UnregisterSettings("Editor", "ContentEditors", "TakeRecorderSequenceEditor");
	}
}

void FTakeRecorderEditorModule::RegisterMessageLogListing()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowFilters = true;
	InitOptions.bShowPages = true;
	MessageLogModule.RegisterLogListing("TakeRecorder", LOCTEXT("TakeRecorderLog", "Take Recorder"), InitOptions);
}

void FTakeRecorderEditorModule::UnregisterMessageLogListing()
{
	if (FMessageLogModule* MessageLogModule = FModuleManager::GetModulePtr<FMessageLogModule>("MessageLog"))
	{
		MessageLogModule->UnregisterLogListing("TakeRecorder");
	}
}

void FTakeRecorderEditorModule::OnEditorClose()
{
	if (UTakeRecorder* ActiveRecorder = UTakeRecorder::GetActiveRecorder())
	{
		ActiveRecorder->Stop();
	}
}

void FTakeRecorderEditorModule::OnRecordingInitialized(UTakeRecorder* InRecorder)
{
	// if we are already tracking a recording, do not replace the data with a new one to avoid dangling delegates.
	if (WeakActiveRecorder.IsValid())
	{
		return;
	}

	WeakActiveRecorder = InRecorder;

	// Attach to the Recorder call backs
	if (InRecorder != nullptr)
	{
		InRecorder->OnRecordingCancelled().AddRaw(this, &FTakeRecorderEditorModule::OnRecordingFinished);
		InRecorder->OnRecordingFinished().AddRaw(this, &FTakeRecorderEditorModule::OnRecordingFinished);
		ConfigureSequencerViewForRecording(InRecorder);
	}
}

void FTakeRecorderEditorModule::OnRecordingFinished(UTakeRecorder* InRecorder)
{
	// If the recorder that has finished is not the one we are tracking, return early.
	if (!WeakActiveRecorder.IsValid() || WeakActiveRecorder.Get() != InRecorder)
	{
		return;
	}

	if (InRecorder != nullptr)
	{
		InRecorder->OnRecordingCancelled().RemoveAll(this);
		InRecorder->OnRecordingFinished().RemoveAll(this);
		RevertSequencerViewForRecording(InRecorder);
	}

	WeakActiveRecorder.Reset();
}

void FTakeRecorderEditorModule::ConfigureSequencerViewForRecording(UTakeRecorder* InRecorder)
{
	TSharedPtr<ISequencer> TargetSequencer = UE::TakeRecorder::Private::GetSequencerForRecorder(InRecorder);
	if (!TargetSequencer.IsValid())
	{
		return;
	}

	if (const UTakeRecorderProjectSettings* ProjectSettings = GetDefault<UTakeRecorderProjectSettings>())
	{
		bWasSequencerSimpleViewBeforeRecording = TargetSequencer->IsSimpleView();
		if (!bWasSequencerSimpleViewBeforeRecording && ProjectSettings->Settings.bRecordWithSimpleSequencerView)
		{
			TrySetSequencerSimpleView(TargetSequencer.ToSharedRef(), true);
		}
	}
}

void FTakeRecorderEditorModule::RevertSequencerViewForRecording(UTakeRecorder* InRecorder)
{
	if (bWasSequencerSimpleViewBeforeRecording)
	{
		// If the sequencer was already in simple view when we started recording, we do not need to do anything.
		return;
	}

	TSharedPtr<ISequencer> TargetSequencer = UE::TakeRecorder::Private::GetSequencerForRecorder(InRecorder);
	if (!TargetSequencer.IsValid())
	{
		return;
	}
	
	TrySetSequencerSimpleView(TargetSequencer.ToSharedRef(), false);
}

bool FTakeRecorderEditorModule::TrySetSequencerSimpleView(TSharedRef<ISequencer> InSequencer, bool bInShowSimple)
{
	bool bViewTypeChanged = false;
	if (bInShowSimple)
	{
		// Only set the view if it is not already set to simple and it supports simple view
		if (InSequencer->GetHostCapabilities().bSupportsSimpleView && !InSequencer->IsSimpleView())
		{
			InSequencer->EnableSimpleView(true);
			bViewTypeChanged = true;
		}
	}
	else if (InSequencer->IsSimpleView())
	{
		// Only set the view if it is not already set to full.
		InSequencer->EnableSimpleView(false);
		bViewTypeChanged = true;
	}

	return bViewTypeChanged;
}

void FTakeRecorderEditorModule::HandleOnPostEngineInit()
{
	if (GEngine)
	{
		if (UTakeRecorderSubsystem* TRSubSystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>())
		{
			TRSubSystem->GetOnRecordingInitializedEvent().AddRaw(this, &FTakeRecorderEditorModule::OnRecordingInitialized);
		}
	}

	if (GIsEditor)
	{
		// If the menus were not registered at module startup, register them now.
		if (!bMenusRegisteredAtModuleStartup)
		{
			RegisterMenus();
		}
	}
}

#undef LOCTEXT_NAMESPACE
