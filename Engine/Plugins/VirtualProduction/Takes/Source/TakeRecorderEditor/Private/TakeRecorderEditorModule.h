// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "Timecode/HitchlessProtectionRootLogic.h"
#include "Timecode/Visualization/Sequencer/SequencerHitchVisualizer.h"
#include "UObject/GCObject.h"
#include "Timecode/HitchlessProtectionRootLogic.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FExtender;
class FTakePresetActions;
class UTakePreset;
class FSerializedRecorder;
class UTakeRecorder;
class UTakeRecorderSources;
class USequencerSettings;
class ISequencer;

class FTakeRecorderEditorModule : public IModuleInterface, public FGCObject
{
public:
	FTakeRecorderEditorModule();

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface
	
private:
	
	//~ Begin FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return "FTakeRecorderModule"; }
	//~ End FGCObject Interface

	void RegisterDetailCustomizations();
	void UnregisterDetailCustomizations();

	void RegisterLevelEditorExtensions();
	void UnregisterLevelEditorExtensions() const;

	void RegisterSettings();
	void UnregisterSettings();

	/** Register / unregister the "TakeRecorder" listing with the editor Message Log so messages
	    written via FMessageLog("TakeRecorder") get a labelled entry in the Message Log UI. */
	void RegisterMessageLogListing();
	void UnregisterMessageLogListing();

	void RegisterMenus();

	void OnEditorClose();
	
	/** Callback when a new recording is initialized. */
	void OnRecordingInitialized(UTakeRecorder* InRecorder);
	/** Callback triggered when a recording is finished - either stopped or canceled. */
	void OnRecordingFinished(UTakeRecorder* InRecorder);

	/** Configure the sequencer view for recording for the given recorder. */
	void ConfigureSequencerViewForRecording(UTakeRecorder* InRecorder);

	/** Revert the sequencer view to its pre recording state. */
	void RevertSequencerViewForRecording(UTakeRecorder* InRecorder);

	/**
	 * Try and set the sequencer simple view state. 
	 * This will only return true if the view type was changed.
	 */
	bool TrySetSequencerSimpleView(TSharedRef<ISequencer> InSequencer, bool bInShowSimple);

	/** Function called when the engine has finished initialzing. */
	void HandleOnPostEngineInit();

private:
	/** If the sequencer was in simple view when recording started. */
	bool bWasSequencerSimpleViewBeforeRecording = false;

	FDelegateHandle LevelEditorLayoutExtensionHandle;
	FDelegateHandle LevelEditorTabManagerChangedHandle;
	FDelegateHandle ModulesChangedHandle;

	TSharedPtr<FSerializedRecorder> SerializedRecorder;

	/**
	 * Sets up all sub-systems related to managing timecode in Take Recorder
	 * 
	 * For future maintainers: this could totally live in a separate module / plugin / whatever.
	 * There's no technical reason to have this live in TakeRecorder. The reason it's here is that we probably always want hitching to be handled
	 * gracefully.
	 */
	TUniquePtr<UE::TakeRecorder::FHitchlessProtectionRootLogic> TimecodeManagement;
#if WITH_EDITOR
	TUniquePtr<UE::TakeRecorder::FSequencerHitchVisualizer> SequencerHitchVisualizer;
#endif

	/** Extra class default objects that should appear in the Take Recorder panel. */
	TArray<TWeakObjectPtr<>> ExternalObjects;

	/** The settings object registers with the Settings module. */
	TObjectPtr<USequencerSettings> SequencerSettings;

	/** If the menus were registered at module startup. */
	bool bMenusRegisteredAtModuleStartup = false;

	/** Weak pointer to the active Take Recorder. */
	TWeakObjectPtr<UTakeRecorder> WeakActiveRecorder = nullptr;
};