// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"

FName FTakeRecorderStyle::StyleName("TakeRecorderStyle");

FTakeRecorderStyle::FTakeRecorderStyle()
	: FSlateStyleSet(StyleName)
{
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	const FVector2D Icon48x48(48.0f, 48.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	const FLinearColor White(FLinearColor::White);
	const FLinearColor AlmostWhite( FColor(200, 200, 200) );
	const FLinearColor VeryLightGrey( FColor(128, 128, 128) );
	const FLinearColor LightGrey( FColor(96, 96, 96) );
	const FLinearColor MediumGrey( FColor(62, 62, 62) );
	const FLinearColor DarkGrey( FColor(30, 30, 30) );
	const FLinearColor Black(FLinearColor::Black);


	const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("Takes"))->GetContentDir();
	SetContentRoot(ContentDir);
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	FButtonStyle Button = FButtonStyle()
		.SetNormal(FSlateBoxBrush(RootToContentDir("ButtonHoverHint.png"), FMargin(4/16.0f), FLinearColor(1,1,1,0.15f)))
		.SetHovered(FSlateBoxBrush(RootToContentDir("ButtonHoverHint.png"), FMargin(4/16.0f), FLinearColor(1,1,1,0.25f)))
		.SetPressed(FSlateBoxBrush(RootToContentDir("ButtonHoverHint.png"), FMargin(4/16.0f), FLinearColor(1,1,1,0.30f)))
		.SetNormalPadding( FMargin(0,0,0,1) )
		.SetPressedPadding( FMargin(0,1,0,0) );

	FButtonStyle FlatButton = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("FlatButton.Success");
	FlatButton.SetNormalPadding(FMargin(0,0,0,1));
	FlatButton.SetNormalPadding(FMargin(0,1,0,0));

	FComboButtonStyle ComboButton = FComboButtonStyle()
		.SetButtonStyle(Button.SetNormal(FSlateNoResource()))
		.SetDownArrowImage(FSlateImageBrush(RootToCoreContentDir(TEXT("Common/ComboArrow.png")), Icon8x8))
		.SetMenuBorderBrush(FSlateBoxBrush(RootToCoreContentDir(TEXT("Old/Menu_Background.png")), FMargin(8.0f/64.0f)))
		.SetMenuBorderPadding(FMargin(0.0f));

	FButtonStyle PressHintOnly = FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateNoResource())
		.SetPressed(FSlateNoResource())
		.SetNormalPadding( FMargin(0,0,0,1) )
		.SetPressedPadding( FMargin(0,1,0,0) );

	FSpinBoxStyle TakeInput = FSpinBoxStyle()
		.SetTextPadding(FMargin(0))
		.SetBackgroundBrush(FSlateNoResource()) 
		.SetHoveredBackgroundBrush(FSlateNoResource())
		.SetInactiveFillBrush(FSlateNoResource())
		.SetActiveFillBrush(FSlateNoResource())
		.SetForegroundColor(FSlateColor::UseForeground())
		.SetArrowsImage(FSlateNoResource());

	FCheckBoxStyle RecordButton = FCheckBoxStyle()
		.SetUncheckedImage(FSlateImageBrush(RootToContentDir(TEXT("RecordButton_Idle.png")), Icon32x32))
		.SetUncheckedHoveredImage(FSlateImageBrush(RootToContentDir(TEXT("RecordButton_Hovered.png")), Icon32x32))
		.SetUncheckedPressedImage(FSlateImageBrush(RootToContentDir(TEXT("RecordButton_Pressed.png")), Icon32x32))
		.SetCheckedImage(FSlateImageBrush(RootToContentDir(TEXT("StopButton_Idle.png")), Icon32x32))
		.SetCheckedHoveredImage(FSlateImageBrush(RootToContentDir(TEXT("StopButton_Hovered.png")), Icon32x32))
		.SetCheckedPressedImage(FSlateImageBrush(RootToContentDir(TEXT("StopButton_Pressed.png")), Icon32x32))
		.SetPadding( FMargin(0,0,0,1) );


	FCheckBoxStyle SwitchStyle = FCheckBoxStyle()
		.SetForegroundColor(FLinearColor::White)
		.SetUncheckedImage(		  FSlateImageBrush(RootToContentDir(TEXT("Switch_OFF.png")), FVector2D(28.0F, 14.0F)))
		.SetUncheckedHoveredImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_OFF.png")), FVector2D(28.0F, 14.0F)))
		.SetUncheckedPressedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_OFF.png")), FVector2D(28.0F, 14.0F)))
		.SetCheckedImage(         FSlateImageBrush(RootToContentDir(TEXT("Switch_ON.png")),  FVector2D(28.0F, 14.0F)))
		.SetCheckedHoveredImage(  FSlateImageBrush(RootToContentDir(TEXT("Switch_ON.png")),  FVector2D(28.0F, 14.0F)))
		.SetCheckedPressedImage(  FSlateImageBrush(RootToContentDir(TEXT("Switch_ON.png")),  FVector2D(28.0F, 14.0F)))
		.SetPadding( FMargin(0,0,0,1) );

	FTextBlockStyle TextStyle = FTextBlockStyle()
		.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 9))
		.SetColorAndOpacity(AlmostWhite);

	FEditableTextBoxStyle EditableTextStyle = FEditableTextBoxStyle()
		.SetTextStyle(FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 9))
		.SetBackgroundImageNormal(FSlateNoResource())
		.SetBackgroundImageHovered(FSlateNoResource())
		.SetBackgroundImageFocused(FSlateNoResource())
		.SetBackgroundImageReadOnly(FSlateNoResource())
		.SetBackgroundColor(FLinearColor::Transparent)
		.SetForegroundColor(FSlateColor::UseForeground());


	const FCheckBoxStyle ToggleButtonStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage( FSlateNoResource() )
		.SetUncheckedHoveredImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")), 4.0f/16.0f, FStyleColors::PrimaryHover) )
		.SetUncheckedPressedImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")), 4.0f/16.0f, FStyleColors::PrimaryPress) )
		.SetCheckedImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")),  4.0f/16.0f, FStyleColors::PrimaryPress) )
		.SetCheckedHoveredImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")),  4.0f/16.0f, FStyleColors::PrimaryHover) )
		.SetCheckedPressedImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")),  4.0f/16.0f, FStyleColors::PrimaryPress) )
		.SetUndeterminedImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")),  4.0f/16.0f, FStyleColors::SelectInactive) )
		.SetUndeterminedHoveredImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")),  4.0f/16.0f, FStyleColors::SelectInactive) )
		.SetUndeterminedPressedImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")),  4.0f/16.0f, FStyleColors::SelectInactive) );

	const FCheckBoxStyle ToggleButtonIndicatorStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage( FSlateNoResource() )
		.SetUncheckedHoveredImage( FSlateNoResource() )
		.SetUncheckedPressedImage( FSlateNoResource() )
		.SetCheckedImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")),  4.0f/16.0f, FStyleColors::PrimaryPress) )
		.SetCheckedHoveredImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")),  4.0f/16.0f, FStyleColors::PrimaryHover) )
		.SetCheckedPressedImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")),  4.0f/16.0f, FStyleColors::PrimaryPress) )
		.SetUndeterminedImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")),  4.0f/16.0f, FStyleColors::SelectInactive) )
		.SetUndeterminedHoveredImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")),  4.0f/16.0f, FStyleColors::SelectInactive) )
		.SetUndeterminedPressedImage( FSlateBoxBrush(RootToCoreContentDir(TEXT("Common/RoundedSelection_16x.png")),  4.0f/16.0f, FStyleColors::SelectInactive) );

	Set( "ToggleButtonCheckbox", ToggleButtonStyle );	
	Set( "ToggleButtonIndicatorCheckbox", ToggleButtonIndicatorStyle );	

	Set("WhiteBrush", new FSlateColorBrush(FLinearColor::White));
	Set("Button", Button);
	Set("ComboButton", ComboButton);
	Set("FlatButton.Success", FlatButton);

	Set("PressHintOnly", PressHintOnly);

	Set("TakeRecorder.TakeInput", TakeInput);
	Set("TakeRecorder.TextBox", TextStyle);
	Set("TakeRecorder.EditableTextBox", EditableTextStyle);
	Set("TakeRecorder.RecordButton", RecordButton);
	Set("TakeRecorder.Cockpit.SmallText", FCoreStyle::GetDefaultFontStyle("Bold", 10));
	Set("TakeRecorder.Cockpit.MediumText", FCoreStyle::GetDefaultFontStyle("Bold", 12));
	Set("TakeRecorder.Cockpit.MediumLargeText", FCoreStyle::GetDefaultFontStyle("Bold", 14));
	Set("TakeRecorder.Cockpit.LargeText", FCoreStyle::GetDefaultFontStyle("Bold", 16));
	Set("TakeRecorder.Cockpit.GiantText", FCoreStyle::GetDefaultFontStyle("Bold", 20));

	Set("TakeRecorder.TabIcon", new FSlateImageBrush(RootToContentDir(TEXT("TabIcon_16x.png")), Icon16x16));


	Set("ClassIcon.TakePreset",      new FSlateImageBrush(RootToContentDir(TEXT("TakePreset_16x.png")), Icon16x16));
	Set("ClassThumbnail.TakePreset", new FSlateVectorImageBrush(RootToContentDir(TEXT("AssetIcons/TakePreset_64.svg")), Icon64x64));

	Set("ClassIcon.TakeRecorderActorSource", new FSlateImageBrush(RootToContentDir(TEXT("TakeRecorderActorSource_16x.png")), Icon16x16));
	Set("ClassThumbnail.TakeRecorderActorSource", new FSlateVectorImageBrush(RootToContentDir(TEXT("AssetIcons/TakeRecorderActorSource_64.svg")), Icon64x64));

	Set("ClassIcon.TakeRecorderCameraCutSource", new FSlateImageBrush(RootToContentDir(TEXT("TakeRecorderCameraCutSource_16x.png")), Icon16x16));
	Set("ClassThumbnail.TakeRecorderCameraCutSource", new FSlateImageBrush(RootToContentDir(TEXT("TakeRecorderCameraCutSource_64x.png")), Icon64x64));

	Set("ClassIcon.TakeRecorderLevelSequenceSource", new FSlateVectorImageBrush(RootToContentDir(TEXT("AssetIcons/TakeRecorderLevelSequenceSource_16.svg")), Icon16x16));
	Set("ClassThumbnail.TakeRecorderLevelSequenceSource", new FSlateVectorImageBrush(RootToContentDir(TEXT("AssetIcons/TakeRecorderLevelSequenceSource_64.svg")), Icon64x64));

	Set("ClassIcon.TakeRecorderLiveLinkSource", new FSlateVectorImageBrush(RootToContentDir(TEXT("AssetIcons/TakeRecorderLiveLinkSource_64.svg")), Icon16x16));
	Set("ClassThumbnail.TakeRecorderLiveLinkSource", new FSlateVectorImageBrush(RootToContentDir(TEXT("AssetIcons/TakeRecorderLiveLinkSource_64.svg")), Icon64x64));

	Set("ClassIcon.TakeRecorderMicrophoneAudioSource", new FSlateImageBrush(RootToContentDir(TEXT("TakeRecorderMicrophoneAudioSource_16x.png")), Icon16x16));
	Set("ClassThumbnail.TakeRecorderMicrophoneAudioSource", new FSlateVectorImageBrush(RootToContentDir(TEXT("AssetIcons/TakeRecorderMicrophoneAudioSource_64.svg")), Icon64x64));

	Set("ClassIcon.TakeRecorderLevelVisibilitySource", new FSlateImageBrush(RootToContentDir(TEXT("TakeRecorderLevelVisibilitySource_16x.png")), Icon16x16));
	Set("ClassThumbnail.TakeRecorderLevelVisibilitySource", new FSlateImageBrush(RootToContentDir(TEXT("TakeRecorderLevelVisibilitySource_64x.png")), Icon64x64));

	Set("ClassIcon.TakeRecorderNearbySpawnedActorSource", new FSlateImageBrush(RootToContentDir(TEXT("TakeRecorderNearbySpawnedActorSource_16x.png")), Icon16x16));
	Set("ClassThumbnail.TakeRecorderNearbySpawnedActorSource", new FSlateVectorImageBrush(RootToContentDir(TEXT("AssetIcons/TakeRecorderNearbySpawnedActorSource_64.svg")), Icon64x64));

	Set("ClassIcon.TakeRecorderPlayerSource", new FSlateImageBrush(RootToContentDir(TEXT("TakeRecorderPlayerSource_16x.png")), Icon16x16));
	Set("ClassThumbnail.TakeRecorderPlayerSource", new FSlateImageBrush(RootToContentDir(TEXT("TakeRecorderPlayerSource_64x.png")), Icon64x64));

	Set("ClassIcon.TakeRecorderWorldSource", new FSlateImageBrush(RootToContentDir(TEXT("TakeRecorderWorldSource_16x.png")), Icon16x16));
	Set("ClassThumbnail.TakeRecorderWorldSource", new FSlateVectorImageBrush(RootToContentDir(TEXT("AssetIcons/TakeRecorderWorldSource_64.svg")), Icon64x64));

	Set("TakeRecorder.TabIcon", new FSlateImageBrush(RootToContentDir(TEXT("TabIcon_16x.png")), Icon16x16));

	Set("TakeRecorder.Source.Label", FTextBlockStyle()
		.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 10)) 
		.SetColorAndOpacity(FSlateColor::UseForeground())
	);

	Set("TakeRecorder.Source.RecordingImage", new FSlateImageBrush(RootToContentDir(TEXT("RecordingIndicator.png")), Icon16x16));
	Set("TakeRecorder.Source.Switch", SwitchStyle);

	Set("TakeRecorder.SavePreset", new FSlateImageBrush(RootToContentDir(TEXT("SavePreset.png")), Icon16x16));
	Set("TakeRecorder.StartNewRecording", new FSlateImageBrush(RootToContentDir(TEXT("StartNewRecording.png")), Icon16x16));
	Set("TakeRecorder.StartNewRecordingButton", new FSlateImageBrush(RootToContentDir(TEXT("StartNewRecording.png")), Icon32x32));
	Set("TakeRecorder.SequencerButton", new FSlateImageBrush(RootToContentDir(TEXT("Sequencer.png")), Icon20x20));
	Set("TakeRecorder.ReviewRecordingButton", new FSlateImageBrush(RootToContentDir(TEXT("ReviewRecording.png")), Icon20x20));
	Set("TakeRecorder.MarkFrame", new FSlateImageBrush(RootToContentDir(TEXT("MarkFrame.png")), Icon20x20));
	Set("TakeRecorder.SequenceToRecordIntoButton", new FSlateVectorImageBrush(RootToContentDir(TEXT("SequenceToRecordInto.svg")), Icon20x20));

	Set("TakeRecorder.Slate", new FSlateColorBrush(MediumGrey));
	Set("TakeRecorder.Slate.ClapperBackground", new FSlateColorBrush(AlmostWhite));
	Set("TakeRecorder.Slate.ClapperForeground", DarkGrey);
	Set("TakeRecorder.Slate.ClapperImage", new FSlateImageBrush(RootToContentDir(TEXT("ClapperHeader.png")), FVector2D(768.0f, 16.0f)) );
	Set("TakeRecorder.Slate.BorderImage", new FSlateBorderBrush(RootToContentDir(TEXT("SlateBorder.png")), FMargin(1/16.f) ) );
	Set("TakeRecorder.Slate.BorderColor", DarkGrey);

	Set("TakeRecorder.TakePresetEditorBorder", new FSlateBoxBrush(RootToContentDir(TEXT("TakePresetEditorBorder.png")), FMargin(4.0f / 16.0f)));
	Set("TakeRecorder.TakeRecorderReviewBorder", new FSlateBoxBrush(RootToContentDir(TEXT("TakeRecorderReviewBorder.png")), FMargin(4.0f / 16.0f)));

	Set("FontAwesome.28", FSlateFontInfo(FAppStyle::Get().GetFontStyle("FontAwesome.16").CompositeFont, 28));

	// Hitch visualization
	const FLinearColor ErrorColor(0.36f, 0.04f, 0.04f);
	const FLinearColor ErrorColor_Hovered(1.00f, 0.05f, 0.05f);
	const FLinearColor WarningColor(0.58f, 0.31f, 0.00f, 0.90f);
	const FLinearColor WarningColor_Hovered(1.00f, 0.38f, 0.00f, 0.90f);
	
	Set("Hitching.SkipMarker.Normal", ErrorColor); 
	Set("Hitching.SkipMarker.Hovered", ErrorColor_Hovered); 
	Set("Hitching.SkipMarker.Icon", new IMAGE_BRUSH_SVG("AlertTriangleSolid", Icon16x16));
	
	Set("Hitching.RepeatMarker.Normal", ErrorColor); 
	Set("Hitching.RepeatMarker.Hovered", ErrorColor_Hovered); 
	Set("Hitching.RepeatMarker.Icon", new IMAGE_BRUSH_SVG("AlertTriangleSolid", Icon16x16));
	
	Set("Hitching.CatchupZone.Normal", WarningColor); 
	Set("Hitching.CatchupZone.Hovered", WarningColor_Hovered); 
	Set("Hitching.CatchupZone.Icon", new FSlateColorBrush(FLinearColor::White)); // The colour does not matter - we'll tint it in the drawing code.

	Set("Hitching.MismatchedFrameRate.Normal", WarningColor); 
	Set("Hitching.MismatchedFrameRate.Hovered", WarningColor_Hovered); 
	Set("Hitching.MismatchedFrameRate.Icon", new IMAGE_BRUSH_SVG("AlertTriangleSolid", Icon16x16));
	Set("Hitching.MismatchedFrameRate.Offset", FVector2f(4.f, 4.f));
	
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FTakeRecorderStyle::~FTakeRecorderStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FTakeRecorderStyle& FTakeRecorderStyle::Get()
{
	static FTakeRecorderStyle Inst;
	return Inst;
}


