// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

/**
 * Slate style set that defines all the styles for the take recorder UI
 */
class FTakeRecorderStyle
	: public FSlateStyleSet
{
public:
	static FName StyleName;

	/** Access the singleton instance for this style set */
	static FTakeRecorderStyle& Get();

private:

	FTakeRecorderStyle();
	~FTakeRecorderStyle();
};
