// Copyright Epic Games, Inc. All Rights Reserved.

#include "HitchProtectionFrameRateModel.h"

#include "Engine/Engine.h"
#include "Engine/TimecodeProvider.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Recorder/TakeRecorderSubsystem.h"
#include "TakeRecorderSettings.h"

#define LOCTEXT_NAMESPACE "HitchProtectionFrameRateModel"

namespace UE::TakeRecorder::HitchProtectionFrameRateModel
{
struct FFrameRateInfo
{
	const FFrameRate TakeRecorderFrameRate = GEngine ? GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>()->GetFrameRate() : FFrameRate(0, 1);
	const TOptional<FFrameRate> TimecodeFrameRate = GEngine && GEngine->GetTimecodeProvider() ? GEngine->GetTimecodeProvider()->GetFrameRate() : TOptional<FFrameRate>();
};
	
bool ShouldShowFrameRateWarning()
{
	const FFrameRateInfo FrameRateInfo;
	return GetMutableDefault<UTakeRecorderProjectSettings>()->HitchProtectionSettings.bEnableHitchProtection
		&& FrameRateInfo.TimecodeFrameRate
		&& FrameRateInfo.TakeRecorderFrameRate != *FrameRateInfo.TimecodeFrameRate;
}

FText GetMismatchedFrameRateWarningTooltipText()
{
	if (!GEngine)
	{
		return FText::GetEmpty();
	}

	const FFrameRateInfo FrameRateInfo;
	const FText TooltipFmt = LOCTEXT(
		"MismatchedFPS",
		"You have selected {0} but the timecode provider is set to {1}."
		"\n\nThis is shown because you have enabled Hitch Protection. "
		"\nHitch analysis is not supported for mismatched FPS."
		"\nHitch protection will still run but there will be no visualization of hitches when you review the recording."
		"\n\nTo fix this, ensure that Take Recorder and your timecode provider have the same FPS."
		);
	return FText::Format(TooltipFmt,
		FrameRateInfo.TakeRecorderFrameRate.ToPrettyText(),
		FrameRateInfo.TimecodeFrameRate.IsSet() ? FrameRateInfo.TimecodeFrameRate->ToPrettyText() : LOCTEXT("Nothing", "nothing")
		);
}
}

#undef LOCTEXT_NAMESPACE