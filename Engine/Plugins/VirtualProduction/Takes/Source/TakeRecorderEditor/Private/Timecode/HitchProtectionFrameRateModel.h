// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FText;

/**
 * This is the "ViewModel" for the warning icon displayed in the cockpit next to the FPS selection.
 * 
 * The warning icon warns the user to configure the same FPS for recording and timecode provider because hitch visualization relies
 * on this (until UE-315784 is implemented).
 */
namespace UE::TakeRecorder::HitchProtectionFrameRateModel
{
/** @return Whether to show the warning icon. */
bool ShouldShowFrameRateWarning();

/** @return The tooltip text to show when the user hovers the warning icon. */
FText GetMismatchedFrameRateWarningTooltipText();
};
