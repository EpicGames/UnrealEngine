// Copyright Epic Games, Inc. All Rights Reserved.

#include "HitchProtectionModel.h"

#include "Engine/Engine.h"

namespace UE::TakeRecorder
{
bool HitchProtectionModel::CanInitializeHitchProtection()
{
	if (!GEngine)
	{
		return false;
	}

	UEngineCustomTimeStep* CustomTimeStep = GEngine->GetCustomTimeStep();
	
	// If a custom time step is set, we cannot run hitch protection because it requires overriding the timestep.
	// We don't want to interrupt a synchronized genlock when the user starts recording.
	return CustomTimeStep == nullptr;
}

FText HitchProtectionModel::GetHitchProtectionDisabledReasonText()
{
	return NSLOCTEXT("TakeRecorder", "HitchProtection.CannotInitializeReason", "To use hitch protection, clear the custom engine timestep first.");
}
}
