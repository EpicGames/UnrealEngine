// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FText;

/**
 * This is a stateless model for hitch protection. 
 */
namespace UE::TakeRecorder::HitchProtectionModel
{
/** @return Whether hitch protection can be initialized. False if there is an incompatible engine timestep set. */
bool CanInitializeHitchProtection(); 

/** @return Returns why hitch protection cannot be initialized */
FText GetHitchProtectionDisabledReasonText();
}
