// Copyright Epic Games, Inc. All Rights Reserved.

#include "HitchlessProtectionRootLogic.h"

#include "HitchProtectionModel.h"
#include "PropertyEditorModule.h"
#include "Recorder/TakeRecorderHitchProtectionParameters.h"
#include "Recorder/TakeRecorderSubsystem.h"
#include "TakeRecorderSettings.h"

namespace UE::TakeRecorder
{
FHitchlessProtectionRootLogic::FHitchlessProtectionRootLogic()
{
	if (IsEngineStartupModuleLoadingComplete())
	{
		SetupTakeRecorderDelegates();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FHitchlessProtectionRootLogic::SetupTakeRecorderDelegates);
	}
}

FHitchlessProtectionRootLogic::~FHitchlessProtectionRootLogic()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	if (UTakeRecorderSubsystem* TakeRecorderSubsystem = GEngine ? GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>() : nullptr)
	{
		TakeRecorderSubsystem->GetOnRecordingInitializedEvent().RemoveAll(this);
		TakeRecorderSubsystem->GetOnRecordingFinishedEvent().RemoveAll(this);
		TakeRecorderSubsystem->GetOnRecordingCancelledEvent().RemoveAll(this);
	}
}

void FHitchlessProtectionRootLogic::SetupTakeRecorderDelegates()
{
	UTakeRecorderSubsystem* TakeRecorderSubsystem = GEngine ? GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>() : nullptr;
	if (ensure(TakeRecorderSubsystem))
	{
		TakeRecorderSubsystem->GetOnRecordingInitializedEvent().AddRaw(this, &FHitchlessProtectionRootLogic::OnInitializeRecording);
		TakeRecorderSubsystem->GetOnRecordingFinishedEvent().AddRaw(this, &FHitchlessProtectionRootLogic::OnFinishRecording);
		TakeRecorderSubsystem->GetOnRecordingCancelledEvent().AddRaw(this, &FHitchlessProtectionRootLogic::OnCancelRecording);
	}
}

void FHitchlessProtectionRootLogic::OnInitializeRecording(UTakeRecorder* TakeRecorder)
{
	const FTakeRecorderHitchProtectionParameters& Settings = GetMutableDefault<UTakeRecorderProjectSettings>()->HitchProtectionSettings;
	if (!Settings.bEnableHitchProtection || !HitchProtectionModel::CanInitializeHitchProtection())
	{
		return;
	}

	const FTimecodeEstimationSetupResult TimecodeSetupResult = TimecodeRegressor.SetupEngineEnvironmentForRecording(Settings);
	if (!TimecodeSetupResult)
	{
		return;
	}
	
	RecordingState.Emplace(TimecodeSetupResult.Estimator, TakeRecorder);
}

void FHitchlessProtectionRootLogic::OnFinishRecording(UTakeRecorder* TakeRecorder)
{
	CleanupRecording();
}

void FHitchlessProtectionRootLogic::OnCancelRecording(UTakeRecorder* TakeRecorder)
{
	CleanupRecording(); 
}

void FHitchlessProtectionRootLogic::CleanupRecording()
{
	RecordingState.Reset();
	TimecodeRegressor.CleanupRecording();
}
}
