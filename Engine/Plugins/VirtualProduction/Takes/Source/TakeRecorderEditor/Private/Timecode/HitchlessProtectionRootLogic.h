// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Recording/FrameHitchDecorationRecorder.h"
#include "Regression/TimecodeRegressionRecordSetup.h"
#include "Templates/UnrealTemplate.h"

namespace UE::TakeRecorder
{
/**
 * Sets up all systems related to managing timecode in Take Recorder and dealing with engine hitches, such as timecode correction during recording and
 * visualization of hitches in Sequencer.
 * 
 * This is the root object class that handles injecting all dependencies into the subsystems. It acts a mediator between the systems ensuring the
 * correct order of operations during recording & reviewing a take.
 */
class FHitchlessProtectionRootLogic : public FNoncopyable
{
public:

	FHitchlessProtectionRootLogic();
	~FHitchlessProtectionRootLogic();

private:

	/**
	 * Overrides the global FApp's timecode by estimating the timecode the frame has using linear regression.
	 * 
	 * E.g. when a 1s hitch occurs at time 14:11:06:01, then the next frame should be 14:11:06:02 even though the real timecode is 14:11:07:02 meaning
	 * that the engine has to catch up. This sets up the relevant systems.
	 */
	FTimecodeRegressionRecordSetup TimecodeRegressor;
	
	struct FRecordingState
	{
		/** Records the underlying timecode provider and the one that is estimated by UTimecodeRegressionProvider. */
		FFrameHitchDecorationRecorder TimecodeRecording;

		explicit FRecordingState(TNotNull<UTimecodeRegressionProvider*> InEstimator, TNotNull<UTakeRecorder*> InTakeRecorder)
			: TimecodeRecording(InEstimator, InTakeRecorder)
		{}
	};
	/** Set while recording. */
	TOptional<FRecordingState> RecordingState;

	/** Starts listening for UTakeRecorder events. */
	void SetupTakeRecorderDelegates();
	/** Sets up hitchless recording. */
	void OnInitializeRecording(UTakeRecorder* TakeRecorder);
	/** Creates timecode tracks requireds for visualization of timecode hitches. */
	void OnFinishRecording(UTakeRecorder* TakeRecorder);
	/** Clears any state */
	void OnCancelRecording(UTakeRecorder* TakeRecorder);

	void CleanupRecording();
};
}

