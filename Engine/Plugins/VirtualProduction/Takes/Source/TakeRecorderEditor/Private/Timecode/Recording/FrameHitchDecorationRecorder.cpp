// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameHitchDecorationRecorder.h"

#include "Engine/TimecodeProvider.h"
#include "Estimation/TimecodeRegressionProvider.h"
#include "Hitching/FrameHitchSceneDecoration.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "Recorder/TakeRecorder.h"
#include "Recorder/TakeRecorderSubsystem.h"
#include "TakeRecorderSources.h"
#include "TakesCoreLog.h"

#define LOCTEXT_NAMESPACE "FTimecodeTrackRecording"

namespace UE::TakeRecorder
{
namespace Private
{
/** Recurse all sections and get the maximum range used. */
static void CombineRange(TOptional<TRange<FFrameNumber>>& InOutRange, TConstArrayView<UMovieSceneTrack*> Tracks)
{
	for (UMovieSceneTrack* Track : Tracks)
	{
		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (Section->HasStartFrame() && Section->HasEndFrame())
			{
				if (!InOutRange.IsSet())
				{
					InOutRange = Section->GetRange();
				}
				else
				{
					InOutRange = TRange<FFrameNumber>::Hull(InOutRange.GetValue(), Section->GetRange());
				}
			}
		}
	}
}

/** Recurse sections and get the maximum range used. */
static TOptional<TRange<FFrameNumber>> GetGlobalFrameRange(const UMovieScene* MovieScene)
{
	TOptional<TRange<FFrameNumber>> FrameRange;

	CombineRange(FrameRange, MovieScene->GetTracks());
	for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
	{
		CombineRange(FrameRange, Binding.GetTracks());
	}

	return FrameRange;
}
}
FFrameHitchDecorationRecorder::FFrameHitchDecorationRecorder(
	TNotNull<UTimecodeRegressionProvider*> InTimecodeEstimator, TNotNull<UTakeRecorder*> InTakeRecorder
	)
	: WeakTimecodeEstimator(InTimecodeEstimator)
	, WeakTakeRecorder(InTakeRecorder)
{
	InTakeRecorder->OnTickRecording().AddRaw(this, &FFrameHitchDecorationRecorder::OnTickRecording);
	InTakeRecorder->OnRecordingStopped().AddRaw(this, &FFrameHitchDecorationRecorder::OnRecordingStopped);
}

FFrameHitchDecorationRecorder::~FFrameHitchDecorationRecorder()
{
	if (UTakeRecorder* TakeRecorder = WeakTakeRecorder.Get())
	{
		TakeRecorder->OnTickRecording().RemoveAll(this);
		TakeRecorder->OnRecordingStopped().RemoveAll(this);
	}
}
	
void FFrameHitchDecorationRecorder::OnTickRecording(UTakeRecorder* InTakeRecorder, const FQualifiedFrameTime& InCurrentFrameTime)
{
	UTimecodeRegressionProvider* TimecodeProvider = WeakTimecodeEstimator.Get();
	
	// Intention: We need to ask the underlying timecode provider what timecode this frame is supposed to have.
	// We CANNOT just call the underlying UTimecodeProvider::GetQualifiedTime because some providers, specifically USystemTimeTimecodeProvider, will resample
	// the timecode and may return a newer timecode. That's problematic as some time may have elapsed since the beginning of the frame and now (so we'd get a different timecode than we got earlier in this frame).
	// So we ask the UTimecodeRegressionProvider what it sampled to make sure we use the timecode that this frame was supposed to have at the beginning of this frame.
	// For the target timecode, we shoul make sure to use exatly the timecode that was sampled at the beginning of this frame... when FApp::CurrentFrameTime was set.
	const TOptional<FQualifiedFrameTime> LastSampled = TimecodeProvider ? TimecodeProvider->GetLastSampledFrameTime() : TOptional<FQualifiedFrameTime>{};
	UE_CLOGF(!LastSampled, LogTakesCore, Warning,
		"No timecode was sampled by the linear regression. Hitch analysis data for frame %ls will use actual frame time instead of underlying target frame time. Hitch visualization will be inaccurate for this frame.",
		*InCurrentFrameTime.ToTimecode().ToString()
		);
	
	TargetTimecodeValues.Emplace(
		InCurrentFrameTime,
		// It's important for analysis, we always add a timecode so TargetTimecodeValues has equal length as UTakeRecorderSources::RecordedTimes.
		// If for whatever reason we cannot determine the "correct" target timecode, we'll just pretend the recorded one is the target.
		LastSampled.Get(InCurrentFrameTime)
		);
}

void FFrameHitchDecorationRecorder::OnRecordingStopped(UTakeRecorder* TakeRecorder) const
{
	using namespace TakesCore;
	ULevelSequence* RootSequence = TakeRecorder->GetSequence();
	UMovieScene* MovieScene = RootSequence->GetMovieScene();

	const TOptional<TRange<FFrameNumber>> GlobalFrameRange = Private::GetGlobalFrameRange(MovieScene);
	if (!GlobalFrameRange)
	{
		return;
	}
	
	UFrameHitchSceneDecoration* Decoration = MovieScene->GetOrCreateDecoration<UFrameHitchSceneDecoration>();
	const ETimecodeExtractionFlags Flags = ETimecodeExtractionFlags::Hours | ETimecodeExtractionFlags::Minutes
		| ETimecodeExtractionFlags::Seconds | ETimecodeExtractionFlags::Frames | ETimecodeExtractionFlags::Times;
	
	FTimecodeExtractionData TargetData = ExtractIntoTimecodeArrays(MovieScene, *GlobalFrameRange, TargetTimecodeValues, Flags);
	FTimecodeExtractionData ActualData = ExtractIntoTimecodeArrays(MovieScene, *GlobalFrameRange, UTakeRecorderSources::RecordedTimes, Flags);
	ensureMsgf(TargetData.Times.Num() == ActualData.Times.Num(),
		TEXT("Number of frames tagged with target timecode is %d but frames with actual timecode is %d. Investigate why they don't coincide."),
		TargetData.Times.Num(), ActualData.Times.Num()
		);
	ensure(TargetData.TCRate == ActualData.TCRate);
	
	Decoration->TargetTimecode.Set(
		MoveTemp(TargetData.Times), MoveTemp(TargetData.Hours), MoveTemp(TargetData.Minutes), MoveTemp(TargetData.Seconds), MoveTemp(TargetData.Frames)
		);
	Decoration->ActualTimecode.Set(
		MoveTemp(ActualData.Times), MoveTemp(ActualData.Hours), MoveTemp(ActualData.Minutes), MoveTemp(ActualData.Seconds), MoveTemp(ActualData.Frames)
		);
	Decoration->TimecodeProviderFrameRate = TargetData.TCRate;
	
	// We don't expect the target frame rate to have been changed since the start of the recording because the UI disables the frame rate from being edited.
	// Should that change in the future, simply set this when recording starts.
	Decoration->RecordFrameRate = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>()->GetFrameRate();
}
}

#undef LOCTEXT_NAMESPACE