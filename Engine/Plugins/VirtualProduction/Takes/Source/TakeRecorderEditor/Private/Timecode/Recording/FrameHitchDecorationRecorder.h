// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderTimeProcessing.h"
#include "Misc/NotNull.h"
#include "Misc/QualifiedFrameTime.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UTimecodeRegressionProvider;
class UTakeRecorder;

namespace UE::TakeRecorder
{
/**
 * Records target and actual per-frame timecode and attached the data as UFrameHitchSceneDecoration to the root UMovieScene.
 * Target timecode: Holds the unestimated timecode of the UTimecodeProvider underlying UTimecodeRegressionProvider.
 * Actual timecode: Holds the timecode of FApp::CurrentFrameTime, which would usually (if the engine is set up) be what UTimecodeRegressionProvider
 * estimates.
 */
class FFrameHitchDecorationRecorder : public FNoncopyable
{
public:
	
	explicit FFrameHitchDecorationRecorder(TNotNull<UTimecodeRegressionProvider*> InTimecodeEstimator, TNotNull<UTakeRecorder*> InTakeRecorder);
	~FFrameHitchDecorationRecorder();

private:

	/** The timecode provider that's doing the estimation of frame times. */
	const TWeakObjectPtr<UTimecodeRegressionProvider> WeakTimecodeEstimator;

	/** Take recorder for which we are recording. */
	const TWeakObjectPtr<UTakeRecorder> WeakTakeRecorder;
	
	/**
	 * Passed in to TakeRecorderSourceHelpers::ProcessRecordedTimes to generate the track for original timecode.
	 * 
	 * We make sure this has equal length as UTakeRecorderSources::RecordedTimes. We achieve this by sampling TargetTimecodeValues whenever
	 * UTakeRecorderSources::RecordedTimes is sampled which is when UTakeRecorder::OnTickRecording is invoked.
	 */
	TakesCore::FArrayOfRecordedTimePairs TargetTimecodeValues;

	/** Called when InTakeRecorder ticks during recording. */
	void OnTickRecording(UTakeRecorder* InTakeRecorder, const FQualifiedFrameTime& InCurrentFrameTime);
	
	/** Creates the tracks and adds them to UTakeRecorder. */
	void OnRecordingStopped(UTakeRecorder* TakeRecorder) const;
};
}


