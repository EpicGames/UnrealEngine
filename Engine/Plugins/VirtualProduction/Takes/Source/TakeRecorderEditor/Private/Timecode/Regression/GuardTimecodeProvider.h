// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Engine.h"
#include "Engine/TimecodeProvider.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/StrongObjectPtrTemplates.h"

namespace UE::TakeRecorder
{
/** Overrides the engine's timecode provider for the lifetime of the instance. */
class FGuardTimecodeProvider
	: public FNoncopyable	// We don't expect any copying
{
	TStrongObjectPtr<UTimecodeProvider> RestoreProvider = nullptr;
public:

	explicit FGuardTimecodeProvider(UTimecodeProvider* Override)
	{
		if (GEngine)
		{
			RestoreProvider.Reset(GEngine->GetTimecodeProvider());
			GEngine->SetTimecodeProvider(Override);
		}
	}

	~FGuardTimecodeProvider()
	{
		if (GEngine)
		{
			GEngine->SetTimecodeProvider(RestoreProvider.Get());
		}
	}
};
}

