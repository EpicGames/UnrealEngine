// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Engine.h"
#include "Engine/EngineCustomTimeStep.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/StrongObjectPtrTemplates.h"

namespace UE::TakeRecorder
{
/** Overrides the engine's timestep for the lifetime of the instance. */
class FGuardTimestep
	: public FNoncopyable	// We don't expect any copying
{
	TStrongObjectPtr<UEngineCustomTimeStep> RestoreTimestep = nullptr;
public:

	explicit FGuardTimestep(UEngineCustomTimeStep* Override)
	{
		if (GEngine)
		{
			RestoreTimestep.Reset(GEngine->GetCustomTimeStep());
			GEngine->SetCustomTimeStep(Override);
		}
	}

	~FGuardTimestep()
	{
		if (GEngine)
		{
			GEngine->SetCustomTimeStep(RestoreTimestep.Get());
		}
	}
};
}

