// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeNotifications.h"

#include "CatchupFixedRateCustomTimeStep.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Dialog/SCustomDialog.h"
#include "Engine/Engine.h"
#include "Estimation/IClockedTimeStep.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Modules/ModuleManager.h"
#include "Recorder/TakeRecorderSubsystem.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FHitchlessTimecodeRecordLogic"

namespace UE::TakeRecorder
{
void ShowNoTimecodeProviderNotification()
{
	if (FSlateApplication::IsInitialized()) // handle headless engine
	{
		FNotificationInfo Notification = LOCTEXT("NoTimecodeProvider.Title", "No timecode provider set-up");
		Notification.SubText = LOCTEXT("NoTimecodeProvider.SubTextFmt", "Hitch protection requires you to set up a timecode provider. Go to Project Settings.");
		Notification.ExpireDuration = 5.f;
		FSlateNotificationManager::Get().AddNotification(Notification)->SetCompletionState(SNotificationItem::CS_Fail);
	}
}

void ShowNoTimeStepClassNotification()
{
	if (FSlateApplication::IsInitialized()) // handle headless engine
	{
		FNotificationInfo Notification = LOCTEXT("NoTimeStepClass.Title", "Failed to create custom time step");
		Notification.SubText = LOCTEXT("NoTimeStepClass.SubTextFmt", "Make sure that a valid custom timestep class is set for Hitch Protection");
		Notification.ExpireDuration = 5.f;
		FSlateNotificationManager::Get().AddNotification(Notification)->SetCompletionState(SNotificationItem::CS_Fail);
	}
}
}

#undef LOCTEXT_NAMESPACE