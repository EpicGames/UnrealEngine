// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UClass;
class UEngineCustomTimeStep;

namespace UE::TakeRecorder
{
/** Shows a notification that the user has not set up any timecode provider. */
void ShowNoTimecodeProviderNotification();

/** Shows a notification that the configured time step class could not be resolved. */
void ShowNoTimeStepClassNotification();
}
