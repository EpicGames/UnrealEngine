// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeRegressionRecordSetup.h"

#include "CatchupFixedRateCustomTimeStep.h"
#include "ClassViewerFilter.h"
#include "Engine/Engine.h"
#include "Estimation/TimecodeRegressionProvider.h"
#include "Recorder/TakeRecorderHitchProtectionParameters.h"
#include "Recorder/TakeRecorderSubsystem.h"
#include "TimecodeNotifications.h"
#include "Timecode/HitchProtectionModel.h"
#include "UObject/Package.h"

namespace UE::TakeRecorder
{
namespace Private
{
/** @return The timecode estimator instance we'll use to estimate timecode - effectively handling any engine hitches. */
static UTimecodeRegressionProvider* CreateTimecodeEstimator(
	UTimecodeProvider* InActualImplementation, const FFrameRate& InRecordFrameRate, float InRegressionFramesInSeconds
	)
{
	UTimecodeRegressionProvider* TimecodeRegression = NewObject<UTimecodeRegressionProvider>(GetTransientPackage());
	TimecodeRegression->SetTimecodeImplementation(InActualImplementation);
	
	// How many frames do we need to sample?
	// E.g. if FrameRate is 60 FPS of 60 frames, then Numerator = 60 meaning 60 / 1 = 60. We want
	InRegressionFramesInSeconds = FMath::Clamp(InRegressionFramesInSeconds, 0.0166, 60);
	TimecodeRegression->NumSampledFrames = InRecordFrameRate.Denominator == 0
		? 1 : InRegressionFramesInSeconds * InRecordFrameRate.Numerator / InRecordFrameRate.Denominator;
	
	return TimecodeRegression;
}
}

FTimecodeEstimationSetupResult FTimecodeRegressionRecordSetup::SetupEngineEnvironmentForRecording(
	const FTakeRecorderHitchProtectionParameters& InParams
	)
{
	FTimecodeEstimationSetupResult Result;
	
	// You are supposed to check this before calling SetupEngineEnvironmentForRecording.
	if (!ensure(HitchProtectionModel::CanInitializeHitchProtection()))
	{
		return Result;
	}
	
	UTimecodeProvider* CurrentTimecodeProvider = GEngine ? GEngine->GetTimecodeProvider() : nullptr;
	if (!CurrentTimecodeProvider)
	{
		ShowNoTimecodeProviderNotification();
		return {};
	}
	const FFrameRate FrameRate = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>()->GetFrameRate();

	UClass* TimestepClass = InParams.CustomTimestep.LoadSynchronous();
	UCatchupFixedRateCustomTimeStep* Timestep = TimestepClass ? NewObject<UCatchupFixedRateCustomTimeStep>(GetTransientPackage(), TimestepClass) : nullptr;
	if (!Timestep)
	{
		ShowNoTimeStepClassNotification();
		return {};
	}
	Timestep->FrameRate = FrameRate;
	Timestep->MaxCatchupSeconds = InParams.MaxCatchupSeconds;
	
	UTimecodeRegressionProvider* Estimator = Private::CreateTimecodeEstimator(
		CurrentTimecodeProvider, FrameRate, InParams.RegressionBufferSizeInSeconds
		);
	RecordData.Emplace(Timestep, Estimator);
	
	Result.Estimator = Estimator;
	return Result;
}
}