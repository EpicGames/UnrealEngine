// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GuardTimecodeProvider.h"
#include "GuardTimestep.h"
#include "Estimation/TimecodeRegressionProvider.h"
#include "Misc/Optional.h"
#include "Recorder/TakeRecorder.h"
#include "Templates/UnrealTemplate.h"

struct FTakeRecorderHitchProtectionParameters;

namespace UE::TakeRecorder
{
/** Holds info about when the engine environment has been set up to estimate timecode. */
struct FTimecodeEstimationSetupResult
{
	/** The timecode estimator that was set up. */
	UTimecodeRegressionProvider* Estimator = nullptr;

	bool IsSuccess() const { return Estimator != nullptr; }
	operator bool() const { return IsSuccess(); }
};
	
/**
 * Sets up the UEngine::TimecodeProvider so timecode is not skipped when an engine frame takes longer than expected ("hitches").
 * This is root class is responsible for setting up this for this workflow and acts as a mediator for setting up relevant systems.
 */
class FTimecodeRegressionRecordSetup : public FNoncopyable
{
public:

	/**
	 * Sets up timecode and optionally the custom time step.
	 * @param InParams Settings for setting up recording.
	 * @return The real timecode provider that we will be estimating. Unset if initialization failed.
	 */
	FTimecodeEstimationSetupResult SetupEngineEnvironmentForRecording(const FTakeRecorderHitchProtectionParameters& InParams);

	/** Clears the time step and timecode provider overrides. */
	void CleanupRecording() { RecordData.Reset(); }

private:

	/** State to save for the duration of a recording. Cleans up engine state after recording. */
	struct FRecordOverrides
	{
		// Member order matters: UEngineCustomTimeStep::Initialize must be run first to give it a chance to set global FApp::CurrentTime. 
		
		/** Set if there was no custom timestep and the user chose one in the pop-up dialogue. */
		TOptional<FGuardTimestep> TimestepOverride;
		/** Set while the recording and if the user did not disable it. */
		FGuardTimecodeProvider TimecodeOverride;

		FRecordOverrides(UTimecodeProvider* InOverride) : TimecodeOverride(InOverride) {}
		FRecordOverrides(UEngineCustomTimeStep* InTimestep, UTimecodeRegressionProvider* InTimecode)
			: TimestepOverride(InPlace, InTimestep), TimecodeOverride(InTimecode) {}
	};
	/** Set while the recording and if the user did not disable this feature. */
	TOptional<FRecordOverrides> RecordData;
};
}

