// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyzedHitchUIDrawingUtils.h"

#include "EHitchDrawFlags.h"
#include "HitchVisualizationStyle.h"
#include "Misc/FrameTime.h"
#include "Rendering/DrawElementTypes.h"
#include "ScrubRangeToScreen.h"
#include "Timecode/Visualization/Drawing/AnalyzedHitchUIHoverInfo.h"
#include "Timecode/Visualization/HitchAnalysis.h"

namespace UE::TakeRecorder::HitchVisualizationUI
{
namespace Private
{
/** Packs args that are used together for drawing geometry in Sequencer's widget overlay areas. */
struct FDrawArgs
{
	/** Converts FFrameTime to Sequencer's widget area. */
	const FScrubRangeToScreen& RangeToScreen;
	/** The geometry area allocated to the overlay */
	const FGeometry& AllottedGeometry;
	/** The base layer ID to draw onto */
	int32 LayerId;

	/** The list to append draw elements to */
	FSlateWindowElementList& OutDrawElements;

	explicit FDrawArgs(const FScrubRangeToScreen& RangeToScreen, const FGeometry& AllottedGeometry, int32 LayerId, FSlateWindowElementList& OutDrawElements)
		: RangeToScreen(RangeToScreen)
		, AllottedGeometry(AllottedGeometry)
		, LayerId(LayerId)
		, OutDrawElements(OutDrawElements)
	{}
};

static FGeometry MakeMarkerIconGeometry(
	const FFrameTime& InTime, const FScrubRangeToScreen& InRangeToScreen, const FGeometry& InAllottedGeometry, const FMarkerStyle& InStyle
	)
{
	const float MarkerPos = InRangeToScreen.FrameToLocalX(InTime);
	const FVector2f Position = FVector2f(MarkerPos, 0) + InStyle.MarkerBrushOffset;
	const FVector2f Size = InStyle.IconBrush->GetImageSize();
	return InAllottedGeometry.MakeChild(Size, FSlateLayoutTransform(Position));
}

/**
 * @return The frame at which to paint the range's right vertical line.
 * 
 * The line should be drawn at the beginning of the first frame after the range. Suppose we had a catchup range consisting of a single frame.
 * Then StartTime and EndTime would be equal ... we wouldn't want to draw the left and right vertical lines at the same spot, i.e. a single line.
 */
static FFrameTime GetCatchupRangeLastVerticalLineFrame(const FCatchupTimeRange& InTimeRange)
{
	return InTimeRange.FirstOkFrame ? *InTimeRange.FirstOkFrame : InTimeRange.EndTime;
}

static FGeometry MakeCatchupZoneGeometry(
	const FCatchupTimeRange& InTimeRange, const FScrubRangeToScreen& InRangeToScreen, const FGeometry& InAllottedGeometry
	)
{
	const float Left = InRangeToScreen.FrameToLocalX(InTimeRange.StartTime);
	const float Right = InRangeToScreen.FrameToLocalX(GetCatchupRangeLastVerticalLineFrame(InTimeRange));
	const FVector2f Position(Left, 0);
	const FVector2f Size(Right - Left, InAllottedGeometry.GetLocalSize().Y);
	return InAllottedGeometry.MakeChild(Size, FSlateLayoutTransform(Position));
}
	
static void DrawVerticalLine(
	const FFrameTime& InFrameTime, const FDrawArgs& InDrawArgs, const FLinearColor& InColor
	)
{
	const float LinePos = InDrawArgs.RangeToScreen.FrameToLocalX(InFrameTime);
	
	TArray<FVector2D> LinePoints;
	LinePoints.AddUninitialized(2);
	LinePoints[0] = FVector2D(LinePos, 0.0f);
	LinePoints[1] = FVector2D(LinePos, FMath::FloorToFloat(InDrawArgs.AllottedGeometry.Size.Y));

	// Avoid painting over the tree view on the left, or the dockable tabs on the right.
	const bool bOutOfBounds = LinePos < 0.f || LinePos >= InDrawArgs.AllottedGeometry.Size.X;
	if (bOutOfBounds)
	{
		return;
	}
	
	FSlateDrawElement::MakeLines(
		InDrawArgs.OutDrawElements,
		InDrawArgs.LayerId,
		InDrawArgs.AllottedGeometry.ToPaintGeometry(),
		LinePoints,
		ESlateDrawEffect::None,
		InColor,
		false,
		1.f
	);
}
	
static void DrawMarkerVerticalLines(
	TConstArrayView<FUnexpectedTimecodeMarker> InMarkers, const TOptional<int32>& InHoveredIndex, const FMarkerStyle& InStyle,
	const FDrawArgs& InDrawArgs
	)
{
	for (int32 Index = 0; Index < InMarkers.Num(); ++Index)
	{
		const FUnexpectedTimecodeMarker& Marker = InMarkers[Index];
		const bool bIsHovered = InHoveredIndex && *InHoveredIndex == Index;
		DrawVerticalLine(Marker.Frame, InDrawArgs, bIsHovered ? InStyle.HoveredTint : InStyle.NormalTint);
	}
}

static void DrawCatchupVerticalLines(
	TConstArrayView<FCatchupTimeRange> InTimes, const TOptional<int32>& InHoveredIndex, const FCatchupZoneStyle& InStyle,
	const FDrawArgs& InDrawArgs
	)
{
	for (int32 Index = 0; Index < InTimes.Num(); ++Index)
	{
		const FCatchupTimeRange& CatchupRange = InTimes[Index];
		const bool bIsHovered = InHoveredIndex && *InHoveredIndex == Index;
		const FLinearColor& InColor = bIsHovered ? InStyle.CatchupZone_Hovered : InStyle.CatchupZone_Normal;
		
		DrawVerticalLine(CatchupRange.StartTime, InDrawArgs, InColor);
		DrawVerticalLine(GetCatchupRangeLastVerticalLineFrame(CatchupRange), InDrawArgs, InColor);
	}
}
	
static void DrawMarkerWarningIcons(
	TConstArrayView<FUnexpectedTimecodeMarker> InMarkers, const TOptional<int32>& InHoveredIndex, const FMarkerStyle& InStyle,
	const FDrawArgs& InDrawArgs
	)
{
	for (int32 Index = 0; Index < InMarkers.Num(); ++Index)
	{
		const FUnexpectedTimecodeMarker& Marker = InMarkers[Index];
		const bool bIsHovered = InHoveredIndex && *InHoveredIndex == Index;
		FSlateDrawElement::MakeBox(
			InDrawArgs.OutDrawElements,
			InDrawArgs.LayerId,
			MakeMarkerIconGeometry(Marker.Frame, InDrawArgs.RangeToScreen, InDrawArgs.AllottedGeometry, InStyle).ToPaintGeometry(),
			InStyle.IconBrush,
			ESlateDrawEffect::None,
			bIsHovered ? InStyle.HoveredTint : InStyle.NormalTint
			);
	}
}

static void DrawCatchupZones(
	TConstArrayView<FCatchupTimeRange> InTimes,  const TOptional<int32>& InHoveredIndex, const FCatchupZoneStyle& InStyle,
	const FDrawArgs& InDrawArgs
	)
{
	for (int32 Index = 0; Index < InTimes.Num(); ++Index)
	{
		const FCatchupTimeRange& CatchupRange = InTimes[Index];
		const bool bIsHovered = InHoveredIndex && *InHoveredIndex == Index;
		FSlateDrawElement::MakeBox(
			InDrawArgs.OutDrawElements,
			InDrawArgs.LayerId,
			MakeCatchupZoneGeometry(CatchupRange, InDrawArgs.RangeToScreen, InDrawArgs.AllottedGeometry).ToPaintGeometry(),
			InStyle.CatchupZoneBrush,
			ESlateDrawEffect::None,
			bIsHovered ? InStyle.CatchupZone_Hovered : InStyle.CatchupZone_Normal
			);
	}
}
}
	
int32 DrawTimeSliderArea(
	const FTimecodeHitchData& InData, const FAnalyzedHitchUIHoverInfo& InHoverInfo, const FScrubRangeToScreen& InRangeToScreen,
	const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InLayerId,
	EHitchDrawFlags InFlags
	)
{
	using namespace Private;
	const FDrawArgs InDrawArgs(InRangeToScreen, InAllottedGeometry, InLayerId, OutDrawElements);
	const FHitchVisualizationStyle Style;
	
	if (EnumHasAnyFlags(InFlags, EHitchDrawFlags::DrawCatchupRanges))
	{
		DrawCatchupVerticalLines(InData.CatchupTimes, InHoverInfo.CatchupTimeIndex, Style.CatchupZoneStyle, InDrawArgs);
		DrawCatchupZones(InData.CatchupTimes, InHoverInfo.CatchupTimeIndex, Style.CatchupZoneStyle, InDrawArgs);
	}

	if (EnumHasAnyFlags(InFlags, EHitchDrawFlags::DrawSkippedTimecodeMarkers))
	{
		DrawMarkerVerticalLines(InData.SkippedTimecodeMarkers, InHoverInfo.SkippedMarkerIndex, Style.SkipMarkerStyle, InDrawArgs);
		DrawMarkerWarningIcons(InData.SkippedTimecodeMarkers, InHoverInfo.SkippedMarkerIndex, Style.SkipMarkerStyle, InDrawArgs);
	}
	
	if (EnumHasAnyFlags(InFlags, EHitchDrawFlags::DrawRepeatedTimecodeMarkers))
	{
		DrawMarkerVerticalLines(InData.RepeatedTimecodeMarkers, InHoverInfo.RepeatedMarkerIndex, Style.RepeatMarkerStyle, InDrawArgs);
		DrawMarkerWarningIcons(InData.RepeatedTimecodeMarkers, InHoverInfo.RepeatedMarkerIndex, Style.RepeatMarkerStyle, InDrawArgs);
	}
	
	return InLayerId + 1;
}

int32 DrawTrackArea(
	const FTimecodeHitchData& InData, const FAnalyzedHitchUIHoverInfo& InHoverInfo, const FScrubRangeToScreen& InRangeToScreen,
	const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InLayerId,
	EHitchDrawFlags InFlags
	)
{
	using namespace Private;
	const FDrawArgs InDrawArgs(InRangeToScreen, InAllottedGeometry, InLayerId, OutDrawElements);
	const FHitchVisualizationStyle Style;
	
	if (EnumHasAnyFlags(InFlags, EHitchDrawFlags::DrawCatchupRanges))
	{
		DrawCatchupVerticalLines(InData.CatchupTimes, InHoverInfo.CatchupTimeIndex, Style.CatchupZoneStyle, InDrawArgs);
	}
	
	if (EnumHasAnyFlags(InFlags, EHitchDrawFlags::DrawSkippedTimecodeMarkers))
	{
		DrawMarkerVerticalLines(InData.SkippedTimecodeMarkers, InHoverInfo.SkippedMarkerIndex, Style.SkipMarkerStyle, InDrawArgs);
	}

	if (EnumHasAnyFlags(InFlags, EHitchDrawFlags::DrawRepeatedTimecodeMarkers))
	{
		DrawMarkerVerticalLines(InData.RepeatedTimecodeMarkers, InHoverInfo.RepeatedMarkerIndex, Style.RepeatMarkerStyle, InDrawArgs);
	}
	
	return InLayerId + 1;
}

namespace Private
{
enum class EHoverCheckFlags : uint8
{
	None,
	Icon = 1 << 0,
	Lines = 1 << 1,
	All = Icon | Lines
};
ENUM_CLASS_FLAGS(EHoverCheckFlags);

static bool IsLineHovered(
	const FFrameTime InTime, const FVector2f& InAbsoluteCursorPos,
	const FScrubRangeToScreen& InRangeToScreen, const FGeometry& InAllottedGeometry
	)
{
	const float LinePosition = InRangeToScreen.FrameToLocalX(InTime);
	
	constexpr float HoverWidth = 2.f;
	const float Left = LinePosition - HoverWidth;
	const float Right = LinePosition + HoverWidth;
	
	const FVector2f Position(Left, 0);
	const FVector2f Size(Right - Left, InAllottedGeometry.GetLocalSize().Y);
	const FGeometry LineGeometry =  InAllottedGeometry.MakeChild(Size, FSlateLayoutTransform(Position));

	return LineGeometry.IsUnderLocation(InAbsoluteCursorPos);
}
	
static TOptional<int32> ComputeHoverState_Markers(
	const TArray<FUnexpectedTimecodeMarker>& InMarkers, const FVector2f& InAbsoluteCursorPos,
	const FScrubRangeToScreen& InRangeToScreen, const FGeometry& InAllottedGeometry, const FMarkerStyle& InStyle,
	EHoverCheckFlags InFlags
	)
{
	for (int32 Index = 0; Index < InMarkers.Num(); ++Index)
	{
		const FFrameTime& FrameTime = InMarkers[Index].Frame;
		
		if (EnumHasAnyFlags(InFlags, EHoverCheckFlags::Icon))
		{
			const FGeometry Geometry = MakeMarkerIconGeometry(FrameTime, InRangeToScreen, InAllottedGeometry, InStyle);
			if (Geometry.IsUnderLocation(InAbsoluteCursorPos))
			{
				return Index;
			}
		}
		
		if (EnumHasAnyFlags(InFlags, EHoverCheckFlags::Lines)
			&& IsLineHovered(FrameTime, InAbsoluteCursorPos, InRangeToScreen, InAllottedGeometry))
		{
			return Index;
		}
	}
	return {};
}

static TOptional<int32> ComputeHoverState_Catchup(
	const TArray<FCatchupTimeRange>& InTimes, const FVector2f& InAbsoluteCursorPos,
	const FScrubRangeToScreen& InRangeToScreen, const FGeometry& InAllottedGeometry,
	EHoverCheckFlags InFlags
)
{
	for (int32 Index = 0; Index < InTimes.Num(); ++Index)
	{
		const FCatchupTimeRange& Zone = InTimes[Index];
		
		if (EnumHasAnyFlags(InFlags, EHoverCheckFlags::Icon))
		{
			const FGeometry Geometry = MakeCatchupZoneGeometry(Zone, InRangeToScreen, InAllottedGeometry);
			if (Geometry.IsUnderLocation(InAbsoluteCursorPos))
			{
				return Index;
			}
		}
		
		if (EnumHasAnyFlags(InFlags, EHoverCheckFlags::Lines)
			&& (IsLineHovered(Zone.StartTime, InAbsoluteCursorPos, InRangeToScreen, InAllottedGeometry)
				|| IsLineHovered(GetCatchupRangeLastVerticalLineFrame(Zone), InAbsoluteCursorPos, InRangeToScreen, InAllottedGeometry)))
		{
			return Index;
		}
	}
	return {};
}

static FAnalyzedHitchUIHoverInfo ComputeHoverState(
	const FVector2f& InAbsoluteCursorPos, const FTimecodeHitchData& InData, const FScrubRangeToScreen& InRangeToScreen,
	const FGeometry& InAllottedGeometry,
	EHitchDrawFlags InFlags, EHoverCheckFlags InAreas
	)
{
	using namespace Private;
	
	FAnalyzedHitchUIHoverInfo HoverInfo;
	const FHitchVisualizationStyle Style;

	if (EnumHasAnyFlags(InFlags, EHitchDrawFlags::DrawSkippedTimecodeMarkers))
	{
		HoverInfo.SkippedMarkerIndex = ComputeHoverState_Markers(
			InData.SkippedTimecodeMarkers, InAbsoluteCursorPos, InRangeToScreen, InAllottedGeometry, Style.SkipMarkerStyle, InAreas
			);
	}
	
	if (EnumHasAnyFlags(InFlags, EHitchDrawFlags::DrawRepeatedTimecodeMarkers)
		// Prefer hovering above items
		&& !HoverInfo.IsHovered())
	{
		HoverInfo.RepeatedMarkerIndex = ComputeHoverState_Markers(
			InData.RepeatedTimecodeMarkers, InAbsoluteCursorPos, InRangeToScreen, InAllottedGeometry, Style.RepeatMarkerStyle, InAreas
			);
	}
	
	if (EnumHasAnyFlags(InFlags, EHitchDrawFlags::DrawCatchupRanges)
		// Prefer hovering above items
		&& !HoverInfo.IsHovered())
	{
		HoverInfo.CatchupTimeIndex = ComputeHoverState_Catchup(
			InData.CatchupTimes, InAbsoluteCursorPos, InRangeToScreen, InAllottedGeometry, InAreas
			);
	}

	return HoverInfo;
}
}

FAnalyzedHitchUIHoverInfo ComputeHoverStateForTimeSliderArea(
	const FVector2f& InAbsoluteCursorPos,
	const FTimecodeHitchData& InData, const FScrubRangeToScreen& InRangeToScreen,
	const FGeometry& InAllottedGeometry, EHitchDrawFlags InFlags
)
{
	const FAnalyzedHitchUIHoverInfo Info = ComputeHoverState(InAbsoluteCursorPos, InData, InRangeToScreen, InAllottedGeometry, InFlags,
		Private::EHoverCheckFlags::All
		);
	return Info;
}
}
