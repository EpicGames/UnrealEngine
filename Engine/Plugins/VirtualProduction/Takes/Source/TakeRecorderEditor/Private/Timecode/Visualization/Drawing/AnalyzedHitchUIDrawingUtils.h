// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EHitchDrawFlags.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"

class FSlateWindowElementList;
namespace UE::TakeRecorder { struct FAnalyzedHitchUIHoverInfo; }
namespace UE::TakeRecorder { struct FScrubRangeToScreen; }
namespace UE::TakeRecorder { struct FTimecodeHitchData; }
struct FGeometry;

namespace UE::TakeRecorder::HitchVisualizationUI
{
/** Draws the top part (markers, etc.) of InData. */
int32 DrawTimeSliderArea(
	const FTimecodeHitchData& InData, const FAnalyzedHitchUIHoverInfo& InHoverInfo, const FScrubRangeToScreen& InRangeToScreen,
	const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InLayerId,
	EHitchDrawFlags InFlags = EHitchDrawFlags::All
	);
	
/** Draws the bottom part (markers, etc.) of InData. */
int32 DrawTrackArea(
	const FTimecodeHitchData& InData, const FAnalyzedHitchUIHoverInfo& InHoverInfo, const FScrubRangeToScreen& InRangeToScreen,
	const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InLayerId,
	EHitchDrawFlags InFlags = EHitchDrawFlags::All
	);
	
/** Determines the hovered geometry in the time slider area. */
FAnalyzedHitchUIHoverInfo ComputeHoverStateForTimeSliderArea(
	const FVector2f& InAbsoluteCursorPos,
	const FTimecodeHitchData& InData, const FScrubRangeToScreen& InRangeToScreen,
	const FGeometry& InAllottedGeometry, EHitchDrawFlags InFlags = EHitchDrawFlags::All
	);
}
