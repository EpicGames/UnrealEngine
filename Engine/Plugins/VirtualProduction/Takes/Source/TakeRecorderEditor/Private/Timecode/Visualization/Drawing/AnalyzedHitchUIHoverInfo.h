// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "HAL/Platform.h"

namespace UE::TakeRecorder
{
/**
 * Information about which hitch visualization geometry is hovered.
 * This is relevant only for when there is hitch analysis data.
 * Only one of the optionals is set at any time.
 */
struct FAnalyzedHitchUIHoverInfo
{
	/** Index to the FTimecodeHitchData::SkippedTimecodeMarkers that should be drawn as set. */
	TOptional<int32> SkippedMarkerIndex;
	
	/** Index to the FTimecodeHitchData::RepeatedTimecodeMarkers that should be drawn as set. */
	TOptional<int32> RepeatedMarkerIndex;

	/** Index to the FTimecodeHitchData::RepeatedTimecodeMarkers that should be drawn as hovered. */
	TOptional<int32> CatchupTimeIndex;
	
	/** Resets this info so nothing is hovered. */
	void Reset()
	{
		SkippedMarkerIndex.Reset();
		RepeatedMarkerIndex.Reset();
		CatchupTimeIndex.Reset();
	}

	/** @return Whether anything is hovered */
	bool IsHovered() const { return SkippedMarkerIndex || RepeatedMarkerIndex || CatchupTimeIndex; }
	/** @return Whether anything is hovered */
	operator bool() const { return IsHovered(); }

	friend bool operator==(const FAnalyzedHitchUIHoverInfo& Left, const FAnalyzedHitchUIHoverInfo& Right)
	{
		return Left.SkippedMarkerIndex == Right.SkippedMarkerIndex
			&& Left.RepeatedMarkerIndex == Right.RepeatedMarkerIndex
			&& Left.CatchupTimeIndex == Right.CatchupTimeIndex;
	}

	friend bool operator!=(const FAnalyzedHitchUIHoverInfo& Left, const FAnalyzedHitchUIHoverInfo& Right)
	{
		return !(Left == Right);
	}
};
}
