// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

namespace UE::TakeRecorder
{
enum class EHitchDrawFlags : uint8
{
	None = 0,

	DrawSkippedTimecodeMarkers = 1 << 0,
	DrawRepeatedTimecodeMarkers = 1 << 1,
	DrawCatchupRanges = 1 << 2,

	All = DrawSkippedTimecodeMarkers | DrawRepeatedTimecodeMarkers | DrawCatchupRanges
};
ENUM_CLASS_FLAGS(EHitchDrawFlags);
}
