// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderStyle.h"
#include "Styling/SlateBrush.h"

namespace UE::TakeRecorder
{
struct FMarkerStyle
{
	/** Color that the marker vertical line & warning icon should have when drawn normally. */
	FLinearColor NormalTint = FTakeRecorderStyle::Get().GetColor("Hitching.SkipMarker.Normal");
	
	/** Color that the marker vertical line & warning icon should have when drawn hovered. */
	FLinearColor HoveredTint = FTakeRecorderStyle::Get().GetColor("Hitching.SkipMarker.Hovered");
	
	/** Brush to drawn next to vertical line on marker. */
	const FSlateBrush* IconBrush = FTakeRecorderStyle::Get().GetBrush("Hitching.SkipMarker.Icon");

	/** Offset that marker icon should have from its vertical line. */
	FVector2f MarkerBrushOffset{ 4.f, 4.f };
};

struct FCatchupZoneStyle
{
	/** Color that the catchup zone vertical lines & box brush should have when drawn normally. */
	FLinearColor CatchupZone_Normal = FTakeRecorderStyle::Get().GetColor("Hitching.CatchupZone.Normal");
	
	/** Color that the catchup zone vertical lines & box brush should have when drawn hovered. */
	FLinearColor CatchupZone_Hovered = FTakeRecorderStyle::Get().GetColor("Hitching.CatchupZone.Hovered");
	
	/** Brush to drawn between left and right vertical line in catchup zone. */
	const FSlateBrush* CatchupZoneBrush = FTakeRecorderStyle::Get().GetBrush("Hitching.CatchupZone.Icon");
};
	
struct FHitchVisualizationStyle
{
	/** How to draw markers for when the engine skipped timecode on a frame. */
	FMarkerStyle SkipMarkerStyle;
	
	/** How to draw markers for when the engine repeated timecode. */
	FMarkerStyle RepeatMarkerStyle
	{
		FTakeRecorderStyle::Get().GetColor("Hitching.SkipMarker.Normal"),
		FTakeRecorderStyle::Get().GetColor("Hitching.SkipMarker.Hovered"),
		FTakeRecorderStyle::Get().GetBrush("Hitching.SkipMarker.Icon")
	};
	
	/** How to draw the catchup zone, i.e. an area in which the engine was trailing target timecode. */
	FCatchupZoneStyle CatchupZoneStyle;
};
}
