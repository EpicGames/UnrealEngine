// Copyright Epic Games, Inc. All Rights Reserved.

#include "MismatchedFrameRateDrawingUtils.h"

#include "ISequencer.h"
#include "MismatchedFrameRateUIHoverInfo.h"
#include "ScrubRangeToScreen.h"
#include "TakeRecorderStyle.h"

namespace UE::TakeRecorder::MismatchedFrameRateUI
{
static FGeometry MakeMarkerIconGeometry(
	const FFrameNumber& InTime, const FScrubRangeToScreen& InRangeToScreen, const FGeometry& InAllottedGeometry, const FSlateBrush* InBrush
)
{
	const float MarkerPos = InRangeToScreen.FrameToLocalX(InTime);
	const FVector2f Position = FVector2f(MarkerPos, 0) + FTakeRecorderStyle::Get().GetVector("Hitching.MismatchedFrameRate.Offset");
	const FVector2f Size = InBrush->GetImageSize();
	return InAllottedGeometry.MakeChild(Size, FSlateLayoutTransform(Position));
}
	
int32 DrawWarningIcon(
	const ISequencer& InSequencer, const FMismatchedFrameRateUIHoverInfo& InHoverInfo, const FScrubRangeToScreen& InRangeToScreen,
	const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InLayerId
)
{
	const FSlateBrush* Brush = FTakeRecorderStyle::Get().GetBrush("Hitching.MismatchedFrameRate.Icon");
	const FLinearColor NormaTint = FTakeRecorderStyle::Get().GetColor("Hitching.MismatchedFrameRate.Normal");
	const FLinearColor HoveredTint = FTakeRecorderStyle::Get().GetColor("Hitching.MismatchedFrameRate.Hovered");
	
	const FFrameNumber StartOfRange = InSequencer.GetPlaybackRange().GetLowerBoundValue();
	const FGeometry IconGeometry = MakeMarkerIconGeometry(StartOfRange, InRangeToScreen, InAllottedGeometry, Brush);
	
	const bool bIsHovered = InHoverInfo.bIsWarningMarkerHovered;
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		InLayerId,
		IconGeometry.ToPaintGeometry(),
		Brush,
		ESlateDrawEffect::None,
		bIsHovered ? HoveredTint : NormaTint
		);
	
	return InLayerId;
}

FMismatchedFrameRateUIHoverInfo ComputeHoverStateForTimeSliderArea(
	const ISequencer& InSequencer, const FVector2f& InAbsoluteCursorPos,
	const FScrubRangeToScreen& InRangeToScreen, const FGeometry& InAllottedGeometry
	)
{
	const FSlateBrush* Brush = FTakeRecorderStyle::Get().GetBrush("Hitching.MismatchedFrameRate.Icon");
	const FFrameNumber StartOfRange = InSequencer.GetPlaybackRange().GetLowerBoundValue();
	const FGeometry IconGeometry = MakeMarkerIconGeometry(StartOfRange, InRangeToScreen, InAllottedGeometry, Brush);
	
	return FMismatchedFrameRateUIHoverInfo{ IconGeometry.IsUnderLocation(InAbsoluteCursorPos) };
}
}
