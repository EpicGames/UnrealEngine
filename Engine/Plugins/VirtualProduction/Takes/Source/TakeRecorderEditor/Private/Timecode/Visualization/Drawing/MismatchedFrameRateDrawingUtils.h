// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/Vector2D.h"

class FSlateWindowElementList;
class ISequencer;
namespace UE::TakeRecorder { struct FMismatchedFrameRateUIHoverInfo; }
namespace UE::TakeRecorder { struct FScrubRangeToScreen; }
struct FGeometry;

namespace UE::TakeRecorder::MismatchedFrameRateUI
{
/** Draws the icon for warning the user about mismatched frame rates. */
int32 DrawWarningIcon(
	const ISequencer& InSequencer, const FMismatchedFrameRateUIHoverInfo& InHoverInfo, const FScrubRangeToScreen& InRangeToScreen,
	const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InLayerId
	);

/** Determines the hovered geometry in the time slider area. */
FMismatchedFrameRateUIHoverInfo ComputeHoverStateForTimeSliderArea(
	const ISequencer& InSequencer, const FVector2f& InAbsoluteCursorPos,
	const FScrubRangeToScreen& InRangeToScreen, const FGeometry& InAllottedGeometry
	);
}
