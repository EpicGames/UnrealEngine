// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "HAL/Platform.h"

namespace UE::TakeRecorder
{
/** This is hover information about the UI that is displayed when no analysis was performed due to mismatched frames. */
struct FMismatchedFrameRateUIHoverInfo
{
	/** Whether the warning marker which tells the user that analysis was skipped is hovered. */
	bool bIsWarningMarkerHovered = false;

	/** Resets this info so nothing is hovered. */
	void Reset()
	{
		bIsWarningMarkerHovered = false;
	}

	/** @return Whether anything is hovered */
	bool IsHovered() const { return bIsWarningMarkerHovered; }
	/** @return Whether anything is hovered */
	operator bool() const { return IsHovered(); }
};
}
