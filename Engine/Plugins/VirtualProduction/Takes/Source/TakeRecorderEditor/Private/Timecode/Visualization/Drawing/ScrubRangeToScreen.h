// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerTimeSliderController.h"
#include "Misc/FrameTime.h"
#include "Misc/FrameRate.h"

namespace UE::TakeRecorder
{
/** Helps convert FFrameTime to coordinates in widget space for finding the X-coordinate of vertical lines in Sequencer. */
struct FScrubRangeToScreen : FSequencerTimeSliderController::FScrubRangeToScreen
{
	const FFrameRate FrameRate;

	explicit FScrubRangeToScreen(const FFrameRate& InFrameRate, const TRange<double>& InViewInput, const FVector2D& InWidgetSize)
		: FSequencerTimeSliderController::FScrubRangeToScreen(InViewInput, InWidgetSize)
		, FrameRate(InFrameRate)
	{}

	/** FrameTime -> local Widget Space */
	float FrameToLocalX(const FFrameTime& InFrameTime) const
	{
		return FrameToLocalX(InFrameTime.FrameNumber);
	}
	
	/** FrameTime -> local Widget Space */
	float FrameToLocalX(const FFrameNumber& InFrameNumber) const
	{
		const double Seconds = InFrameNumber / FrameRate; 
		return InputToLocalX(Seconds);
	}
};
}
