// Copyright Epic Games, Inc. All Rights Reserved.

#include "HitchAnalysis.h"

#include "Editor/Sequencer/Private/SSequencer.h"
#include "Hitching/FrameHitchSceneDecoration.h"
#include "Misc/ScopeExit.h"
#include "MovieScene.h"

namespace UE::TakeRecorder
{
namespace Private
{
static TAutoConsoleVariable<bool> CVarLogFrameMarkers(
	TEXT("TakeRecorder.Hitching.LogFrameMarkers"),
	false,
	TEXT("Whether to log the actual and expected timecode for each frame marker that the hitching visualization shows.")
	);
static TAutoConsoleVariable<bool> CVarLogFrameCatchupRanges(
	TEXT("TakeRecorder.Hitching.LogCatchupRanges"),
	false,
	TEXT("Whether to log start and end of catchup ranges the hitching visualization shows.")
	);
	
template<typename TCallback> requires std::is_invocable_v<TCallback, const FFrameNumber& /*InCurrent*/, const FTimecode& /*InTarget*/, const FTimecode& /*InActual*/>
static void ForEachFrame(UFrameHitchSceneDecoration& InDecoration, const FFrameRate& InRate, TCallback&& InCallback)
{
	const TConstArrayView<FFrameNumber> TargetTimes = InDecoration.TargetTimecode.GetFrameTimes();
	const TConstArrayView<FFrameNumber> ActualTimes = InDecoration.ActualTimecode.GetFrameTimes();
	// If this fires, the data probably was not recorded correctly / messed with. For each frame, we record target and actual time.
	if (!ensureMsgf(TargetTimes.Num() == ActualTimes.Num(), TEXT("Recorded frame times do not match up")))
	{
		return;
	}
	
	for (int32 Index = 0; Index < TargetTimes.Num(); ++Index)
	{
		const FFrameNumber Current = TargetTimes[Index];
		// If this fires, the data probably was not recorded correctly / messed with. For each frame, we record target and actual time.
		if (!ensureMsgf(ActualTimes[Index] == Current, TEXT("Recorded frame times do not match up")))
		{
			continue;
		}
		
		const TOptional<FTimecode> TargetTimecode = InDecoration.TargetTimecode.Evaluate(Current);
		const TOptional<FTimecode> ActualTimecode = InDecoration.ActualTimecode.Evaluate(Current);
		if (ensure(TargetTimecode && ActualTimecode))
		{
			InCallback(Current, *TargetTimecode, *ActualTimecode);
		}
	}
}
	
static FTimecode GetNextTimecode(const FTimecode& InCurrent, int32 InNumTimecodeFrames)
{
	FTimecode Next(InCurrent.Hours, InCurrent.Minutes, InCurrent.Seconds, InCurrent.Frames + 1, InCurrent.Subframe, InCurrent.bDropFrameFormat);
	
	if (Next.Frames >= InNumTimecodeFrames)
	{
		Next.Frames = 0;
		++Next.Seconds;
	}
	
	if (Next.Seconds >= 60)
	{
		Next.Seconds = 0;
		++Next.Minutes;
	}
	
	if (Next.Minutes >= 60)
	{
		Next.Minutes = 0;
		++Next.Hours;
	}
	return Next;
}

struct FAnalyzedMarkers
{
	TArray<FUnexpectedTimecodeMarker> Skipped;
	TArray<FUnexpectedTimecodeMarker> Repeated;
};
	
static FAnalyzedMarkers AnalyzeSkippedTimecodes(
	UFrameHitchSceneDecoration& InDecoration, const FFrameRate& InSequencerRate, int32 InNumTimecodeFrames
	)
{
	FAnalyzedMarkers Result;

	TOptional<FTimecode> LastTimecode;
	ForEachFrame(InDecoration, InSequencerRate,
		[&LastTimecode, InNumTimecodeFrames, &Result](const FFrameNumber& InCurrent, const FTimecode&, const FTimecode& InActual)
		{
			const TOptional<FTimecode> ExpectedTimecode = LastTimecode ? GetNextTimecode(*LastTimecode, InNumTimecodeFrames) : TOptional<FTimecode>{};
			ON_SCOPE_EXIT { LastTimecode = InActual; };
				
			if (!ExpectedTimecode || InActual == *ExpectedTimecode)
			{
				return;
			}

			const bool bIsRepeat = LastTimecode && InActual == *LastTimecode;
			TArray<FUnexpectedTimecodeMarker>& Markers = bIsRepeat ? Result.Repeated : Result.Skipped;
			Markers.Add(FUnexpectedTimecodeMarker(InCurrent, InActual, *ExpectedTimecode));
				
			UE_CLOGF(CVarLogFrameMarkers.GetValueOnAnyThread(), LogTemp, Warning, "%ls timecode at %d. Actual: %ls \tExpected: %ls \tLast: %ls",
				bIsRepeat ? TEXT("Repeated") : TEXT("Skipped"), InCurrent.Value, *InActual.ToString(), *ExpectedTimecode->ToString(),
				LastTimecode ? *LastTimecode->ToString() : TEXT("unset")
				);
		});
	
	return Result;
}

static void DumpCatchupRanges(UFrameHitchSceneDecoration& InDecoration, const TArray<FCatchupTimeRange>& InRanges)
{
	if (!CVarLogFrameCatchupRanges.GetValueOnAnyThread())
	{
		return;
	}
	
	for (const FCatchupTimeRange& Range : InRanges)
	{
		const TOptional<FTimecode> Start_TargetFrameNumber = InDecoration.TargetTimecode.Evaluate(Range.StartTime);
		const TOptional<FTimecode> Start_ActualFrameNumber = InDecoration.ActualTimecode.Evaluate(Range.StartTime);
		const TOptional<FTimecode> End_TargetFrameNumber = InDecoration.TargetTimecode.Evaluate(Range.EndTime);
		const TOptional<FTimecode> End_ActualFrameNumber = InDecoration.ActualTimecode.Evaluate(Range.EndTime);

		if (!ensure(Start_TargetFrameNumber && Start_ActualFrameNumber && End_TargetFrameNumber && End_ActualFrameNumber))
		{
			return;
		}
		
		UE_CLOGF(CVarLogFrameCatchupRanges.GetValueOnAnyThread(), LogTemp, Warning,
			"Catch-up range:\n\tStart at %d\tTarget TC: %ls \tActual TC: %ls\n\tEnd at %d   \tTarget TC: %ls \tActual TC: %ls",
			Range.StartTime.Value, *Start_TargetFrameNumber->ToString(), *Start_ActualFrameNumber->ToString(),
			Range.EndTime.Value, *End_TargetFrameNumber->ToString(), *End_ActualFrameNumber->ToString()
		);
	}
}

static TArray<FCatchupTimeRange> AnalyzeCatchupZones(
	UFrameHitchSceneDecoration& InDecoration, const FFrameRate& InSequencerRate
	)
{
	TArray<FCatchupTimeRange> Result;
	
	TOptional<FCatchupTimeRange> OpenCatchupBracket;
	ForEachFrame(InDecoration, InSequencerRate,
		[&OpenCatchupBracket, &InSequencerRate, &Result](const FFrameNumber& InCurrent, const FTimecode& InTarget, const FTimecode& InActual)
		{
			const FFrameNumber TargetFrameNumber = InTarget.ToFrameNumber(InSequencerRate);
			const FFrameNumber ActualFrameNumber = InActual.ToFrameNumber(InSequencerRate);
			if (!OpenCatchupBracket && ActualFrameNumber < TargetFrameNumber)
			{
				OpenCatchupBracket = FCatchupTimeRange(InCurrent, InCurrent);
			}
			else if (OpenCatchupBracket && ActualFrameNumber < TargetFrameNumber)
			{
				OpenCatchupBracket->EndTime = InCurrent;
			}
			else if (OpenCatchupBracket
				// We don't expect actual to ever be higher than target, but we'll handle that case defensively;
				// UCatchupFixedRateCustomTimeStep should have prevented the engine getting ahead of platform time.
				&& ActualFrameNumber >= TargetFrameNumber) 
			{
				OpenCatchupBracket->FirstOkFrame = InCurrent;
				
				Result.Add(*OpenCatchupBracket);
				OpenCatchupBracket.Reset();
			}
		});

	// There may be a catch-up zone at the end where the engine never caught up
	if (OpenCatchupBracket)
	{
		Result.Add(*OpenCatchupBracket);
	}

	DumpCatchupRanges(InDecoration, Result);
	return Result;
}
}
	
TValueOrError<FTimecodeHitchData, FHitchAnalysisErrorInfo> AnalyseHitches(ISequencer& InSequencer)
{
	UMovieSceneSequence* RootSequence = InSequencer.GetRootMovieSceneSequence();
	UMovieScene* MovieScene = RootSequence->GetMovieScene();
	UFrameHitchSceneDecoration* Decoration = MovieScene->FindDecoration<UFrameHitchSceneDecoration>();
	if (!Decoration)
	{
		return MakeError(FHitchAnalysisErrorInfo{ EHitchAnalysisError::NoData });
	}

	// For now, analysis of mismatching frame rates is not supported. 
	if (Decoration->RecordFrameRate != Decoration->TimecodeProviderFrameRate)
	{
		return MakeError(FHitchAnalysisErrorInfo{
			EHitchAnalysisError::FrameRateMismatch,
			FFrameRateMismatchData{ Decoration->RecordFrameRate, Decoration->TimecodeProviderFrameRate }
		});
	}
	
	const FFrameRate Rate = InSequencer.GetFocusedTickResolution();
	Private::FAnalyzedMarkers Markers = Private::AnalyzeSkippedTimecodes(*Decoration, Rate, Decoration->RecordFrameRate.Numerator);
	return MakeValue(FTimecodeHitchData
	{
		MoveTemp(Markers.Skipped), MoveTemp(Markers.Repeated),
		Private::AnalyzeCatchupZones(*Decoration, Rate),
	});
}

bool HasHitchData(ISequencer& InSequencer)
{
	UMovieSceneSequence* RootSequence = InSequencer.GetRootMovieSceneSequence();
	UMovieScene* MovieScene = RootSequence->GetMovieScene();
	UFrameHitchSceneDecoration* Decoration = MovieScene->FindDecoration<UFrameHitchSceneDecoration>();
	return Decoration != nullptr;
}
}