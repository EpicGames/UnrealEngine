// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "Misc/Timecode.h"

class ISequencer;

namespace UE::TakeRecorder
{
/** A section of time where the engine fell behind and tried to catchup. */
struct FCatchupTimeRange
{
	/** The first frame at which the timecode started being behind */
	FFrameNumber StartTime;
	/** The last frame at which the timecode was still behind */
	FFrameNumber EndTime;

	/**
	 * The frame at which the timecode has caught up again.
	 * That's the frame immediately after EndTime.
	 *
	 * Unset if the recording ended with a frame at which timecode was behind.
	 */
	TOptional<FFrameNumber> FirstOkFrame;

	explicit FCatchupTimeRange(const FFrameNumber& InStartTime, const FFrameNumber& InEndTime, const TOptional<FFrameNumber>& InFirstOkFrame = {})
		: StartTime(InStartTime)
		, EndTime(InEndTime)
		, FirstOkFrame(InFirstOkFrame)
	{}
};

/** Data about a frame at which timecode was skipped */
struct FUnexpectedTimecodeMarker
{
	/** The frame at which the timecode did not match. */
	FFrameNumber Frame;

	/** The actual timecode the frame had. */
	FTimecode ActualTimecode;
	/** The timecode that was expected at this frame. */
	FTimecode ExpectedFrame;

	explicit FUnexpectedTimecodeMarker(const FFrameNumber& InFrame, const FTimecode& InActualTimecode, const FTimecode& InExpectedFrame)
		: Frame(InFrame)
		, ActualTimecode(InActualTimecode)
		, ExpectedFrame(InExpectedFrame)
	{}
};

/** All data required to draw */
struct FTimecodeHitchData
{
	/** Frames at which the timecode was skipped (previous frame was not current frame - 1). */
	TArray<FUnexpectedTimecodeMarker> SkippedTimecodeMarkers;

	/** Frames at which timecode was repeated (previous frame had the same timecode). */
	TArray<FUnexpectedTimecodeMarker> RepeatedTimecodeMarkers;

	/** Sections at which the engine fell behind timecode. */
	TArray<FCatchupTimeRange> CatchupTimes;

	void Reset()
	{
		SkippedTimecodeMarkers.Reset();
		CatchupTimes.Reset();
	}
};

enum class EHitchAnalysisError : uint8
{
	/** There was no UFrameHitchSceneDecoration attached, so we skipped analysis.  */
	NoData,

	/**
	 * Take Recorder's record frame rate did not match the frame rate of underlying timecode provider.
	 * 
	 * For now, analysis is only supported when the frame rates are equal.
	 * 
	 * It's technically possible to implement but we just didn't implement it, yet. To implement, we need to change the timecode that is expected each
	 * frame. For example, if the timecode provider is 48 FPS and TR 24 FPS, example frames we may expect would be
	 * - frame 1 -> 17:42:42:05,
	 * - frame 2 -> 17:42:42:07,
	 * - frame 3 -> 17:42:42:09, etc.
	 */
	FrameRateMismatch,

	Num
};

struct FFrameRateMismatchData
{
	/** Frame rate that TR recorded at. */
	FFrameRate TakeRecorderRate;
	/** Frame rate that the timecode provider had during recording. */
	FFrameRate TimecodeRate;
};
	
struct FHitchAnalysisErrorInfo
{
	/** The reason why analysis failed. */
	EHitchAnalysisError Reason;

	/** Set if Reason == FrameRateMismatch. */
	TOptional<FFrameRateMismatchData> MismatchInfo;
};

/**
 * Tries to find the required tracks for analysing hitches, and then analyses them.
 * @return Analyzed hitch data if required tracks were found.
 */
TValueOrError<FTimecodeHitchData, FHitchAnalysisErrorInfo> AnalyseHitches(ISequencer& InSequencer);

/** @return Whether there is any hitch data that can be analyzed. */
bool HasHitchData(ISequencer& InSequencer);

}
