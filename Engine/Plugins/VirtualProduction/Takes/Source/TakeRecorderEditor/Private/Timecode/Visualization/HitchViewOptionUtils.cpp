// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "HitchViewOptionUtils.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HitchAnalysis.h"
#include "Hitching/FrameHitchSceneDecoration.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "ScopedTransaction.h"
#include "SequencerToolMenuContext.h"
#include "TakeRecorderCommands.h"
#include "TakeRecorderHitchVisualizationSettings.h"
#include "Templates/SharedPointer.h"

#define LOCTEXT_NAMESPACE "FHitchVisualizerInSequencer"

namespace UE::TakeRecorder
{
namespace MenuExtensionDetail
{
static FText GetRecordingIntegrityText()
{
	return LOCTEXT("RecordingIntegrity", "Recording Integrity");
}
}
	
void ExtendSequencerViewOptionsWithHitchVisualization()
{
	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu("Sequencer.MainToolBar.ViewOptions");
	ToolMenu->AddDynamicSection("HitchVisualization", FNewToolMenuDelegate::CreateLambda([](UToolMenu* ToolMenu)
	{
		USequencerToolMenuContext* Context = ToolMenu->Context.FindContext<USequencerToolMenuContext>();
		const TSharedPtr<ISequencer> SequencerPin = ensure(Context) ? Context->WeakSequencer.Pin() : nullptr;
		if (!SequencerPin || !HasHitchData(*SequencerPin))
		{
			return;
		}

		FToolMenuSection& Section = ToolMenu->AddSection(TEXT("HitchVisualization"), MenuExtensionDetail::GetRecordingIntegrityText());
			
		Section.AddMenuEntry(
			"SkipMarkers",
			LOCTEXT("SkipMarkers.Title", "Show Frame Drop Markers"),
			LOCTEXT("SkipMarkers.Description", "Whether to place markers on frames where a timecode frame was skipped."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([]{ UTakeRecorderHitchVisualizationSettings::Get()->ToggleShowFrameDropMarkers(); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([]{ return UTakeRecorderHitchVisualizationSettings::Get()->GetShowFrameDropMarkers(); })
				),
			EUserInterfaceActionType::ToggleButton
		);

		Section.AddMenuEntry(
			"RepeatMarkers",
			LOCTEXT("RepeatMarkers.Title", "Show Frame Repeat Markers"),
			LOCTEXT("RepeatMarkers.Description", "Whether to place markers on frames where a timecode frame was repeated."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([]{ UTakeRecorderHitchVisualizationSettings::Get()->ToggleShowFrameRepeatMarkers(); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([]{ return UTakeRecorderHitchVisualizationSettings::Get()->GetShowFrameRepeatMarkers(); })
				),
			EUserInterfaceActionType::ToggleButton
		);

		Section.AddMenuEntry(
			"ShowCatchupRanges",
			LOCTEXT("ShowCatchupRanges.Title", "Show Hitch Recovery Ranges"),
			LOCTEXT("ShowCatchupRanges.Description", "Whether to show areas in which the engine could not keep up, i.e. was running behind."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([]{ UTakeRecorderHitchVisualizationSettings::Get()->ToggleShowCatchupRanges(); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([]{ return UTakeRecorderHitchVisualizationSettings::Get()->GetShowCatchupRanges(); })
				),
				EUserInterfaceActionType::ToggleButton
		);

		Section.AddMenuEntry("ClearRecordingIntegrityData", FTakeRecorderCommands::Get().ClearRecordingIntegrityData);
	}), FToolMenuInsert(NAME_None, EToolMenuInsertType::Last));
}

void ExtendSequencerTimeSliderContextMenu()
{
	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu("Sequencer.TimelineMenu");
	ToolMenu->AddDynamicSection("HitchVisualization", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		const USequencerToolMenuContext* SequencerContext = InMenu->FindContext<USequencerToolMenuContext>();
		const TSharedPtr<ISequencer> SequencerPin = SequencerContext ? SequencerContext->WeakSequencer.Pin() : nullptr;
		if (!ensure(SequencerContext))
		{
			return;
		}
		FToolMenuSection& Section = InMenu->AddSection(TEXT("HitchVisualization"), MenuExtensionDetail::GetRecordingIntegrityText());
		if (HasHitchData(*SequencerPin))
		{
			// Just AddMenuEntry does not work. Seems like the command list is not added. FSequencerHitchVisualizer binds to the Sequencer command list.
			Section.AddMenuEntryWithCommandList(FTakeRecorderCommands::Get().ClearRecordingIntegrityData, SequencerPin->GetCommandBindings());
		}
	}), FToolMenuInsert(NAME_None, EToolMenuInsertType::Last));
}

void Execute_ClearRecordingIntegrityData(TWeakPtr<ISequencer> InSequencer)
{
	const TSharedPtr<ISequencer> SequencerPin = InSequencer.Pin();
	UMovieSceneSequence* Sequence = SequencerPin ? SequencerPin->GetRootMovieSceneSequence() : nullptr;
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (MovieScene && !MovieScene->IsReadOnly())
	{
		const FScopedTransaction Transaction(LOCTEXT("ClearData", "Clear Recording Integrity Data"));
		MovieScene->Modify();
		MovieScene->RemoveDecoration<UFrameHitchSceneDecoration>();
	}
}

bool CanExecute_ClearRecordingIntegrityData(TWeakPtr<ISequencer> InSequencer)
{
	const TSharedPtr<ISequencer> SequencerPin = InSequencer.Pin();
	UMovieSceneSequence* Sequence = SequencerPin ? SequencerPin->GetRootMovieSceneSequence() : nullptr;
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	return MovieScene && !MovieScene->IsReadOnly();
}
}

#undef LOCTEXT_NAMESPACE
#endif