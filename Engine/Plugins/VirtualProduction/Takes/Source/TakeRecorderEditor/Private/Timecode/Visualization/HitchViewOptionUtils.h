// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Templates/SharedPointerFwd.h"

class ISequencer;
class FExtender;

namespace UE::TakeRecorder
{
class FHitchViewModel_AnalyzedData;
	
/**
 * Extends Sequencer view options adding:
 * - Show Frame Drop Markers
 * - Show Catchup Ranges
 * - Clear Recording Integrity Markers and Ranges 
 */
void ExtendSequencerViewOptionsWithHitchVisualization();

/**
 * Extends the top-time slider context menu with:
 * - Clear Recording Integrity Markers and Ranges
 */
void ExtendSequencerTimeSliderContextMenu();

/** Handles the FTakeRecorderCommands::Get().ClearRecordingIntegrityData. */
void Execute_ClearRecordingIntegrityData(TWeakPtr<ISequencer> InSequencer);
/** Whether FTakeRecorderCommands::Get().ClearRecordingIntegrityData can be executed, i.e. movie scene is not read-only. */
bool CanExecute_ClearRecordingIntegrityData(TWeakPtr<ISequencer> InSequencer);
}

#endif