// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Framework/Application/SlateApplication.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Widgets/SToolTip.h"

class ISequencer;
class IToolTip;
struct FFrameTime;

namespace UE::TakeRecorder
{
class FHitchViewModel;

/** Responsible for showing and hiding a tooltip based on the currently hovered visualization UI. */
template<typename THoverState>
class THitchTooltipShower : public FNoncopyable
{
public:
	
	/** @return Whether tooltips are allowed to be shown currently. */
	bool CanShowTooltips() const
	{
		FSlateApplication& SlateApplication = FSlateApplication::Get();
		return SlateApplication.GetAllowTooltips() && SlateApplication.GetPressedMouseButtons().IsEmpty();
	}
	
	/** Shows a visualization tooltip if currently hovering */
	void UpdateTooltipState(const THoverState& InHoverInfo, const TAttribute<FText>& InTooltipText)
	{
		if (InHoverInfo.IsHovered() && CanShowTooltips())
		{
			RecreateIfContentChanged(InHoverInfo, InTooltipText);
		}
		else
		{
			TooltipState.Reset();
		}
	}

	/** Ensures the tooltip is hidden. */
	void ClearTooltip()
	{
		TooltipState.Reset();
	}

private:
	
	struct FTooltipState
	{
		/** The hovered UI elements when the tooltip was overriden. */
		const THoverState HoverInfo;

		/** The hitch visualization tooltip we've temporarily spawned. */
		const TSharedPtr<IToolTip> OurTooltip;

		explicit FTooltipState(const THoverState& InHoverInfo, const TAttribute<FText>& InTooltipText)
			: HoverInfo(InHoverInfo)
			, OurTooltip(SNew(SToolTip)
				.Text(InTooltipText)
				.IsInteractive(true) // We need to set the tooltip as interactive as otherwise the tooltip is immediately hidden next frame via widget search
				.OnSetInteractiveWindowLocation_Lambda([](FVector2D& OutPosition){ OutPosition =  FSlateApplication::Get().GetCursorPos(); })
				)
		{
			FSlateApplication& SlateApplication = FSlateApplication::Get();
			SlateApplication.SpawnToolTip(OurTooltip.ToSharedRef(), SlateApplication.GetCursorPos());
		}
		
		~FTooltipState()
		{
			FSlateApplication& SlateApplication = FSlateApplication::Get();
			SlateApplication.CloseToolTip();
		}
	};

	/** Active tooltip override. Cleans up the tooltip on destruction. */
	TOptional<FTooltipState> TooltipState;

	/** (Re-)creates the tooltip if the hover a UI element is hovered. If the hovered UI has changed since last time, the tooltip is recreated. */
	void RecreateIfContentChanged(const THoverState& InHoverInfo, const TAttribute<FText>& InTooltipText)
	{
		const bool bNeedsTooltip = !TooltipState || InHoverInfo != TooltipState->HoverInfo;
		if (bNeedsTooltip)
		{
			TooltipState.Emplace(InHoverInfo, InTooltipText);
		}
	}
};
}
#endif