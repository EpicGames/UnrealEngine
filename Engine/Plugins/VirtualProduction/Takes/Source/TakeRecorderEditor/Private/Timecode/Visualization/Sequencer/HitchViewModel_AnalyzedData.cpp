// Copyright Epic Games, Inc. All Rights Reserved.

#include "HitchViewModel_AnalyzedData.h"

#include "Timecode/Visualization/Drawing/ScrubRangeToScreen.h"
#include "Timecode/Visualization/Drawing/AnalyzedHitchUIDrawingUtils.h"
#include "Timecode/Visualization/TakeRecorderHitchVisualizationSettings.h"
#include "Timecode/Visualization/Drawing/MismatchedFrameRateDrawingUtils.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "FHitchVisualizationViewModel"

namespace UE::TakeRecorder
{
namespace Private
{
static FText MakeHitchVisualizationTooltipText(
	const ISequencer& InSequencer, const FTimecodeHitchData& InHitchData, const FAnalyzedHitchUIHoverInfo& InAnalysisHoverInfo
)
{
	const TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface = InSequencer.GetNumericTypeInterface();
	const auto ConvertFrameToText = [&NumericTypeInterface](const FFrameNumber& InTime)
	{
		const FString AsFrameString = ensure(NumericTypeInterface)
			? NumericTypeInterface->ToString(InTime.Value)
			: FString::FromInt(InTime.Value);
		return FText::FromString(AsFrameString);
	};
	
	if (InAnalysisHoverInfo.RepeatedMarkerIndex && ensure(InHitchData.RepeatedTimecodeMarkers.IsValidIndex(*InAnalysisHoverInfo.RepeatedMarkerIndex)))
	{
		const FText Fmt = LOCTEXT(
			"RepeatedMarkerFmt",
			"Timecode was repeated at frame {0}.\nActual timecode:{1}\nExpected timecode:{2}"
			);
		const FUnexpectedTimecodeMarker& Data = InHitchData.RepeatedTimecodeMarkers[*InAnalysisHoverInfo.RepeatedMarkerIndex];
		return FText::Format(Fmt,
			ConvertFrameToText(Data.Frame),
			FText::FromString(Data.ActualTimecode.ToString()),
			FText::FromString(Data.ExpectedFrame.ToString())
			);
	}
	
	if (InAnalysisHoverInfo.SkippedMarkerIndex && ensure(InHitchData.SkippedTimecodeMarkers.IsValidIndex(*InAnalysisHoverInfo.SkippedMarkerIndex)))
	{
		const FText Fmt = LOCTEXT(
			"SkippedMarkerFmt",
			"A frame drop was detected at frame {0}.\nActual timecode:{1}\nExpected timecode:{2}\n\nData may be missing from record."
			);
		const FUnexpectedTimecodeMarker& Data = InHitchData.SkippedTimecodeMarkers[*InAnalysisHoverInfo.SkippedMarkerIndex];
		return FText::Format(Fmt,
			ConvertFrameToText(Data.Frame),
			FText::FromString(Data.ActualTimecode.ToString()),
			FText::FromString(Data.ExpectedFrame.ToString())
			);
	}
	
	if (InAnalysisHoverInfo.CatchupTimeIndex && ensure(InHitchData.CatchupTimes.IsValidIndex(*InAnalysisHoverInfo.CatchupTimeIndex)))
	{
		const FText Fmt = LOCTEXT(
			"CatchupZoneFmt",
			"Take Recorder was unable to keep up with the target timestep.\n\nData may have been lost or misplotted in this range [{0} - {1}]."
			);
		const FCatchupTimeRange& Data = InHitchData.CatchupTimes[*InAnalysisHoverInfo.CatchupTimeIndex];
		return FText::Format(Fmt, ConvertFrameToText(Data.StartTime), ConvertFrameToText(Data.EndTime));
	}
	return FText::GetEmpty();
}
}
	
UE_SEQUENCER_DEFINE_CASTABLE(FHitchViewModel_AnalyzedData);

int32 FHitchViewModel_AnalyzedData::OnPaintTimeSliderOverlay(
	const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements,
	int32 LayerId, const FWidgetStyle& InWidgetStyle
	)
{
	if (!CanPaint())
	{
		TooltipShowerHack.ClearTooltip();
		return LayerId;
	}

	UTakeRecorderHitchVisualizationSettings* Settings = UTakeRecorderHitchVisualizationSettings::Get();
	UpdateHoverState_TimeSlider(Args, AllottedGeometry, Settings->GetFlagsForUI());
	
	return HitchVisualizationUI::DrawTimeSliderArea(
		ViewModelData, TimeSliderHoverInfo,
		MakeScrubRangeToScreen(AllottedGeometry), AllottedGeometry, OutDrawElements, LayerId, Settings->GetFlagsForUI()
		);
}

int32 FHitchViewModel_AnalyzedData::OnPaintTrackAreaOverlay(
	const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements,
	int32 LayerId, const FWidgetStyle& InWidgetStyle
	)
{
	if (!CanPaint())
	{
		return LayerId;
	}

	UTakeRecorderHitchVisualizationSettings* Settings = UTakeRecorderHitchVisualizationSettings::Get();
	return HitchVisualizationUI::DrawTrackArea(
		ViewModelData, TimeSliderHoverInfo,
		MakeScrubRangeToScreen(AllottedGeometry), AllottedGeometry, OutDrawElements, LayerId, Settings->GetFlagsForUI()
		);
}

	
void FHitchViewModel_AnalyzedData::UpdateHoverState_TimeSlider(
	const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, EHitchDrawFlags InFlags
	)
{
	// InAllottedGeometry is in paint space. Mouse operations, etc. need to be done in tick space.
	// The difference is that in paint space everything is shifted by GetWindowToDesktopTransform... so we need to undo that shift.
	FGeometry TickSpaceGeometry = InAllottedGeometry;
	TickSpaceGeometry.AppendTransform(FSlateLayoutTransform(InArgs.GetWindowToDesktopTransform()));
	
	// Don't draw the UI as hovered while the user is dragging the time slider.
	const bool bCanHover = FSlateApplication::Get().GetPressedMouseButtons().IsEmpty();
	if (bCanHover)
	{
		TimeSliderHoverInfo = HitchVisualizationUI::ComputeHoverStateForTimeSliderArea(
			FSlateApplication::Get().GetCursorPos(), ViewModelData, MakeScrubRangeToScreen(TickSpaceGeometry),
			TickSpaceGeometry, InFlags
		);
	}
	else 
	{
		TimeSliderHoverInfo.Reset();
		
	}

	// As we're manually managing the tooltip state, reevaluate whether it should be shown based off of our new hover state.
	TooltipShowerHack.UpdateTooltipState(
		TimeSliderHoverInfo,
		TAttribute<FText>::CreateLambda([this]
		{
			const TSharedPtr<ISequencer> Sequencer = WeakOwningSequencer.Pin();
			return Private::MakeHitchVisualizationTooltipText(*Sequencer, ViewModelData, TimeSliderHoverInfo);
		}));
}

FScrubRangeToScreen FHitchViewModel_AnalyzedData::MakeScrubRangeToScreen(const FGeometry& AllottedGeometry) const
{
	const TSharedPtr<ISequencer> SequencerPin = WeakOwningSequencer.Pin();
	return FScrubRangeToScreen(SequencerPin->GetFocusedTickResolution(), SequencerPin->GetViewRange(), AllottedGeometry.Size);
}
}

#undef LOCTEXT_NAMESPACE