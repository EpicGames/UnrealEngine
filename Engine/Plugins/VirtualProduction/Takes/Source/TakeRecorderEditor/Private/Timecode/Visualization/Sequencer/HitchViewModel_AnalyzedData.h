// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "HitchTooltipShower.h"
#include "MVVM/Extensions/ITopTimeSliderOverlayExtension.h"
#include "MVVM/Extensions/ITrackAreaOverlayExtension.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "Misc/Attribute.h"
#include "Timecode/Visualization/HitchAnalysis.h"
#include "Timecode/Visualization/TakeRecorderHitchVisualizationSettings.h"
#include "Timecode/Visualization/Drawing/AnalyzedHitchUIHoverInfo.h"

class IToolTip;

namespace UE::TakeRecorder
{
struct FScrubRangeToScreen;
	
/**
 * Draws analyzed hitch data:
 * - Timecode skip & repeats draw a vertical line and a warning icon (in red ),
 * - Catch up zones, when the engine was behind the target timecode (in yellow),
 * - Draws a tooltip when UI in the timeslider section is hovered. The content is also drawn brighter. @see TooltipShowerHack.
 *
 * This is set up by FSequencerHitchVisualizer.
 * @see FSequencerHitchVisualizer
 */
class FHitchViewModel_AnalyzedData
	: public Sequencer::FViewModel
	, public Sequencer::ITopTimeSliderOverlayExtension
	, public Sequencer::ITrackAreaOverlayExtension
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE(FHitchViewModel_AnalyzedData, ITopTimeSliderOverlayExtension, ITrackAreaOverlayExtension);

	explicit FHitchViewModel_AnalyzedData(
		TWeakPtr<ISequencer> InWeakOwningSequencer, FTimecodeHitchData InHitchData, TAttribute<bool> InCanPaintAttr = true
		)
		: WeakOwningSequencer(MoveTemp(InWeakOwningSequencer))
		, CanPaintAttr(MoveTemp(InCanPaintAttr))
		, ViewModelData(MoveTemp(InHitchData))
	{}
	
	//~ Begin ITimeSliderOverlayExtension Interface
	virtual int32 OnPaintTimeSliderOverlay(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle) override;
	//~ End ITimeSliderOverlayExtension Interface
	
	//~ Begin ITrackAreaOverlayExtension Interface
	virtual int32 OnPaintTrackAreaOverlay(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle) override;
	//~ End ITrackAreaOverlayExtension Interface

private:

	/** The sequencer that this model is saved in. */
	const TWeakPtr<ISequencer> WeakOwningSequencer;

	/** Decides whether we can paint. */
	const TAttribute<bool> CanPaintAttr;

	/** The hitch data that is being visualized. */
	const FTimecodeHitchData ViewModelData;
	/** In time slider area, info about which of HitchData should be drawn has hovered. Also used for determining what tooltip to show. */
	FAnalyzedHitchUIHoverInfo TimeSliderHoverInfo;

	/**
	 * Handles showing and hiding the tooltip for our hitch visualization UI.
	 *
	 * This is a "hack" because we should prefer be using Sequencer's hotspot API: ideally, we'd tell the hotspot API what the user is currently
	 * focusing. Then, the API would ask what the tooltip should be and handle showing it. However, the API is limited:
	 * - the hotspot API is limited to track areas but we need it for the time slider area as well, and
	 * - the hotspot API provides not way to show tooltips for now.
	 * Once the hotspot API supports those operations, this hack should be replaced.
	 */
	THitchTooltipShower<FAnalyzedHitchUIHoverInfo> TooltipShowerHack;

	/** Updates TimeSliderHoverInfo and the resulting tooltip state.*/
	void UpdateHoverState_TimeSlider(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, EHitchDrawFlags InFlags);

	/** @return Utility for converting FFrameTime to widget paint positions. */
	FScrubRangeToScreen MakeScrubRangeToScreen(const FGeometry& AllottedGeometry) const;

	/** @return Whether this model should paint itself. */
	bool CanPaint() const { return CanPaintAttr.Get(); }
};
}
#endif