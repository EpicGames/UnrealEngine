// Copyright Epic Games, Inc. All Rights Reserved.

#include "HitchViewModel_MismatchedFrameRate.h"

#include "Timecode/Visualization/Drawing/ScrubRangeToScreen.h"
#include "Timecode/Visualization/Drawing/AnalyzedHitchUIDrawingUtils.h"
#include "Timecode/Visualization/TakeRecorderHitchVisualizationSettings.h"
#include "Timecode/Visualization/Drawing/MismatchedFrameRateDrawingUtils.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "FHitchVisualizationViewModel"

namespace UE::TakeRecorder
{
namespace Private
{
static FText MakeHitchVisualizationTooltipText(const FFrameRateMismatchData& InHitchViewModel)
{
	const FText FormatTxt = LOCTEXT("SkippedAnalysis", 
		"Hitch analysis was skipped.\n\nThe data was recorded at {0} but the timecode provider was at {1}."
		"\nAnalysis of mismatched FPS is currently unsupported."
		"\n\nTo prevent this, when recording, ensure that Take Recorder and your timecode provider have the same FPS."
		);
	return FText::Format(FormatTxt, InHitchViewModel.TakeRecorderRate.ToPrettyText(), InHitchViewModel.TimecodeRate.ToPrettyText());
}
}
	
UE_SEQUENCER_DEFINE_CASTABLE(FHitchViewModel_MismatchedFrameRate);

int32 FHitchViewModel_MismatchedFrameRate::OnPaintTimeSliderOverlay(
	const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements,
	int32 LayerId, const FWidgetStyle& InWidgetStyle
	)
{
	if (!CanPaint())
	{
		TooltipShowerHack.ClearTooltip();
		return LayerId;
	}

	UTakeRecorderHitchVisualizationSettings* Settings = UTakeRecorderHitchVisualizationSettings::Get();
	UpdateHoverState_TimeSlider(Args, AllottedGeometry, Settings->GetFlagsForUI());
	
	const TSharedPtr<ISequencer> Sequencer = WeakOwningSequencer.Pin();
	return ensure(Sequencer)
		? MismatchedFrameRateUI::DrawWarningIcon(
			*Sequencer, TimeSliderHoverInfo,
			MakeScrubRangeToScreen(AllottedGeometry), AllottedGeometry, OutDrawElements, LayerId
		)
		: LayerId;
}
	
void FHitchViewModel_MismatchedFrameRate::UpdateHoverState_TimeSlider(
	const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, EHitchDrawFlags InFlags
	)
{
	// InAllottedGeometry is in paint space. Mouse operations, etc. need to be done in tick space.
	// The difference is that in paint space everything is shifted by GetWindowToDesktopTransform... so we need to undo that shift.
	FGeometry TickSpaceGeometry = InAllottedGeometry;
	TickSpaceGeometry.AppendTransform(FSlateLayoutTransform(InArgs.GetWindowToDesktopTransform()));
	
	// Don't draw the UI as hovered while the user is dragging the time slider.
	const bool bCanHover = FSlateApplication::Get().GetPressedMouseButtons().IsEmpty();
	if (!bCanHover)
	{
		TimeSliderHoverInfo.Reset();
	}
	else if (const TSharedPtr<ISequencer> Sequencer = WeakOwningSequencer.Pin())
	{
		TimeSliderHoverInfo = MismatchedFrameRateUI::ComputeHoverStateForTimeSliderArea(
			*Sequencer, FSlateApplication::Get().GetCursorPos(), MakeScrubRangeToScreen(TickSpaceGeometry), TickSpaceGeometry
		);
	}

	// As we're manually managing the tooltip state, reevaluate whether it should be shown based off of our new hover state.
	TooltipShowerHack.UpdateTooltipState(
		TimeSliderHoverInfo,
		TAttribute<FText>::CreateLambda([this]
		{
			return Private::MakeHitchVisualizationTooltipText(ViewModelData);
		}));
}

FScrubRangeToScreen FHitchViewModel_MismatchedFrameRate::MakeScrubRangeToScreen(const FGeometry& AllottedGeometry) const
{
	const TSharedPtr<ISequencer> SequencerPin = WeakOwningSequencer.Pin();
	return FScrubRangeToScreen(SequencerPin->GetFocusedTickResolution(), SequencerPin->GetViewRange(), AllottedGeometry.Size);
}
}

#undef LOCTEXT_NAMESPACE