// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "HitchTooltipShower.h"
#include "MVVM/Extensions/ITopTimeSliderOverlayExtension.h"
#include "MVVM/Extensions/ITrackAreaOverlayExtension.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "Misc/Attribute.h"
#include "Timecode/Visualization/Drawing/MismatchedFrameRateUIHoverInfo.h"
#include "Timecode/Visualization/HitchAnalysis.h"
#include "Timecode/Visualization/TakeRecorderHitchVisualizationSettings.h"

class IToolTip;

namespace UE::TakeRecorder
{
struct FScrubRangeToScreen;
	
/**
 * Draws a warning icon in the time slider area.
 *
 * The warning tells the user that hitch analysis did not take place because the frame rate that Take Recorder recorded at was different from that
 * frame rate that the underlying timecode provider sampled at.
 */
class FHitchViewModel_MismatchedFrameRate
	: public Sequencer::FViewModel
	, public Sequencer::ITopTimeSliderOverlayExtension
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE(FHitchViewModel_MismatchedFrameRate, ITopTimeSliderOverlayExtension);

	explicit FHitchViewModel_MismatchedFrameRate(
		TWeakPtr<ISequencer> InWeakOwningSequencer, FFrameRateMismatchData InHitchData, TAttribute<bool> InCanPaintAttr = true
		)
		: WeakOwningSequencer(MoveTemp(InWeakOwningSequencer))
		, CanPaintAttr(MoveTemp(InCanPaintAttr))
		, ViewModelData(MoveTemp(InHitchData))
	{}
	
	//~ Begin ITimeSliderOverlayExtension Interface
	virtual int32 OnPaintTimeSliderOverlay(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle) override;
	//~ End ITimeSliderOverlayExtension Interface

private:

	/** The sequencer that this model is saved in. */
	const TWeakPtr<ISequencer> WeakOwningSequencer;

	/** Decides whether we can paint. */
	const TAttribute<bool> CanPaintAttr;

	/** The hitch data that is being visualized. */
	const FFrameRateMismatchData ViewModelData;
	/** In time slider area, info about which of HitchData should be drawn has hovered. Also used for determining what tooltip to show. */
	FMismatchedFrameRateUIHoverInfo TimeSliderHoverInfo;

	/**
	 * Handles showing and hiding the tooltip for our hitch visualization UI.
	 *
	 * This is a "hack" because we should prefer be using Sequencer's hotspot API: ideally, we'd tell the hotspot API what the user is currently
	 * focusing. Then, the API would ask what the tooltip should be and handle showing it. However, the API is limited:
	 * - the hotspot API is limited to track areas but we need it for the time slider area as well, and
	 * - the hotspot API provides not way to show tooltips for now.
	 * Once the hotspot API supports those operations, this hack should be replaced.
	 */
	THitchTooltipShower<FMismatchedFrameRateUIHoverInfo> TooltipShowerHack;

	/** Updates TimeSliderHoverInfo and the resulting tooltip state.*/
	void UpdateHoverState_TimeSlider(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, EHitchDrawFlags InFlags);

	/** @return Utility for converting FFrameTime to widget paint positions. */
	FScrubRangeToScreen MakeScrubRangeToScreen(const FGeometry& AllottedGeometry) const;

	/** @return Whether this model should paint itself. */
	bool CanPaint() const { return CanPaintAttr.Get(); }
};
}
#endif