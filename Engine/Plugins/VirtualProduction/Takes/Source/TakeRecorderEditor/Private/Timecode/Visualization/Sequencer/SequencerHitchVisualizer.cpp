// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "SequencerHitchVisualizer.h"

#include "HitchViewModel_AnalyzedData.h"
#include "HitchViewModel_MismatchedFrameRate.h"
#include "ISequencerModule.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MovieScene.h"
#include "TakeRecorderCommands.h"
#include "Timecode/Visualization/HitchViewOptionUtils.h"

namespace UE::TakeRecorder
{
namespace Private
{
static void UnmapInjectedCommands(const TArray<TWeakPtr<ISequencer>>& InCustomizedSequencers)
{
	if (!FTakeRecorderCommands::IsRegistered())
	{
		return;	
	}
	for (const TWeakPtr<ISequencer>& WeakSequencer : InCustomizedSequencers)
	{
		const TSharedPtr<ISequencer> SequencerPin = WeakSequencer.Pin();
		if (!SequencerPin)
		{
			continue;
		}
		
		SequencerPin->GetCommandBindings()->UnmapAction(FTakeRecorderCommands::Get().ClearRecordingIntegrityData);
	}
}
}
	
FSequencerHitchVisualizer::FSequencerHitchVisualizer()
{
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	OnSequencerCreatedHandle = SequencerModule.RegisterOnSequencerCreated(
		FOnSequencerCreated::FDelegate::CreateRaw(this, &FSequencerHitchVisualizer::OnSequencerCreated)
		);

	ExtendSequencerViewOptionsWithHitchVisualization();
	ExtendSequencerTimeSliderContextMenu();
}

FSequencerHitchVisualizer::~FSequencerHitchVisualizer()
{
	if (ISequencerModule* SequencerModule = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer"))
	{
		SequencerModule->UnregisterOnSequencerCreated(OnSequencerCreatedHandle);
		Private::UnmapInjectedCommands(CustomizedSequencers);
	}
}

void FSequencerHitchVisualizer::OnSequencerCreated(TSharedRef<ISequencer> InSequencer)
{
	using namespace UE::Sequencer;

	const TValueOrError<FTimecodeHitchData, FHitchAnalysisErrorInfo> AnalysisResult = AnalyseHitches(*InSequencer);
	if (AnalysisResult.HasError() && AnalysisResult.GetError().Reason == EHitchAnalysisError::NoData)
	{
		return;
	}

	// To avoid implementing an undo command for undoing clearing the take data, we'll only draw the UI when the decorator is present.
	const TAttribute<bool> CanPaint = TAttribute<bool>::CreateLambda([WeakSequencer = InSequencer.ToWeakPtr()]
	{
		const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		return Sequencer && HasHitchData(*Sequencer);
	});
	
	FViewModelChildren Panels = InSequencer->GetViewModel()->GetEditorPanels();
	if (AnalysisResult.HasValue())
	{
		Panels.AddChild(MakeShared<FHitchViewModel_AnalyzedData>(InSequencer, AnalysisResult.GetValue(), CanPaint));
	}
	else
	{
		Panels.AddChild(MakeShared<FHitchViewModel_MismatchedFrameRate>(InSequencer, *AnalysisResult.GetError().MismatchInfo, CanPaint));
	}

	InSequencer->GetCommandBindings()->MapAction(
		FTakeRecorderCommands::Get().ClearRecordingIntegrityData,
		FExecuteAction::CreateStatic(&TakeRecorder::Execute_ClearRecordingIntegrityData, InSequencer.ToWeakPtr()),
		FCanExecuteAction::CreateStatic(&TakeRecorder::CanExecute_ClearRecordingIntegrityData, InSequencer.ToWeakPtr())
		);
}
}

#endif