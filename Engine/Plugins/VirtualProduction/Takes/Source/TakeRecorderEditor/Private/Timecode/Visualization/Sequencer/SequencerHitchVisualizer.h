// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#if WITH_EDITOR
class FExtender;
class ISequencer;

namespace UE::TakeRecorder
{
class FHitchViewModel_AnalyzedData;

/** Hooks into Sequencer and extends the UI to visualize hitches. */
class FSequencerHitchVisualizer : public FNoncopyable
{
public:
	
	FSequencerHitchVisualizer();
	~FSequencerHitchVisualizer();

private:

	/** Handles for OnSequencerCreated. */
	FDelegateHandle OnSequencerCreatedHandle;
	
	/** Extends Sequencer's view options */
	const TSharedPtr<FExtender> ViewOptionsExtender;

	/** The sequencers we have customized thus far. */
	TArray<TWeakPtr<ISequencer>> CustomizedSequencers;

	/** Invoked when a sequencer is created: extends the UI with visualization UI, if needed. */
	void OnSequencerCreated(TSharedRef<ISequencer> InSequencer);
};
}
#endif


