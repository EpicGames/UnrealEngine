// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderHitchVisualizationSettings.h"

#include "Drawing/EHitchDrawFlags.h"

bool UTakeRecorderHitchVisualizationSettings::GetShowFrameDropMarkers() const
{
	return bShowFrameDropMarkers;
}
void UTakeRecorderHitchVisualizationSettings::ToggleShowFrameDropMarkers()
{
	bShowFrameDropMarkers = !bShowFrameDropMarkers;
	SaveConfig();
}

bool UTakeRecorderHitchVisualizationSettings::GetShowFrameRepeatMarkers() const
{
	return bShowFrameRepeatMarkers;
}

void UTakeRecorderHitchVisualizationSettings::ToggleShowFrameRepeatMarkers()
{
	bShowFrameRepeatMarkers = !bShowFrameRepeatMarkers;
	SaveConfig();
}

bool UTakeRecorderHitchVisualizationSettings::GetShowCatchupRanges() const
{
	return bShowCatchupRanges;
}
void UTakeRecorderHitchVisualizationSettings::ToggleShowCatchupRanges()
{
	bShowCatchupRanges = !bShowCatchupRanges;
	SaveConfig();
}

UE::TakeRecorder::EHitchDrawFlags UTakeRecorderHitchVisualizationSettings::GetFlagsForUI() const
{
	using namespace UE::TakeRecorder;
	EHitchDrawFlags Flags = EHitchDrawFlags::None;

	if (bShowCatchupRanges)
	{
		Flags |= EHitchDrawFlags::DrawCatchupRanges;
	}

	if (bShowFrameDropMarkers)
	{
		Flags |= EHitchDrawFlags::DrawSkippedTimecodeMarkers;
	}

	if (bShowFrameRepeatMarkers)
	{
		Flags |= EHitchDrawFlags::DrawRepeatedTimecodeMarkers;
	}

	return Flags;
}

void UTakeRecorderHitchVisualizationSettings::PostInitProperties()
{
	Super::PostInitProperties();
	LoadConfig();
}

#if WITH_EDITOR
void UTakeRecorderHitchVisualizationSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		// Dragging spinboxes causes this to be called every frame so we wait until they've finished dragging before saving.
		SaveConfig();
	}
}
#endif