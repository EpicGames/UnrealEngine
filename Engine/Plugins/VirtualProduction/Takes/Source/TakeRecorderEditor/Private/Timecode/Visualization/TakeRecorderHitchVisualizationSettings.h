// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "TakeRecorderHitchVisualizationSettings.generated.h"

namespace UE::TakeRecorder { enum class EHitchDrawFlags : uint8; }

UCLASS(Config = EditorSettings)
class UTakeRecorderHitchVisualizationSettings : public UObject
{
	GENERATED_BODY()
public:

	static UTakeRecorderHitchVisualizationSettings* Get()
	{
		return GetMutableDefault<UTakeRecorderHitchVisualizationSettings>();
	}

	bool GetShowFrameDropMarkers() const;
	void ToggleShowFrameDropMarkers();

	bool GetShowFrameRepeatMarkers() const;
	void ToggleShowFrameRepeatMarkers();

	bool GetShowCatchupRanges() const;
	void ToggleShowCatchupRanges();

	UE::TakeRecorder::EHitchDrawFlags GetFlagsForUI() const;

	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
private:
	
	/** Whether to place markers on frames where a timecode frame was skipped. */
	UPROPERTY(Config)
	bool bShowFrameDropMarkers = true;

	/** Whether to place markers on frames where a timecode frame was repeated. */
	UPROPERTY(Config)
	bool bShowFrameRepeatMarkers = true;

	/** Whether to show areas in which the engine could not keep up, i.e. was running behind. */
	UPROPERTY(Config)
	bool bShowCatchupRanges = false;
};