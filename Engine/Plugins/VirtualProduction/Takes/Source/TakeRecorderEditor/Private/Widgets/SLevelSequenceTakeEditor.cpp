// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SLevelSequenceTakeEditor.h"
#include "Customization/AudioInputChannelPropertyCustomization.h"
#include "Customization/TakeRecorderAudioSettingsCustomization.h"
#include "Customization/TakePresetRecorderCustomization.h"
#include "Customization/RecordedPropertyCustomization.h"
#include "Customization/RecorderPropertyMapCustomization.h"
#include "Customization/RecorderSourceObjectCustomization.h"
#include "TakePreset.h"
#include "TakeMetaData.h"
#include "TakePresetSettings.h"
#include "TakeRecorderSource.h"
#include "TakeRecorderSourceProperty.h"
#include "TakeRecorderSources.h"
#include "TakeRecorderStyle.h"
#include "Widgets/STakeRecorderSources.h"
#include "ITakeRecorderModule.h"
#include "LevelSequence.h"
#include "ITakeRecorderSourcesManager.h"

// Core includes
#include "Modules/ModuleManager.h"
#include "Algo/Sort.h"
#include "UObject/UObjectIterator.h"
#include "Templates/SubclassOf.h"
#include "ClassIconFinder.h"
#include "Misc/MessageDialog.h"

// AssetRegistry includes
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"

// ContentBrowser includes
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

// AssetTools includes
#include "AssetToolsModule.h"

// Slate includes
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Views/SListView.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/SlateIconFinder.h"
#include "SPositiveActionButton.h"

// Style includes
#include "Styling/AppStyle.h"

// UnrealEd includes
#include "ScopedTransaction.h"

// PropertyEditor includes
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "DetailWidgetRow.h"
#include "Recorder/TakeRecorderSubsystem.h"

#define LOCTEXT_NAMESPACE "SLevelSequenceTakeEditor"


TArray<UClass*> FindRecordingSourceClasses()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TArray<FAssetData> ClassList;

	FARFilter Filter;
	Filter.ClassPaths.Add(UTakeRecorderSource::StaticClass()->GetClassPathName());

	// Include any Blueprint based objects as well, this includes things like Blutilities, UMG, and GameplayAbility objects
	Filter.bRecursiveClasses = true;
	AssetRegistryModule.Get().GetAssets(Filter, ClassList);

	TArray<UClass*> Classes;

	for (const FAssetData& Data : ClassList)
	{
		UClass* Class = Data.GetClass();
		if (Class)
		{
			Classes.Add(Class);
		}
	}

	for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
	{
		if (ClassIterator->IsChildOf(UTakeRecorderSource::StaticClass()) && !ClassIterator->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			Classes.Add(*ClassIterator);
		}
	}

	return Classes;
}

UE_DISABLE_OPTIMIZATION_SHIP
void SLevelSequenceTakeEditor::Construct(const FArguments& InArgs)
{
	bRequestDetailsRefresh = true;
	LevelSequenceAttribute = InArgs._LevelSequence;

	OnDetailsPropertiesChangedEvent = InArgs._OnDetailsPropertiesChanged;
	OnDetailsViewAddedEvent = InArgs._OnDetailsViewAdded;

	DetailsBox = SNew(SScrollBox);
	DetailsBox->SetScrollBarRightClickDragAllowed(true);

	SourcesWidget = SNew(STakeRecorderSources)
	.OnSelectionChanged(this, &SLevelSequenceTakeEditor::OnSourcesSelectionChanged);

	CheckForNewLevelSequence();
	InitializeAudioSettings();

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Vertical)

		+ SSplitter::Slot()
		.Value(.5f)
		[
			SNew(SBorder)
			.Padding(4)
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			[
				SourcesWidget.ToSharedRef()
			]
		]

		+ SSplitter::Slot()
		.Value(.5f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.f, 4.f, 4.f, 2.f)
			[
				SAssignNew(SearchBox, SSearchBox)
				.HintText(LOCTEXT("SearchSettingsHint", "Search Settings"))
				.OnTextChanged(this, &SLevelSequenceTakeEditor::OnSearchTextChanged)
				.OnTextCommitted(this, &SLevelSequenceTakeEditor::OnSearchTextCommitted)
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				DetailsBox.ToSharedRef()
			]
		]
	];
}

TSharedRef<SWidget> SLevelSequenceTakeEditor::MakeAddSourceButton()
{
	return SNew(SPositiveActionButton)
		.OnGetMenuContent(this, &SLevelSequenceTakeEditor::OnGenerateSourcesMenu)
		.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
		.Text(LOCTEXT("AddNewSource_Text", "Source"));
}
UE_ENABLE_OPTIMIZATION_SHIP

void SLevelSequenceTakeEditor::AddExternalSettingsObject(UObject* InObject)
{
	check(InObject);

	ExternalSettingsObjects.AddUnique(InObject);
	bRequestDetailsRefresh = true;
}

bool SLevelSequenceTakeEditor::RemoveExternalSettingsObject(UObject* InObject)
{
	check(InObject);

	int32 NumRemoved = ExternalSettingsObjects.Remove(InObject);
	if (NumRemoved > 0)
	{
		bRequestDetailsRefresh = true;
		return true;
	}

	return false;
}

void SLevelSequenceTakeEditor::CheckForNewLevelSequence()
{
	ULevelSequence* NewLevelSequence = LevelSequenceAttribute.Get();
	if (CachedLevelSequence != NewLevelSequence)
	{
		CachedLevelSequence = NewLevelSequence;

		UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindOrAddSources(NewLevelSequence);

		SourcesWidget->SetSourceObject(Sources);
		bRequestDetailsRefresh = true;
	}
}

void SLevelSequenceTakeEditor::InitializeAudioSettings()
{
	// Enumerate audio devices before building the UI. Note, this can be expensive depending on the hardware
	// attached to the machine, so we wait as late as possible before enumerating.
	if (UTakeRecorderAudioInputSettings* AudioInputSettings = TakeRecorderAudioSettingsUtils::GetTakeRecorderAudioInputSettings())
	{
		AudioInputSettings->EnumerateAudioDevices();
	}
}

void SLevelSequenceTakeEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	CheckForNewLevelSequence();
	if (bRequestDetailsRefresh)
	{
		UpdateDetails();
		bRequestDetailsRefresh = false;
	}
}

TSharedRef<SWidget> SLevelSequenceTakeEditor::OnGenerateSourcesMenu()
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	{
		ULevelSequence*       LevelSequence = LevelSequenceAttribute.Get();
		UTakeRecorderSources* Sources       = ITakeRecorderSourcesManager::GetChecked().FindOrAddSources(LevelSequence);

		if (Sources)
		{
			ITakeRecorderModule& TakeRecorderModule = ITakeRecorderModule::Get();
			TakeRecorderModule.PopulateSourcesMenu(Extender, Sources);
		}
	}

	FMenuBuilder MenuBuilder(true, nullptr, Extender);

	MenuBuilder.BeginSection("Sources", LOCTEXT("SourcesMenuSection", "Available Sources"));
	{
		TArray<UClass*> SourceClasses = FindRecordingSourceClasses();
		Algo::SortBy(SourceClasses, &UClass::GetDisplayNameText, FText::FSortPredicate());

		for (UClass* Class : SourceClasses)
		{

			TSubclassOf<UTakeRecorderSource> SubclassOf = Class;

			UTakeRecorderSource* Default = Class->GetDefaultObject<UTakeRecorderSource>();
			MenuBuilder.AddMenuEntry(
				Default->GetAddSourceDisplayText(),
				Class->GetToolTipText(true),
				FSlateIconFinder::FindIconForClass(Class),
				FUIAction(
					FExecuteAction::CreateSP(this, &SLevelSequenceTakeEditor::AddSourceFromClass, SubclassOf),
					FCanExecuteAction::CreateSP(this, &SLevelSequenceTakeEditor::CanAddSourceFromClass, SubclassOf)
				)
			);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SLevelSequenceTakeEditor::AddSourceFromClass(TSubclassOf<UTakeRecorderSource> SourceClass)
{
	ULevelSequence*       LevelSequence = LevelSequenceAttribute.Get();
	UTakeRecorderSources* Sources       = ITakeRecorderSourcesManager::GetChecked().FindOrAddSources(LevelSequence);

	if (*SourceClass && Sources)
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("AddNewSource", "Add New {0} Source"), SourceClass->GetDisplayNameText()));
		Sources->Modify();

		Sources->AddSource(SourceClass);
	}
}

bool SLevelSequenceTakeEditor::CanAddSourceFromClass(TSubclassOf<UTakeRecorderSource> SourceClass)
{
	ULevelSequence*       LevelSequence = LevelSequenceAttribute.Get();
	UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindOrAddSources(LevelSequence);

	if (*SourceClass && Sources)
	{
		UTakeRecorderSource* Default = SourceClass->GetDefaultObject<UTakeRecorderSource>();
		return Default->CanAddSource(Sources);
	}

	return false;
}

void SLevelSequenceTakeEditor::OnSourcesSelectionChanged(TSharedPtr<ITakeRecorderSourceTreeItem>, ESelectInfo::Type)
{
	bRequestDetailsRefresh = true;
}

void SLevelSequenceTakeEditor::OnSearchTextChanged(const FText& InFilterText)
{
	CurrentSearchText = InFilterText;
	const bool bIsSearchActive = !InFilterText.IsEmpty();

	for (const auto& Pair : ClassToDetailsView)
	{
		if (Pair.Value.IsValid())
		{
			Pair.Value->SetSearchText(InFilterText);

			// Collapse details views that have no matching properties to avoid empty "All results have been filtered" messages
			if (bIsSearchActive && Pair.Value->GetPropertiesInOrderDisplayed().Num() == 0)
			{
				Pair.Value->SetVisibility(EVisibility::Collapsed);
			}
			else
			{
				Pair.Value->SetVisibility(EVisibility::Visible);
			}
		}
	}
}

void SLevelSequenceTakeEditor::OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType)
{
	OnSearchTextChanged(InFilterText);
}

void SLevelSequenceTakeEditor::OnDetailsPropertiesChanged(const FPropertyChangedEvent& InEvent)
{
	OnDetailsPropertiesChangedEvent.ExecuteIfBound(InEvent);
}

bool SLevelSequenceTakeEditor::PromptUserForTargetRecordClassChange(const UClass* NewClass) const
{
	UTakeRecorderSubsystem* Subsystem = GEngine ? GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>() : nullptr;
	const bool bAllowSilently = Subsystem && !Subsystem->HasPendingChanges();
	if (bAllowSilently)
	{
		return true;
	}
	
	const FText WarningMessage(LOCTEXT(
		"Warning_ChangeTargetLevelSequenceClass",
		"Changing the class requires clearing the pending take.\nYour current changes will be discarded.\n\nDo you want to clear the pending take?")
		);
	return FMessageDialog::Open(EAppMsgType::OkCancel, WarningMessage) == EAppReturnType::Ok;
}

void SLevelSequenceTakeEditor::AddDetails(const TPair<const UClass*, TArray<UObject*> >& Pair, TArray<FObjectKey>& PreviousClasses)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bCustomFilterAreaLocation = true;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowScrollBar = false;

	PreviousClasses.Remove(Pair.Key);

	TSharedPtr<IDetailsView> ExistingDetails = ClassToDetailsView.FindRef(Pair.Key);
	if (ExistingDetails.IsValid())
	{
		ExistingDetails->SetObjects(Pair.Value);
	}
	else
	{
		using namespace UE::TakeRecorder;
		TSharedRef<IDetailsView> Details = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

		// Register the custom property layout for all object types to rename the category to the object type
		// @note: this is registered as a base for all objects on the details panel that
		// overrides the category name for *all* properties in the object. This makes property categories irrelevant for recorder sources,
		// And may also interfere with any other detail customizations for sources as a whole if any are added in future (property type customizations will still work fine)
		// We may want to change this in future but it seems like the neatest way to make top level categories have helpful names.
		Details->RegisterInstancedCustomPropertyLayout(UTakeRecorderSource::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda(&MakeShared<FRecorderSourceObjectCustomization>));

		Details->RegisterInstancedCustomPropertyTypeLayout(FTakeRecorderTargetRecordClassProperty::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateLambda([this]
			{
				return MakeShared<FTakePresetRecorderCustomization>(
					FPromptChangeTargetRecordClass::CreateSP(this, &SLevelSequenceTakeEditor::PromptUserForTargetRecordClassChange)
					);
			}));
		Details->RegisterInstancedCustomPropertyTypeLayout(FName(TEXT("AudioInputDeviceChannelProperty")), FOnGetPropertyTypeCustomizationInstance::CreateLambda(&MakeShared<FAudioInputChannelPropertyCustomization >));
		Details->RegisterInstancedCustomPropertyTypeLayout(FName(TEXT("ActorRecorderPropertyMap")), FOnGetPropertyTypeCustomizationInstance::CreateLambda(&MakeShared<FRecorderPropertyMapCustomization >));
		Details->RegisterInstancedCustomPropertyTypeLayout(FName(TEXT("ActorRecordedProperty")), FOnGetPropertyTypeCustomizationInstance::CreateLambda(&MakeShared<FRecordedPropertyCustomization >));
		Details->SetObjects(Pair.Value);

		bool bDetailsCanBeEnabled = true;
		if (ULevelSequence* AttrLevelSequence = LevelSequenceAttribute.IsSet() ? LevelSequenceAttribute.Get() : nullptr)
		{
			if (UTakeMetaData* TakeMetaData = AttrLevelSequence->FindMetaData<UTakeMetaData>())
			{
				const UTakeRecorderSubsystem* Subsystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>();
				const bool bRecordIsIntoSequence = Subsystem != nullptr && Subsystem->GetTakeRecorderMode() == ETakeRecorderMode::RecordIntoSequence;
				bDetailsCanBeEnabled = !TakeMetaData->Recorded() || bRecordIsIntoSequence;
			}
		}
		Details->SetEnabled(bDetailsCanBeEnabled);

		Details->OnFinishedChangingProperties().AddSP(this, &SLevelSequenceTakeEditor::OnDetailsPropertiesChanged);
		OnDetailsViewAddedEvent.ExecuteIfBound(Details);

		if (!CurrentSearchText.IsEmpty())
		{
			Details->SetSearchText(CurrentSearchText);

			if (Details->GetPropertiesInOrderDisplayed().Num() == 0)
			{
				Details->SetVisibility(EVisibility::Collapsed);
			}
		}

		DetailsBox->AddSlot()
			[
				Details
			];

		ClassToDetailsView.Add(Pair.Key, Details);
	}
}

void SLevelSequenceTakeEditor::UpdateDetails()
{
	TMap<const UClass*, TArray<UObject*>> ExternalClassToSources;

	for (TWeakObjectPtr<> WeakExternalObj : ExternalSettingsObjects)
	{
		UObject* Object = WeakExternalObj.Get();
		if (Object)
		{
			ExternalClassToSources.FindOrAdd(Object->GetClass()).Add(Object);
		}
	}

	TMap<const UClass*, TArray<UObject*>> ClassToSources;

	TArray<UTakeRecorderSource*> SelectedSources;
	SourcesWidget->GetSelectedSources(SelectedSources);

	// Create 1 details panel per source class type
	for (UTakeRecorderSource* Source : SelectedSources)
	{
		ClassToSources.FindOrAdd(Source->GetClass()).Add(Source);

		// Each source can provide an array of additional settings objects. This allows sources to dynamically
		// spawn settings that aren't part of the base class but still have them presented in the UI in a way that
		// gets hidden automatically.
		for (UObject* SettingsObject : Source->GetAdditionalSettingsObjects())
		{
			ClassToSources.FindOrAdd(SettingsObject->GetClass()).Add(SettingsObject);
		}
	}

	TArray<FObjectKey> PreviousClasses;
	ClassToDetailsView.GenerateKeyArray(PreviousClasses);

	// Clear all existing details views if there are external settings objects, so that they can be displayed last
	if (ExternalSettingsObjects.Num())
	{
		for (FObjectKey StaleClass : PreviousClasses)
		{
			TSharedPtr<IDetailsView> Details = ClassToDetailsView.FindRef(StaleClass);
			DetailsBox->RemoveSlot(Details.ToSharedRef());
			ClassToDetailsView.Remove(StaleClass);
		}

		PreviousClasses.Empty();
	}

	for (auto& Pair : ClassToSources)
	{
		AddDetails(Pair, PreviousClasses);
	}

	for (auto& Pair : ExternalClassToSources)
	{
		AddDetails(Pair, PreviousClasses);
	}
	
	for (FObjectKey StaleClass : PreviousClasses)
	{
		TSharedPtr<IDetailsView> Details = ClassToDetailsView.FindRef(StaleClass);
		DetailsBox->RemoveSlot(Details.ToSharedRef());
		ClassToDetailsView.Remove(StaleClass);
	}
}


#undef LOCTEXT_NAMESPACE
