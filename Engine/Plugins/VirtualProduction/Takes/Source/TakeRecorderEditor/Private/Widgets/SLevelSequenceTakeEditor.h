// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectKey.h"
#include "PropertyEditorDelegates.h"

struct FAssetData;
struct ITakeRecorderSourceTreeItem;

class SScrollBox;
class SSearchBox;
class ULevelSequence;
class IDetailsView;
class UTakeRecorderSource;
class STakeRecorderSources;

template<class> class TSubclassOf;

/**
 * Widget used by both the take preset asset editor, and take recorder panel that allows editing the take information for an externally provided level sequence
 */
class SLevelSequenceTakeEditor : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnDetailsPropertiesChanged, const FPropertyChangedEvent&);
	DECLARE_DELEGATE_OneParam(FOnDetailsViewAdded, const TWeakPtr<IDetailsView>&);

	SLATE_BEGIN_ARGS(SLevelSequenceTakeEditor)
		: _LevelSequence(nullptr)
		{}

		SLATE_ATTRIBUTE(ULevelSequence*, LevelSequence)
		SLATE_EVENT(FOnDetailsPropertiesChanged, OnDetailsPropertiesChanged)
		SLATE_EVENT(FOnDetailsViewAdded, OnDetailsViewAdded)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/**
	 * Construct a button that can add sources to this widget's preset
	 */
	TSharedRef<SWidget> MakeAddSourceButton();

	/**
	 * Add a new externally controlled settings object to the details UI on this widget
	 */
	void AddExternalSettingsObject(UObject* InObject);

	/**
	 * Removes an externally controlled settings object from the details UI on this widget
	 * @return True if it was removed, false otherwise
	 */
	bool RemoveExternalSettingsObject(UObject* InObject);

private:

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	/**
	 * Check to see whether the level sequence ptr has changed, and propagate that change if necessary
	 */
	void CheckForNewLevelSequence();

	/**
	 * Initialize the audio settings object so that microphone sources can build their input channel menus
	 * based on the current audio device
	 */
	void InitializeAudioSettings();

	void AddDetails(const TPair<const UClass*, TArray<UObject*> >& Pair, TArray<FObjectKey>& PreviousClasses);

	/**
	 * Update the details panel for the current selection
	 */
	void UpdateDetails();

private:

	TSharedRef<SWidget> OnGenerateSourcesMenu();

	void AddSourceFromClass(TSubclassOf<UTakeRecorderSource> SourceClass);
	bool CanAddSourceFromClass(TSubclassOf<UTakeRecorderSource> SourceClass);
	
	void OnSourcesSelectionChanged(TSharedPtr<ITakeRecorderSourceTreeItem>, ESelectInfo::Type);
	/**
	 * When the search text changes, propagate to all details views.
	 */
	void OnSearchTextChanged(const FText& InFilterText);
	void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType);

	/**
	 * When a details box has a property changed.
	 */
	void OnDetailsPropertiesChanged(const FPropertyChangedEvent& InEvent);

	/** Asks the user whether they want to really change UTakePresetSettings::TargetRecordClass. */
	bool PromptUserForTargetRecordClassChange(const UClass* NewClass) const;

private:

	bool bRequestDetailsRefresh;
	TAttribute<ULevelSequence*> LevelSequenceAttribute;
	TWeakObjectPtr<ULevelSequence> CachedLevelSequence;

	TSharedPtr<STakeRecorderSources> SourcesWidget;
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<SScrollBox> DetailsBox;
	FText CurrentSearchText;
	TMap<FObjectKey, TSharedPtr<IDetailsView>> ClassToDetailsView;

	TArray<TWeakObjectPtr<>> ExternalSettingsObjects;

	/** Called when properties in the details panel have changed. */
	FOnDetailsPropertiesChanged OnDetailsPropertiesChangedEvent;
	/** Called when a details view has been added. */
	FOnDetailsViewAdded OnDetailsViewAddedEvent;
};