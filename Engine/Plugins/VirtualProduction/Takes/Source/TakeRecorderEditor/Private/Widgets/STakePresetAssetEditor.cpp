// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STakePresetAssetEditor.h"
#include "Widgets/TakeRecorderWidgetConstants.h"
#include "Widgets/SLevelSequenceTakeEditor.h"
#include "Widgets/STakeRecorderTabContent.h"
#include "ScopedSequencerPanel.h"
#include "TakePresetToolkit.h"
#include "TakePreset.h"
#include "TakeRecorderStyle.h"

// Slate includes
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

// Style includes
#include "Styling/AppStyle.h"

// UnrealEd includes
#include "FileHelpers.h"
#include "Toolkits/ToolkitManager.h"

#define LOCTEXT_NAMESPACE "STakePresetAssetEditor"

STakePresetAssetEditor::~STakePresetAssetEditor()
{
	FToolkitManager::Get().CloseToolkit(Toolkit.ToSharedRef());
	Toolkit.Reset();
}

UE_DISABLE_OPTIMIZATION_SHIP
void STakePresetAssetEditor::Construct(const FArguments& InArgs, TSharedPtr<FTakePresetToolkit> InToolkit, TWeakPtr<STakeRecorderTabContent> OuterTabContent)
{
	WeakTabContent = OuterTabContent;
	Toolkit = InToolkit;

	SequencerPanel = MakeShared<FScopedSequencerPanel>(MakeAttributeSP(this, &STakePresetAssetEditor::GetLevelSequence));

	TSharedRef<SLevelSequenceTakeEditor> LevelSequenceTakeEditor =
		SNew(SLevelSequenceTakeEditor)
		.LevelSequence(this, &STakePresetAssetEditor::GetLevelSequence);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(TakeRecorder::ToolbarPadding)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FTakeRecorderStyle::Get().GetBrush("TakeRecorder.TakePresetEditorBorder"))
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(TakeRecorder::ButtonOffset)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					LevelSequenceTakeEditor->MakeAddSourceButton()
				]

				+ SHorizontalBox::Slot()
				[
					SNew(SSplitter)
				]

				+ SHorizontalBox::Slot()
				.Padding(TakeRecorder::ButtonOffset)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
					.ContentPadding(TakeRecorder::ButtonPadding)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("SavePresetButton", "Save this take preset"))
					.ForegroundColor(FSlateColor::UseForeground())
					.OnClicked(this, &STakePresetAssetEditor::OnSavePreset)
					[
						SNew(SImage)
						.Image(FTakeRecorderStyle::Get().GetBrush("TakeRecorder.SavePreset"))
					]
				]

				+ SHorizontalBox::Slot()
				.Padding(TakeRecorder::ButtonOffset)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
					.ContentPadding(TakeRecorder::ButtonPadding)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("NewRecording", "Start a new recording using this Take Preset as a base"))
					.ForegroundColor(FSlateColor::UseForeground())
					.OnClicked(this, &STakePresetAssetEditor::NewRecordingFromThis)
					[
						SNew(SImage)
						.Image(FTakeRecorderStyle::Get().GetBrush("TakeRecorder.StartNewRecording"))
					]
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SequencerPanel->MakeToggleButton()
				]
			]
		]

		+ SVerticalBox::Slot()
		[
			LevelSequenceTakeEditor
		]
	];
}
UE_ENABLE_OPTIMIZATION_SHIP

ULevelSequence* STakePresetAssetEditor::GetLevelSequence() const
{
	UTakePreset* TakePreset = Toolkit->GetTakePreset();
	return TakePreset ? TakePreset->GetLevelSequence() : nullptr;
}

FReply STakePresetAssetEditor::OnSavePreset()
{
	UTakePreset* TakePreset = Toolkit->GetTakePreset();
	if (TakePreset)
	{
		FEditorFileUtils::PromptForCheckoutAndSave({ TakePreset->GetOutermost() }, false, false);
	}

	return FReply::Handled();
}

FReply STakePresetAssetEditor::NewRecordingFromThis()
{
	UTakePreset* TakePreset = Toolkit->GetTakePreset();
	TSharedPtr<STakeRecorderTabContent> TabContent = WeakTabContent.Pin();

	if (TabContent.IsValid() && TakePreset)
	{
		TabContent->SetupForRecording(TakePreset);
	}


	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
