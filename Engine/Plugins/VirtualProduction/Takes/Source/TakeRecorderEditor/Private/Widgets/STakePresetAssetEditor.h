// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

struct FScopedSequencerPanel;

class ULevelSequence;
class FTakePresetToolkit;
class STakeRecorderTabContent;

/**
 * Outermost widget used for editing UTakePreset assets
 */
class STakePresetAssetEditor : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STakePresetAssetEditor){}
	SLATE_END_ARGS()

	~STakePresetAssetEditor();

	/**
	 * Construct the widget from the asset's toolkit and the owning tab content
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FTakePresetToolkit> InToolkit, TWeakPtr<STakeRecorderTabContent> OuterTabContent);

	ULevelSequence* GetLevelSequence() const;

private:

	FReply OnSavePreset();
	FReply NewRecordingFromThis();

private:

	/** Weak-ptr back to the owning tab content so we can switch out the content for a take recorder panel if necessary */
	TWeakPtr<STakeRecorderTabContent> WeakTabContent;

	/** The asset toolkit for the editing preset asset. This widget keeps the editor alive so that it can be invoked by the asset editor manager if necessary. */
	TSharedPtr<FTakePresetToolkit> Toolkit;

	/** Scoped panel that handles opening and closing the sequencer pane for this preset */
	TSharedPtr<FScopedSequencerPanel> SequencerPanel;
};