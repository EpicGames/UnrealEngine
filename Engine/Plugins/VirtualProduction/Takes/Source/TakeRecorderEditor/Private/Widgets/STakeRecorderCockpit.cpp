// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STakeRecorderCockpit.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/TakeRecorderWidgetConstants.h"
#include "Widgets/STakeRecorderTabContent.h"
#include "TakesCoreBlueprintLibrary.h"
#include "TakeMetaData.h"
#include "TakeRecorderCommands.h"
#include "ITakeRecorderModule.h"
#include "TakeRecorderSettings.h"
#include "TakeRecorderStyle.h"
#include "TakeRecorderSource.h"
#include "TakeRecorderSources.h"
#include "Misc/Timecode.h"
#include "Misc/FrameRate.h"
#include "MovieScene.h"
#include "Recorder/TakeRecorder.h"
#include "Recorder/TakeRecorderBlueprintLibrary.h"
#include "Recorder/TakeRecorderSubsystem.h"
#include "LevelSequence.h"
#include "Algo/Find.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ITakeRecorderSourcesManager.h"

#include "MovieScene.h"
#include "MovieSceneToolsProjectSettings.h"

// AssetRegistry includes
#include "AssetRegistry/AssetRegistryModule.h"

// TimeManagement includes
#include "FrameNumberNumericInterface.h"

// Slate includes
#include "Widgets/SBoxPanel.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SFrameRatePicker.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

// Style includes
#include "Styling/AppStyle.h"
#include "EditorFontGlyphs.h"

// UnrealEd includes
#include "Algo/Compare.h"
#include "ScopedTransaction.h"
#include "Editor/EditorEngine.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "Modules/ModuleManager.h"
#include "LevelEditor.h"
#include "ISettingsModule.h"
#include "Dialog/SMessageDialog.h"
#include "Kismet2/DebuggerCommands.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "PropertyEditorModule.h"
#include "UObject/TextProperty.h"

#include "ISequencer.h"
#include "ILevelSequenceEditorToolkit.h"

// NamingTokens includes
#include "ITakeRecorderNamingTokensModule.h"
#include "NamingTokens.h"
#include "NamingTokensEngineSubsystem.h"
#include "NamingTokensSpecifiers.h"
#include "STakeRecorderPanel.h"
#include "TakeRecorderNamingTokensData.h"
#include "Timecode/HitchProtectionFrameRateModel.h"
#include "Widgets/Layout/SScaleBox.h"

extern UNREALED_API UEditorEngine* GEditor;

#define LOCTEXT_NAMESPACE "STakeRecorderCockpit"

STakeRecorderCockpit::~STakeRecorderCockpit()
{
	FEditorDelegates::PostUndoRedo.Remove((PostUndoRedoDelegateHandle));

	if (!ensure(TransactionIndex == INDEX_NONE))
	{
		GEditor->CancelTransaction(TransactionIndex);
	}
}

UTakeMetaData* STakeRecorderCockpit::GetMetaData() const
{
	return TakeRecorderSubsystem->GetTakeMetaData();
}

UTakeMetaData* STakeRecorderCockpit::GetMetaDataChecked() const
{
	UTakeMetaData* TakeMetaData = GetMetaData();
	check(TakeMetaData);
	return TakeMetaData;
}

struct SNonThrottledButton : SButton
{
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		FReply Reply = SButton::OnMouseButtonDown(MyGeometry, MouseEvent);
		if (Reply.IsEventHandled())
		{
			Reply.PreventThrottling();
		}
		return Reply;
	}
};

struct FDigitsTypeInterface : INumericTypeInterface<int32>
{
	virtual FString ToString(const int32& Value) const override
	{
		const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();

		return FString::Printf(TEXT("%0*d"), ProjectSettings->TakeNumDigits, Value);
	}

	virtual TOptional<int32> FromString(const FString& InString, const int32& ExistingValue) override
	{
		return (int32)FCString::Atoi(*InString);
	}

	virtual int32 GetMinFractionalDigits() const override { return 0; }
	virtual int32 GetMaxFractionalDigits() const override { return 0; }

	virtual void SetMinFractionalDigits(const TAttribute<TOptional<int32>>& NewValue) override {}
	virtual void SetMaxFractionalDigits(const TAttribute<TOptional<int32>>& NewValue) override {}

	virtual bool IsCharacterValid(TCHAR InChar) const override { return true; }
};

namespace UE::TakeRecorder::Private
{
	/** The pre-defined token name representing TAKE. */
	const FString TokenKeyTake(TEXT("take"));
	/** The pre-defined token name representing SLATE. */
	const FString TokenKeySlate(TEXT("slate"));

	/** Property name of the TakeSaveDir. */
	const FName TakeSaveDirPropertyName(GET_MEMBER_NAME_CHECKED(FTakeRecorderProjectParameters, TakeSaveDir));
	/** Property name of the RootTakeSaveDir. */
	const FName RootTakeSaveDirPropertyName(GET_MEMBER_NAME_CHECKED(FTakeRecorderProjectParameters, RootTakeSaveDir));

	/** Verify the token key is allowed to be user defined. */
	bool CanTokenBeUserDefined(const FString& InTokenKey)
	{
		return !InTokenKey.Equals(TokenKeySlate, ESearchCase::IgnoreCase) && !InTokenKey.Equals(TokenKeyTake, ESearchCase::IgnoreCase);
	}
	
	/** The token provided matches our own tokens exactly. */
	bool IsExplicitlyOurToken(const FString& InTokenKey)
	{
		return InTokenKey.Equals(TokenKeyTake, ESearchCase::CaseSensitive) || InTokenKey.Equals(TokenKeySlate, ESearchCase::CaseSensitive);
	}
}

void STakeRecorderCockpit::Construct(const FArguments& InArgs)
{
	PostUndoRedoDelegateHandle = FEditorDelegates::PostUndoRedo.AddRaw(this, &STakeRecorderCockpit::PostUndoRedo);

	CachedTakeSlate.Empty();
	CachedTakeNumber = -1;

	TakeRecorderSubsystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>();
	check(TakeRecorderSubsystem.IsValid());
	
	UpdateTakeError();
	UpdateRecordError();

	DigitsTypeInterface = MakeShareable(new FDigitsTypeInterface);

	BindCommands();

	TransactionIndex = INDEX_NONE;

	int32 Column[] = { 0, 1, 2 };
	int32 Row[]    = { 0, 1, 2 };

	TSharedPtr<SOverlay> OverlayHolder;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FTakeRecorderStyle::Get().GetBrush("TakeRecorder.Slate"))
		[

		SNew(SVerticalBox)

		// Slate, Take #, User Defined tokens, and Record Button 
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage_Lambda([this]{ return Reviewing() ? FTakeRecorderStyle::Get().GetBrush("TakeRecorder.TakeRecorderReviewBorder") : FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"); })
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				[
					SNew(SScrollBox).Orientation(Orient_Horizontal)
					+ SScrollBox::Slot()
					[
						SNew(SHorizontalBox)
						// Detected user tokens
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SAssignNew(UserTokensBox, SHorizontalBox)
						]
					]
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SAssignNew(OverlayHolder,SOverlay)

					+ SOverlay::Slot()
					[
						SNew(SBox)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.MaxAspectRatio(1)
						.Padding(FMargin(8.0f, 8.0f, 8.0f, 8.0f))
						.Visibility_Lambda([this]() { return Reviewing() ? EVisibility::Hidden : EVisibility::Visible; })
						[
							SNew(SCheckBox)
							.Style(FTakeRecorderStyle::Get(), "TakeRecorder.RecordButton")
							.OnCheckStateChanged(this, &STakeRecorderCockpit::OnToggleRecording)
							.IsChecked(this, &STakeRecorderCockpit::IsRecording)
							.IsEnabled(this, &STakeRecorderCockpit::CanRecord)
						]
					]

					+ SOverlay::Slot()
					[
						SNew(SBox)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.MaxAspectRatio(1)
						.Padding(FMargin(8.0f, 8.0f, 8.0f, 8.0f))
						.Visibility_Lambda([this]() { return Reviewing() ? EVisibility::Visible : EVisibility::Hidden; })
						[
							SNew(SButton)
							.ContentPadding(TakeRecorder::ButtonPadding)
							.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
							.ToolTipText(LOCTEXT("NewRecording", "Start a new recording using this Take as a base"))
							.ForegroundColor(FSlateColor::UseForeground())
							.OnClicked(this, &STakeRecorderCockpit::NewRecordingFromThis)
							[
								SNew(SImage)
								.Image(FTakeRecorderStyle::Get().GetBrush("TakeRecorder.StartNewRecordingButton"))
							]
						]
					]

					+ SOverlay::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.ToolTipText(this, &STakeRecorderCockpit::GetRecordErrorText)
						.Visibility(this, &STakeRecorderCockpit::GetRecordErrorVisibility)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
						.Text(FEditorFontGlyphs::Exclamation_Triangle)
					]

					+ SOverlay::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.ColorAndOpacity(FAppStyle::Get().GetSlateColor("InvertedForeground"))
						.Visibility(this, &STakeRecorderCockpit::GetCountdownVisibility)
						.Text(this, &STakeRecorderCockpit::GetCountdownText)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2)
				[
					SNew(SComboButton)
					.ContentPadding(2)
					.ForegroundColor(FSlateColor::UseForeground())
					.ComboButtonStyle(FTakeRecorderStyle::Get(), "ComboButton")
					.ToolTipText(LOCTEXT("RecordingOptionsTooltip", "Recording options"))
					.OnGetMenuContent(this, &STakeRecorderCockpit::OnRecordingOptionsMenu)
					.HasDownArrow(false)
					.ButtonContent()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "NormalText.Important")
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
						.Text(FEditorFontGlyphs::Caret_Down)
					]
				]
			]
		]

		// Take Save Dir Example text
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage_Lambda([this]{ return Reviewing() ? FTakeRecorderStyle::Get().GetBrush("TakeRecorder.TakeRecorderReviewBorder") : FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"); })
			[
				SNew(SEditableTextBox)
				.IsReadOnly(true)
				.Style(FTakeRecorderStyle::Get(), "TakeRecorder.EditableTextBox")
				.Font(FTakeRecorderStyle::Get().GetFontStyle("TakeRecorder.Cockpit.MediumText"))
				.Text(this, &STakeRecorderCockpit::GetEvaluatedTakeSaveDirText)
			]
		]

		// Timestamp, Duration, Description and Remaining Metadata
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FTakeRecorderStyle::Get().GetBrush("TakeRecorder.Slate.BorderImage"))
			.BorderBackgroundColor(FTakeRecorderStyle::Get().GetColor("TakeRecorder.Slate.BorderColor"))
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.Padding(8, 4, 0, 4)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FTakeRecorderStyle::Get().GetFontStyle("TakeRecorder.Cockpit.SmallText"))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Text(this, &STakeRecorderCockpit::GetTimestampText)
						.ToolTipText(this, &STakeRecorderCockpit::GetTimestampTooltipText)
					]

					+ SHorizontalBox::Slot()
					[
						SNew(SSpacer)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FTakeRecorderStyle::Get().GetFontStyle("TakeRecorder.Cockpit.MediumText"))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Justification(ETextJustify::Right)
						.Text(this, &STakeRecorderCockpit::GetTimecodeText)
						.ToolTipText(LOCTEXT("Timecode", "The current timecode"))
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SNonThrottledButton)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.ToolTipText(LOCTEXT("AddMarkedFrame", "Click to add a marked frame while recording"))
						.IsEnabled_Lambda([this]() { return IsRecording() == ECheckBoxState::Checked; })
						.OnClicked(this, &STakeRecorderCockpit::OnAddMarkedFrame)
						.ForegroundColor(FSlateColor::UseForeground())
						[
							SNew(SImage)
							.Image(FTakeRecorderStyle::Get().GetBrush("TakeRecorder.MarkFrame"))
						]
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(8, 4)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					[
						SNew(SSpacer)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0, 0, 4, 0)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						MakeHitchProtectionMismatchedFrameRateWarningIcon()
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SComboButton)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.OnGetMenuContent(this, &STakeRecorderCockpit::OnCreateMenu)
						.ForegroundColor(FSlateColor::UseForeground())
						.ButtonContent()
						[
							SNew(STextBlock)
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.Font(FTakeRecorderStyle::Get().GetFontStyle("TakeRecorder.Cockpit.SmallText"))
							.Text(this, &STakeRecorderCockpit::GetFrameRateText)
							.ToolTipText(this, &STakeRecorderCockpit::GetFrameRateTooltipText)
						]
					]
				]

				+SVerticalBox::Slot()
				.Padding(8, 0, 8, 8)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					[
						SNew(SEditableTextBox)
						.IsEnabled(this, &STakeRecorderCockpit::EditingMetaData)
						.Style(FTakeRecorderStyle::Get(), "TakeRecorder.EditableTextBox")
						.Font(FTakeRecorderStyle::Get().GetFontStyle("TakeRecorder.Cockpit.SmallText"))
						.SelectAllTextWhenFocused(true)
						.HintText(LOCTEXT("EnterSlateDescription_Hint", "<description>"))
						.Text(this, &STakeRecorderCockpit::GetUserDescriptionText)
						.OnTextCommitted(this, &STakeRecorderCockpit::SetUserDescriptionText)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpinBox<float>)
						.ToolTipText(LOCTEXT("EngineTimeDilation", "Recording speed"))
						.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
						.OnValueChanged(this, &STakeRecorderCockpit::SetEngineTimeDilation)
						.OnValueCommitted_Lambda([this](float InEngineTimeDilation, ETextCommit::Type) { SetEngineTimeDilation(InEngineTimeDilation); })
						.MinValue(TOptional<float>())
						.MaxValue(TOptional<float>())
						.Value(this, &STakeRecorderCockpit::GetEngineTimeDilation)
						.Delta(0.5f)
					]

					+ SHorizontalBox::Slot()
					.Padding(2, 0, 0, 2)
					.VAlign(VAlign_Bottom)
					.AutoWidth()
					[
						SNew(STextBlock)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Font(FTakeRecorderStyle::Get().GetFontStyle("TakeRecorder.Cockpit.SmallText"))
						.TextStyle(FTakeRecorderStyle::Get(), "TakeRecorder.TextBox")
						.Text(LOCTEXT("EngineTimeDilationLabel", "x"))
					]
				]
			]
		]
		]
	];

	ITakeRecorderModule& TakeRecorderModule = ITakeRecorderModule::Get();
	TArray<TSharedRef<SWidget>> OutExtensions;
	TakeRecorderModule.GetRecordButtonExtensionGenerators().Broadcast(OutExtensions);
	for (const TSharedRef<SWidget>& Widget : OutExtensions)
	{
		OverlayHolder->AddSlot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				Widget
			];
	}
	
	bRefreshUndefinedTokens = true;
}

bool STakeRecorderCockpit::CanStartRecording(FText& OutErrorText) const
{
	bool bCanRecord = CanRecord();
	if (!bCanRecord)
	{
		OutErrorText = RecordErrorText;
	}
	return bCanRecord;
}

FText STakeRecorderCockpit::GetTakeWarningText() const
{
	return TakeErrorText;
}

EVisibility STakeRecorderCockpit::GetTakeWarningVisibility() const
{
	return TakeErrorText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
}

FText STakeRecorderCockpit::GetRecordErrorText() const
{
	return RecordErrorText;
}

EVisibility STakeRecorderCockpit::GetRecordErrorVisibility() const
{
	return RecordErrorText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
}

void STakeRecorderCockpit::UpdateRecordError()
{
	RecordErrorText = FText();
	if (Reviewing())
	{
		// When take meta-data is locked, we cannot record until we hit the "Start a new recording using this Take as a base"
		// For this reason, we don't show any error information because we can always start a new recording from any take
		return;
	}

	ULevelSequence* Sequence = TakeRecorderSubsystem->GetLevelSequence();
	if (!Sequence)
	{
		RecordErrorText = LOCTEXT("ErrorWidget_NoSequence", "There is no sequence to record from. Please re-open Take Recorder.");
		return;
	}

	if (!Sequence->HasAnyFlags(RF_Transient) && TakeRecorderSubsystem->GetTakeRecorderMode() != ETakeRecorderMode::RecordIntoSequence)
	{
		RecordErrorText = FText();
		return;
	}

	// Recording into an existing sequence with timecode-relative start would shift every
	// existing section by the current timecode delta and replace the playback range,
	// invalidating previously recorded data (UE-375286).
	if (TakeRecorderSubsystem->GetTakeRecorderMode() == ETakeRecorderMode::RecordIntoSequence)
	{
		const FTakeRecorderProjectParameters& ProjectSettings = GetDefault<UTakeRecorderProjectSettings>()->Settings;
		if (ProjectSettings.bStartAtCurrentTimecode)
		{
			RecordErrorText = LOCTEXT("ErrorWidget_RecordIntoSequenceStartAtTimecode",
				"Recording into an existing sequence is not supported when 'Start At Current Timecode' is enabled, "
				"because it would shift the existing sections and invalidate previously recorded data. "
				"Disable 'Start At Current Timecode' in the Take Recorder project settings.");
			return;
		}
		if (ProjectSettings.RecordingClockSource == EUpdateClockSource::Timecode)
		{
			RecordErrorText = LOCTEXT("ErrorWidget_RecordIntoSequenceTimecodeClock",
				"Recording into an existing sequence is not supported when the recording clock source is set to Timecode, "
				"because it forces sections to start at the current timecode and would invalidate previously recorded data. "
				"Change the Recording Clock Source in the Take Recorder project settings.");
			return;
		}
	}

	UTakeRecorderSources* SourcesContainer = ITakeRecorderSourcesManager::GetChecked().FindSources(Sequence);
	bool bValidSources = false;
	if (SourcesContainer)
	{
		TArrayView<UTakeRecorderSource* const> SourcesArray = SourcesContainer->GetSources();
		UTakeRecorderSource* const* Source = Algo::FindByPredicate(SourcesArray, [](const UTakeRecorderSource* Source)
		{
			return Source && Source->bEnabled && Source->IsValid();
		});
		bValidSources = Source != nullptr;
	}

	if (!bValidSources)
	{
		RecordErrorText = LOCTEXT("ErrorWidget_NoSources", "There are no currently enabled or valid sources to record from. Please add some above before recording.");
		return;
	}

	UTakeMetaData* TakeMetaData = GetMetaData();

	if (TakeMetaData && TakeMetaData->GetSlate().IsEmpty())
	{
		RecordErrorText = LOCTEXT("ErrorWidget_NoSlate", "You must enter a slate to begin recording.");
		return;
	}

	FString PackageName;

	if (TakeMetaData)
	{
		if (!TakeMetaData->TryGenerateRootAssetPath(GetDefault<UTakeRecorderProjectSettings>()->Settings.GetTakeAssetPath(), PackageName, &RecordErrorText))
		{
			return;
		}
		FText OutReason;
		if (!FPackageName::IsValidLongPackageName(PackageName, false, &OutReason))
		{
			RecordErrorText = FText::Format(LOCTEXT("ErrorWidget_InvalidPath", "{0} is not a valid asset path. {1}"), FText::FromString(PackageName), OutReason);
			return;
		}
	}

	if (TakeMetaData && TakeMetaData->GetFrameRateFromTimecode())
	{
		if (TakeMetaData->GetFrameRate() == FFrameRate())
		{
			RecordErrorText = LOCTEXT("ErrorWidget_FrameRateHigh", "The timecode rate is too high for recording.  Ensure you have a proper timecode provider set in the engine.");
			return;
		}
	}
	const int32 MaxLength = 260;

	if (PackageName.Len() > MaxLength)
	{
		RecordErrorText = FText::Format(LOCTEXT("ErrorWidget_TooLong", "The path to the asset is too long ({0} characters), the maximum is {1}.\nPlease choose a shorter name for the slate or create it in a shallower folder structure with shorter folder names."), FText::AsNumber(PackageName.Len()), FText::AsNumber(MaxLength));
		return;
	}
	ITakeRecorderModule& TakeRecorderModule = FModuleManager::Get().LoadModuleChecked<ITakeRecorderModule>("TakeRecorder");
	TakeRecorderModule.GetRecordErrorCheckGenerator().Broadcast(RecordErrorText);
}

void STakeRecorderCockpit::UpdateTakeError()
{
	TakeErrorText = FText();

	const UTakeMetaData* TakeMetaData = GetMetaDataChecked();
	
	TArray<FAssetData> DuplicateTakes = UTakesCoreBlueprintLibrary::FindTakes(TakeMetaData->GetSlate(), TakeMetaData->GetTakeNumber());

	// If there's only a single one, and it's the one that we're looking at directly, don't show the error
	if (DuplicateTakes.Num() == 1 && DuplicateTakes[0].IsValid())
	{
		ULevelSequence* AlreadyLoaded = FindObject<ULevelSequence>(nullptr, *DuplicateTakes[0].GetObjectPathString());
		if (AlreadyLoaded && AlreadyLoaded->FindMetaData<UTakeMetaData>() == TakeMetaData)
		{
			return;
		}
	}

	if (DuplicateTakes.Num() > 0)
	{
		FTextBuilder TextBuilder;
		TextBuilder.AppendLineFormat(
			LOCTEXT("DuplicateTakeNumber_1", "The following Level {0}|plural(one=Sequence, other=Sequences) {0}|plural(one=was, other=were) also recorded with take {1} of {2}"),
			DuplicateTakes.Num(),
			FText::AsNumber(TakeMetaData->GetTakeNumber()),
			FText::FromString(TakeMetaData->GetSlate())
		);

		for (const FAssetData& Asset : DuplicateTakes)
		{
			TextBuilder.AppendLine(FText::FromName(Asset.PackageName));
		}

		TextBuilder.AppendLine(LOCTEXT("GetNextAvailableTakeNumber", "Click to get the next available take number."));
		TakeErrorText = TextBuilder.ToText();
	}
}

EVisibility STakeRecorderCockpit::GetCountdownVisibility() const
{
	const UTakeRecorder* CurrentRecording = UTakeRecorder::GetActiveRecorder();
	const bool           bIsCountingDown  = CurrentRecording && CurrentRecording->GetState() == ETakeRecorderState::CountingDown;

	return bIsCountingDown ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}

FText STakeRecorderCockpit::GetCountdownText() const
{
	const UTakeRecorder* CurrentRecording = UTakeRecorder::GetActiveRecorder();
	const bool           bIsCountingDown  = CurrentRecording && CurrentRecording->GetState() == ETakeRecorderState::CountingDown;

	return bIsCountingDown ? FText::AsNumber(FMath::CeilToInt(CurrentRecording->GetCountdownSeconds())) : FText();
}

TSharedRef<SWidget> STakeRecorderCockpit::OnRecordingOptionsMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CancelRecording_Text", "Cancel Recording"),
		LOCTEXT("CancelRecording_Tip", "Cancel the current recording, deleting any assets and resetting the take number"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &STakeRecorderCockpit::CancelRecording),
			FCanExecuteAction::CreateLambda([this] { return Recording(); })
		)
	);

	return MenuBuilder.MakeWidget();
}

void STakeRecorderCockpit::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Refresh();
	if (bRefreshUndefinedTokens)
	{
		bRefreshUndefinedTokens = false;
		RefreshUndefinedTokens();
	}
}

void STakeRecorderCockpit::PostUndoRedo()
{
	RefreshUndefinedTokens();
}

TSharedRef<SWidget> STakeRecorderCockpit::MakeHitchProtectionMismatchedFrameRateWarningIcon()
{
	return SNew(SScaleBox)
	[
		SNew(SImage)
		.Image(FTakeRecorderStyle::Get().GetBrush("Hitching.MismatchedFrameRate.Icon"))
		.ColorAndOpacity(FTakeRecorderStyle::Get().GetColor("Hitching.MismatchedFrameRate.Normal"))
		.Visibility_Lambda([]
		{
			return UE::TakeRecorder::HitchProtectionFrameRateModel::ShouldShowFrameRateWarning() ? EVisibility::Visible : EVisibility::Hidden;
		})
		.ToolTipText_Lambda([]{ return UE::TakeRecorder::HitchProtectionFrameRateModel::GetMismatchedFrameRateWarningTooltipText(); })
	];
}

void STakeRecorderCockpit::Refresh()
{
	if (const UTakeMetaData* TakeMetaData = GetMetaData())
	{
		if (TakeMetaData->GetSlate() != CachedTakeSlate || TakeMetaData->GetTakeNumber() != CachedTakeNumber)
		{
			CachedTakeNumber = TakeMetaData->GetTakeNumber();
			CachedTakeSlate = TakeMetaData->GetSlate();

			// Previously, the take error would be updated in Tick(), but the asset registry can be slow, 
			// so it should be sufficient to update it only when the slate changes.
			UpdateTakeError();
		}
	}
	
	UpdateRecordError();
}

void STakeRecorderCockpit::NotifyPropertyUpdated(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.Property)
	{
		FString ResultStr;

		for (int32 Idx = 0; Idx < InPropertyChangedEvent.GetNumObjectsBeingEdited(); ++Idx )
		{
			if (const UObject* Object = InPropertyChangedEvent.GetObjectBeingEdited(Idx))
			{
				bool bForce = false;
				const void* Container = Object;
				TMap<FString, int32> ArrayIndicesPerObject;
				if (InPropertyChangedEvent.MemberProperty && InPropertyChangedEvent.MemberProperty != InPropertyChangedEvent.Property)
				{
					// This could be a member of a struct, which has its own metadata and container.
					if (InPropertyChangedEvent.MemberProperty->HasMetaData(*UE::NamingTokens::Specifiers::UseNamingTokens)
						||
						// @hack: When dealing with the RootTakeSaveDir, we don't have a good way of getting the meta specifier, so we need to
						// check if the property is contained in the ArrayIndices. The MemberProperty *should* be the FDirectoryPath property,
						// but it's actually the FTakeRecorderProjectParameters property.
						// ChainProperty might be better for this, but we're limited to what the details panel property changed event provides us.
						// @todo NamingTokens - If we move to a more global system, we need a more robust way of checking for meta specifiers on FDirectoryPaths.
						// @note: The PresetSaveLocation we use doesn't have this issue and the MemberProperty is correct.
						(InPropertyChangedEvent.GetArrayIndicesPerObject(0, ArrayIndicesPerObject)
							&& ArrayIndicesPerObject.Contains(UE::TakeRecorder::Private::RootTakeSaveDirPropertyName.ToString())))
					{
						bForce = true;
					}
					
					Container = InPropertyChangedEvent.MemberProperty->ContainerPtrToValuePtr<void>(Object);
				}
				
				EvaluateTokensFromProperty(InPropertyChangedEvent.Property, Container, bForce);
				CreateUserTokensUI();
			}
		}
	}
}

void STakeRecorderCockpit::NotifyDetailsViewAdded(const TWeakPtr<IDetailsView>& InDetailsView)
{
	// Cleanup stale entries. This list can change when the user clicks on or off an actor.
	DetailViews.RemoveAll([](const TWeakPtr<IDetailsView>& DetailsView)
	{
		return !DetailsView.IsValid();
	});
	
	DetailViews.Add(InDetailsView);

	// We signal a refresh on the next tick rather than now so all details views have time to process. Most likely
	// this method is being called multiple times since there are multiple objects being added. We want to refresh after
	// all have been added so we don't remove any custom tokens that are still defined.
	bRefreshUndefinedTokens = true;
}

FText STakeRecorderCockpit::GetSlateText() const
{
	return FText::FromString(GetMetaDataChecked()->GetSlate());
}

FText STakeRecorderCockpit::GetTimecodeText() const
{
	return FText::FromString(FApp::GetTimecode().ToString());
}

FText STakeRecorderCockpit::GetUserDescriptionText() const
{
	const UTakeMetaData* TakeMetaData = GetMetaDataChecked();
	return FText::FromString(TakeMetaData->GetDescription());
}

FText STakeRecorderCockpit::GetTimestampText() const
{
	// If not recorded, return current time
	const UTakeMetaData* TakeMetaData = GetMetaDataChecked();
	if (TakeMetaData->GetTimestamp() == FDateTime(0))
	{
		return FText::AsDateTime(FDateTime::UtcNow());
	}
	else
	{
		return FText::AsDateTime(TakeMetaData->GetTimestamp());
	}
}

FText STakeRecorderCockpit::GetTimestampTooltipText() const
{
	// If not recorded, return current time
	const UTakeMetaData* TakeMetaData = GetMetaDataChecked();
	if (TakeMetaData->GetTimestamp() == FDateTime(0))
	{
		return LOCTEXT("CurrentTimestamp", "The current date/time");
	}
	else
	{
		return LOCTEXT("Timestamp", "The date/time this recording was created at");
	}
}

FText STakeRecorderCockpit::GetEvaluatedTakeSaveDirText() const
{
	return TakeRecorderSubsystem->GetNamingTokensData()->EvaluatedTextValue;
}

void STakeRecorderCockpit::SetFrameRate(FFrameRate InFrameRate, bool bFromTimecode)
{
	if (bFromTimecode)
	{
		TakeRecorderSubsystem->SetFrameRateFromTimecode();
	}
	else
	{
		TakeRecorderSubsystem->SetFrameRate(InFrameRate);
	}
}

bool STakeRecorderCockpit::IsSameFrameRate(FFrameRate InFrameRate) const
{
	return (InFrameRate == GetFrameRate());
}

FFrameRate STakeRecorderCockpit::GetFrameRate() const
{
	return TakeRecorderSubsystem->GetFrameRate();
}

FText STakeRecorderCockpit::GetFrameRateText() const
{
	return GetFrameRate().ToPrettyText();
}

FText STakeRecorderCockpit::GetFrameRateTooltipText() const
{
	return LOCTEXT("ProjectFrameRate", "The project timecode frame rate. The resulting recorded sequence will be at this frame rate.");
}

bool STakeRecorderCockpit::IsFrameRateCompatible(FFrameRate InFrameRate) const
{
	ULevelSequence* Sequence   = TakeRecorderSubsystem->GetLevelSequence();
	UMovieScene*    MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

	return MovieScene && InFrameRate.IsMultipleOf(MovieScene->GetTickResolution());
}

bool STakeRecorderCockpit::IsSetFromTimecode() const
{
	return GetMetaDataChecked()->GetFrameRateFromTimecode();
}

void STakeRecorderCockpit::SetSlateText(const FText& InNewText, ETextCommit::Type InCommitType)
{
	UTakeMetaData* TakeMetaData = GetMetaDataChecked();
	if (TakeMetaData->GetSlate() != InNewText.ToString())
	{
		TakeRecorderSubsystem->SetSlateName(InNewText.ToString());
		OnTokenValueUpdated();
	}
}

void STakeRecorderCockpit::SetUserDescriptionText(const FText& InNewText, ETextCommit::Type InCommitType)
{
	UTakeMetaData* TakeMetaData = GetMetaDataChecked();
	if (TakeMetaData->GetDescription() != InNewText.ToString())
	{
		FScopedTransaction Transaction(LOCTEXT("SetDescription_Transaction", "Set Description"));
		TakeMetaData->Modify();

		TakeMetaData->SetDescription(InNewText.ToString());
	}
}

int32 STakeRecorderCockpit::GetTakeNumber() const
{
	return GetMetaDataChecked()->GetTakeNumber();
}

FReply STakeRecorderCockpit::OnSetNextTakeNumber()
{
	UTakeMetaData* TakeMetaData = GetMetaDataChecked();
	const int32 NextTakeNumber = UTakesCoreBlueprintLibrary::ComputeNextTakeNumber(TakeMetaData->GetSlate());
	if (TakeMetaData->GetTakeNumber() != NextTakeNumber)
	{
		FScopedTransaction Transaction(LOCTEXT("SetNextTakeNumber_Transaction", "Set Next Take Number"));

		TakeMetaData->Modify();
		TakeMetaData->SetTakeNumber(NextTakeNumber);
	}

	return FReply::Handled();
}

void STakeRecorderCockpit::OnBeginSetTakeNumber()
{
	const bool bIsInPIEOrSimulate = GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor;

	if (!bIsInPIEOrSimulate)
	{
		check(TransactionIndex == INDEX_NONE);
	}

	TransactionIndex = GEditor->BeginTransaction(nullptr, LOCTEXT("SetTakeNumber_Transaction", "Set Take Number"), nullptr);
	UTakeMetaData* TakeMetaData = GetMetaDataChecked();
	TakeMetaData->Modify();
}

void STakeRecorderCockpit::SetTakeNumber(int32 InNewTakeNumber)
{
	const bool bIsInPIEOrSimulate = GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor;

	if (TransactionIndex != INDEX_NONE || bIsInPIEOrSimulate)
	{
		// Don't emit here, will be done later.
		constexpr bool bEmitChanged = false;
		TakeRecorderSubsystem->SetTakeNumber(InNewTakeNumber, bEmitChanged);
	}
	
	OnTokenValueUpdated();
}

void STakeRecorderCockpit::SetTakeNumber_FromCommit(int32 InNewTakeNumber, ETextCommit::Type InCommitType)
{
	const bool bIsInPIEOrSimulate = GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor;

	UTakeMetaData* TakeMetaData = GetMetaDataChecked();
	if (TransactionIndex == INDEX_NONE && !bIsInPIEOrSimulate)
	{
		if (TakeMetaData->GetTakeNumber() != InNewTakeNumber)
		{
			OnBeginSetTakeNumber();
			OnEndSetTakeNumber(InNewTakeNumber);
		}
	}
	else if (TakeMetaData->GetTakeNumber() != InNewTakeNumber)
	{
		TakeRecorderSubsystem->SetTakeNumber(InNewTakeNumber, /* bEmitChanged */ true);
	}

	OnTokenValueUpdated();
}

void STakeRecorderCockpit::OnEndSetTakeNumber(int32 InFinalValue)
{
	const bool bIsInPIEOrSimulate = GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor;

	if (!bIsInPIEOrSimulate)
	{
		check(TransactionIndex != INDEX_NONE);
	}
	TakeRecorderSubsystem->SetTakeNumber(InFinalValue, /* bEmitChanged */ true);

	GEditor->EndTransaction();
	TransactionIndex = INDEX_NONE;
}

float STakeRecorderCockpit::GetEngineTimeDilation() const
{
	return GetDefault<UTakeRecorderUserSettings>()->Settings.EngineTimeDilation;
}

void STakeRecorderCockpit::SetEngineTimeDilation(float InEngineTimeDilation)
{
	GetMutableDefault<UTakeRecorderUserSettings>()->Settings.EngineTimeDilation = InEngineTimeDilation;
	GetMutableDefault<UTakeRecorderUserSettings>()->SaveConfig();
}

FReply STakeRecorderCockpit::OnAddMarkedFrame()
{
	TakeRecorderSubsystem->MarkFrame();
	return FReply::Handled();
}

bool STakeRecorderCockpit::Reviewing() const 
{
	return TakeRecorderSubsystem->IsReviewing();
}

bool STakeRecorderCockpit::Recording() const
{
	return TakeRecorderSubsystem->IsRecording();
}

ECheckBoxState STakeRecorderCockpit::IsRecording() const
{
	return Recording() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool STakeRecorderCockpit::CanRecord() const
{
	return RecordErrorText.IsEmpty();
}

bool STakeRecorderCockpit::IsLocked() const 
{
	if (const UTakeMetaData* TakeMetaData = GetMetaData())
	{
		return TakeMetaData->IsLocked();
	}
	return false;
}

void STakeRecorderCockpit::OnToggleRecording(ECheckBoxState)
{
	ULevelSequence*       LevelSequence = TakeRecorderSubsystem->GetLevelSequence();
	UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindSources(LevelSequence);

	UTakeRecorder* CurrentRecording = UTakeRecorder::GetActiveRecorder();
	if (CurrentRecording)
	{
		StopRecording();
	}
	else if (LevelSequence && Sources)
	{
		StartRecording();
	}
}

void STakeRecorderCockpit::StopRecording()
{
	TakeRecorderSubsystem->StopRecording();
}

void STakeRecorderCockpit::CancelRecording()
{
	TakeRecorderSubsystem->CancelRecording();
}

void STakeRecorderCockpit::StartRecording()
{
	TakeRecorderSubsystem->StartRecording();
}

FReply STakeRecorderCockpit::NewRecordingFromThis()
{
	ULevelSequence* Sequence = TakeRecorderSubsystem->GetLevelSequence();
	if (!Sequence)
	{
		return FReply::Unhandled();
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<SDockTab> DockTab = LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(ITakeRecorderModule::TakeRecorderTabName);
	if (DockTab.IsValid())
	{
		TSharedRef<STakeRecorderTabContent> TabContent = StaticCastSharedRef<STakeRecorderTabContent>(DockTab->GetContent());
		TabContent->SetupForRecording(Sequence);
		if (const TSharedPtr<STakeRecorderPanel> TakeRecorderPanel = TabContent->GetTakeRecorderPanel().Pin())
		{
			// Go through panel rather than subsystem so sequencer remains open if necessary.
			TakeRecorderPanel->ResetToPendingTake();
			TakeRecorderSubsystem->IncrementTakeNumber();
		}
	}

	return FReply::Handled();
}

void STakeRecorderCockpit::BindCommands()
{
	// Bind our commands to the play world so that we can record in editor and in PIE
	FPlayWorldCommands::GlobalPlayWorldActions->MapAction(
		FTakeRecorderCommands::Get().StartRecording,
		FExecuteAction::CreateSP(this, &STakeRecorderCockpit::StartRecording));
	FPlayWorldCommands::GlobalPlayWorldActions->MapAction(
		FTakeRecorderCommands::Get().StopRecording,
		FExecuteAction::CreateSP(this, &STakeRecorderCockpit::StopRecording));
}

void STakeRecorderCockpit::OnToggleEditPreviousRecording(ECheckBoxState CheckState)
{
	if (Reviewing())
	{
		UTakeMetaData* TakeMetaData = GetMetaDataChecked();
		TakeMetaData->IsLocked() ? TakeMetaData->Unlock() : TakeMetaData->Lock();
	}	
}

bool STakeRecorderCockpit::EditingMetaData() const
{
	return (!Reviewing() || !GetMetaDataChecked()->IsLocked());
}

TSharedRef<SWidget> STakeRecorderCockpit::MakeLockButton()
{
	return SNew(SCheckBox)
	.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
	.Padding(TakeRecorder::ButtonPadding)
	.ToolTipText(LOCTEXT("Modify Slate", "Unlock to modify the slate information for this prior recording."))
	.IsChecked_Lambda([this]() { return GetMetaDataChecked()->IsLocked() ? ECheckBoxState::Unchecked: ECheckBoxState::Checked; } )
	.OnCheckStateChanged(this, &STakeRecorderCockpit::OnToggleEditPreviousRecording)
	.Visibility_Lambda([this]() { return Reviewing() ? EVisibility::Visible : EVisibility::Collapsed; })
	[
		SNew(STextBlock)
		.Justification(ETextJustify::Center)
		.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
		.Text_Lambda([this]() { return GetMetaDataChecked()->IsLocked() ? FEditorFontGlyphs::Lock : FEditorFontGlyphs::Unlock; } )
	];
}

TSharedRef<SWidget> STakeRecorderCockpit::OnCreateMenu()
{
	ULevelSequence* Sequence = TakeRecorderSubsystem->GetLevelSequence();
	if (!Sequence || !Sequence->GetMovieScene())
	{
		return SNullWidget::NullWidget;
	}
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	
	FMenuBuilder MenuBuilder(true, nullptr);

	FFrameRate TickResolution = MovieScene->GetTickResolution();

	TArray<FCommonFrameRateInfo> CompatibleRates;
	for (const FCommonFrameRateInfo& Info : FCommonFrameRates::GetAll())
	{
		if (Info.FrameRate.IsMultipleOf(TickResolution))
		{
			CompatibleRates.Add(Info);
		}
	}

	CompatibleRates.Sort(
		[=](const FCommonFrameRateInfo& A, const FCommonFrameRateInfo& B)
	{
		return A.FrameRate.AsDecimal() < B.FrameRate.AsDecimal();
	}
	);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("RecommendedRates", "Sequence Display Rate"));
	{
		for (const FCommonFrameRateInfo& Info : CompatibleRates)
		{
			MenuBuilder.AddMenuEntry(
				Info.DisplayName,
				Info.Description,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &STakeRecorderCockpit::SetFrameRate, Info.FrameRate,false),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &STakeRecorderCockpit::IsSameFrameRate, Info.FrameRate)
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);

		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.AddMenuSeparator();
	FFrameRate TimecodeFrameRate = FApp::GetTimecodeFrameRate();
	FText DisplayName = FText::Format(LOCTEXT("TimecodeFrameRate", "Timecode ({0})"), TimecodeFrameRate.ToPrettyText());

	MenuBuilder.AddMenuEntry(
		DisplayName,
		DisplayName,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &STakeRecorderCockpit::SetFrameRate, TimecodeFrameRate,true),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &STakeRecorderCockpit::IsSetFromTimecode)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
	return MenuBuilder.MakeWidget();
}

UTakeRecorderNamingTokensData* STakeRecorderCockpit::GetNamingTokensData() const
{
	check(TakeRecorderSubsystem.IsValid());
	return TakeRecorderSubsystem->GetNamingTokensData();
}

FText STakeRecorderCockpit::GetCustomTokenTextValue(FString InTokenKey) const
{
	if (const FText* Value = GetNamingTokensData()->UserDefinedTokens.Find(FNamingTokenData(InTokenKey)))
	{
		return *Value;
	}

	return FText::GetEmpty();
}

void STakeRecorderCockpit::SetCustomTokenTextValue(const FText& InNewText, ETextCommit::Type InCommitType, FString InTokenKey)
{
		FScopedTransaction Transaction(LOCTEXT("SetCustomTokenTextValue_Transaction", "Set Custom Token Text"));
		const FNamingTokenData CustomToken(InTokenKey);
		GetNamingTokensData()->Modify();
		if (UE::TakeRecorder::Private::CanTokenBeUserDefined(InTokenKey))
		{
			GetNamingTokensData()->UserDefinedTokens.FindOrAdd(CustomToken) = InNewText;
		}
		GetNamingTokensData()->VisibleUserTokens.Add(CustomToken.TokenKey);
		OnTokenValueUpdated();
}

void STakeRecorderCockpit::OnTokenValueUpdated()
{
	EvaluateTakeSaveDirTokens();
}

void STakeRecorderCockpit::EvaluateTakeSaveDirTokens()
{
	if (CachedTakeSaveDirProperty.IsValid() && CachedTakeSaveDirContainer)
	{
		EvaluateTokensFromProperty(CachedTakeSaveDirProperty.Get(), CachedTakeSaveDirContainer);
	}
}

void STakeRecorderCockpit::RefreshUndefinedTokens()
{
	// Clicking on different actors will remove the old details panels -- clear out the fields
	// we are tracking. They will be repopulated when evaluating the tokens below.
	GetNamingTokensData()->FieldToUndefinedKeys.Empty();
	
	bool bHasMinRequiredObjects = false;
	const UTakeRecorderUserSettings* UserSettings = GetDefault<UTakeRecorderUserSettings>();
	const UTakeRecorderProjectSettings* ProjectSettings = GetDefault<UTakeRecorderProjectSettings>();

	for (const TWeakPtr<IDetailsView>& DetailsView : DetailViews)
	{
		if (const TSharedPtr<IDetailsView> DetailsPin = DetailsView.Pin())
		{
			// Look if the default user or project settings are available. This should only fail if the user has hidden these objects.
			if (DetailsPin->GetSelectedObjects().Contains(UserSettings)
				|| DetailsPin->GetSelectedObjects().Contains(ProjectSettings))
			{
				bHasMinRequiredObjects = true;
			}
		}
		UpdateUndefinedTokensFromDetailsView(DetailsView);
	}

	// We always need to display tokens for the user/project settings. This should be done after any current details tokens are calculated
	// to maintain order.
	if (!bHasMinRequiredObjects)
	{
		UpdateUndefinedTokensFromTemporaryDetailsView();
	}
	
	CreateUserTokensUI();
}

void STakeRecorderCockpit::UpdateUndefinedTokensFromDetailsView(const TWeakPtr<IDetailsView>& InDetailsView)
{
	if (const TSharedPtr<IDetailsView> DetailsViewPin = InDetailsView.Pin())
	{
		const TArray<FPropertyPath> Properties = DetailsViewPin->GetPropertiesInOrderDisplayed();
		const TArray<TWeakObjectPtr<UObject>> Objects = DetailsViewPin->GetSelectedObjects();
		for (const TWeakObjectPtr<UObject>& EditedObject : Objects)
		{
			if (!EditedObject.IsValid())
			{
				continue;
			}
			auto GetCorrectContainer = [&EditedObject](const FProperty* InProperty) -> UObject*
			{
				if (InProperty)
				{
					// Locate the correct container to use. When using a project settings object it could also be an additional settings object.
					// The additional settings isn't a UPROPERTY and are added to the details via TakeRecorder customization, so they won't be discovered
					// by GetSelectedObjects. Ideally we wouldn't be aware that we are of the type of object we are editing.
					if (const UTakeRecorderProjectSettings* ProjectSettings = Cast<UTakeRecorderProjectSettings>(EditedObject))
					{
						for (const TWeakObjectPtr<UObject>& AdditionalObj : ProjectSettings->AdditionalSettings)
						{
							if (UClass* OwnerClass = InProperty->GetOwnerClass())
							{
								if (AdditionalObj.IsValid() && AdditionalObj->IsA(OwnerClass))
								{
									return AdditionalObj.Get();
								}
							}
						}
					}
				}
				return EditedObject.Get();
			};
		
			for (const FPropertyPath& PropertyPath : Properties)
			{
				void* CurrentContainer = GetCorrectContainer(PropertyPath.GetRootProperty().Property.Get());
				const int32 NumSegments = PropertyPath.GetNumProperties();

				// For multi-length segments adjust the container.
				for (int32 Idx = 0; Idx < NumSegments - 1; ++Idx)
				{
					const FProperty* Prop = PropertyPath.GetPropertyInfo(Idx).Property.Get();
					if (!Prop)
					{
						CurrentContainer = nullptr;
						break;
					}
					CurrentContainer = Prop->ContainerPtrToValuePtr<void>(CurrentContainer);
					if (!CurrentContainer)
					{
						break;
					}
				}

				const FProperty* LeafProperty = PropertyPath.GetLeafMostProperty().Property.Get();
				if (!LeafProperty || !CurrentContainer)
				{
					continue;
				}

				bool bForce = false;
				if (const FStructProperty* StructProperty = CastField<FStructProperty>(LeafProperty))
				{
					if (StructProperty->HasMetaData(*UE::NamingTokens::Specifiers::UseNamingTokens))
					{
						// The property we evaluate won't have the metadata so we need to force it.
						bForce = true;
					
						// Need to adjust the container for the struct.
						CurrentContainer = StructProperty->ContainerPtrToValuePtr<void>(CurrentContainer);
				
						// Directory and FilePaths won't have their string Path iterated here, so find it manually. Not USTRUCTs so have to check name.
						const FName StructName = StructProperty->Struct->GetFName();
						if (StructName == FName("DirectoryPath"))
						{
							LeafProperty = FindFProperty<FStrProperty>(StructProperty->Struct, GET_MEMBER_NAME_CHECKED(FDirectoryPath, Path));
						}
						else if (StructName == FName("FilePath"))
						{
							LeafProperty = FindFProperty<FStrProperty>(StructProperty->Struct, GET_MEMBER_NAME_CHECKED(FFilePath, FilePath));
						}
					}
				}

				EvaluateTokensFromProperty(LeafProperty, CurrentContainer, bForce);
			}
		}
	}
}

void STakeRecorderCockpit::UpdateUndefinedTokensFromTemporaryDetailsView()
{
	if (TemporaryDetailsViews.Num() == 0)
	{
		UTakeRecorderProjectSettings* ProjectSettings = GetMutableDefault<UTakeRecorderProjectSettings>();
		UTakeRecorderUserSettings* UserSettings = GetMutableDefault<UTakeRecorderUserSettings>();
		
		const TArray<UObject*> ObjectsToDisplay { ProjectSettings, UserSettings };

		for (UObject* Object : ObjectsToDisplay)
		{
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			DetailsViewArgs.bHideSelectionTip = true;
			DetailsViewArgs.bShowScrollBar = false;

			TSharedPtr<IDetailsView> TemporaryDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

			// Have to pass individual objects through, won't work if we just use one view with all our objects.
			const TArray<UObject*> Objects { Object };
			TemporaryDetailsView->SetObjects(Objects);
			TemporaryDetailsView->SetEnabled(false);

			TemporaryDetailsViews.Add(MoveTemp(TemporaryDetailsView));
		}
	}

	for (const TSharedPtr<IDetailsView>& TemporaryDetailsView : TemporaryDetailsViews)
	{
		UpdateUndefinedTokensFromDetailsView(TemporaryDetailsView.ToWeakPtr());
	}
}

void STakeRecorderCockpit::EvaluateTokensFromProperty(const FProperty* InProperty, const void* InContainer, bool bForce)
{
	check(InProperty);
	check(InContainer);

	const FProperty* OwnerProperty = InProperty->GetOwnerProperty();
	const bool bIsNamingTokensField = bForce
		|| InProperty->HasMetaData(*UE::NamingTokens::Specifiers::UseNamingTokens)
		|| (OwnerProperty && OwnerProperty->HasMetaData(*UE::NamingTokens::Specifiers::UseNamingTokens));
	
	if (!bIsNamingTokensField)
	{
		return;
	}
	
	FText ResultText;
	if (const FStrProperty* StringProp = CastField<FStrProperty>(InProperty))
	{
		if (const FString* ValuePtr = StringProp->ContainerPtrToValuePtr<FString>(InContainer))
		{
			ResultText = FText::FromString(*ValuePtr);
		}
	}
	else if (const FTextProperty* TextProp = CastField<FTextProperty>(InProperty))
	{
		if (const FText* TextPtr = TextProp->ContainerPtrToValuePtr<FText>(InContainer))
		{
			ResultText = *TextPtr;
		}
	}
	
	FNamingTokenResultData NamingTokenResultData;
	if (!ResultText.IsEmpty())
	{
		// Create an identifier which handles duplicate property names on different objects.
		// Only valid for the container's life cycle, which should match this widget.
		const FString FieldName = FString::Printf(TEXT("%s_%p"), *InProperty->GetName(), InContainer);

		// We will automatically inject our tokens to the take recorder naming tokens via our pre-evaluate hook.
		NamingTokenResultData = GetMetaDataChecked()->ProcessTokens(ResultText, nullptr);
		
		// Manually handle after all evaluation has finished. We don't need to listen for a post-evaluate event
		// because we want the finalized text processed for all tokens classes.
		// Build list of unidentified tokens, in-order of appearance. Include external tokens since they are being added by us.
		TArray<FString> UndefinedTokenKeys;
		for (const FNamingTokenValueData& Token : NamingTokenResultData.TokenValues)
		{
			if ((!Token.bWasEvaluated
				|| GetNamingTokensData()->UserDefinedTokens.Contains(FNamingTokenData(Token.TokenKey)))
					// Include pre-defined take and slate keys so they will get sorted with user tokens.
				|| (Token.TokenKey == UE::TakeRecorder::Private::TokenKeyTake
					|| Token.TokenKey == UE::TakeRecorder::Private::TokenKeySlate))
			{
				UndefinedTokenKeys.Add(Token.TokenKey);
			}
		}

		if (!FieldName.IsEmpty())
		{
			// Track the unidentified keys for this field.
			TArray<FString>& TokenKeysForField = GetNamingTokensData()->FindOrAddTokenKeysForField(FieldName);
			TokenKeysForField = UndefinedTokenKeys;
		}
	}

	// Cache the value if we're the TakeSaveDir, so it can be displayed in the UI as an example.
	if (InProperty->GetFName() == UE::TakeRecorder::Private::TakeSaveDirPropertyName)
	{
		CachedTakeSaveDirProperty = InProperty;
		CachedTakeSaveDirContainer = InContainer;
		GetNamingTokensData()->Modify();
		GetNamingTokensData()->EvaluatedTextValue = NamingTokenResultData.EvaluatedText;
	}
}

void STakeRecorderCockpit::CreateUserTokensUI()
{
	// Assemble the custom tokens in use. This step needs to happen after NamingTokensData->FieldToUndefinedKeys has
	// been fully populated so we don't remove custom tokens that are still in use.
	{
		const TMap<FNamingTokenData, FText> CustomUserTokensCopy = GetNamingTokensData()->UserDefinedTokens;
		for (const TTuple<FNamingTokenData, FText>& CustomToken : CustomUserTokensCopy)
		{
			// If no field holds a reference to this key then we can remove it.
			if (!GetNamingTokensData()->IsTokenKeyUndefined(CustomToken.Key.TokenKey))
			{
				// Only remove the visible entry, but leave the value in CustomUserTokens. If the user is switching
				// between objects it's possible one object has fields with different tokens than another, and we don't
				// want to lose the old entered data.
				GetNamingTokensData()->VisibleUserTokens.Remove(CustomToken.Key.TokenKey);
			}
		}

		// Look for new unidentified tokens that a user can define.
		TMap<FNamingTokenData, FText>& ExistingUserDefinedTokens = GetNamingTokensData()->UserDefinedTokens;
		TMap<FNamingTokenData, FText> ActiveUserTokens;
		for (const FTakeRecorderNamingTokensFieldMapping& UndefinedTokens : GetNamingTokensData()->FieldToUndefinedKeys)
		{
			for (const FString& UndefinedTokenStr : UndefinedTokens.UndefinedKeys)
			{
				bool bUserTokenAdded = false;
				FNamingTokenData UndefinedToken(UndefinedTokenStr);
				if (!ActiveUserTokens.Contains(UndefinedToken)
					&& UE::TakeRecorder::Private::CanTokenBeUserDefined(UndefinedTokenStr))
				{
					// Check for existence so we don't overwrite the user defined value.
					ActiveUserTokens.Add(UndefinedToken);
					bUserTokenAdded = true;
					
					// Restore any previous values.
					if (FText* ExistingTokenValue = ExistingUserDefinedTokens.Find(UndefinedToken))
					{
						ActiveUserTokens[UndefinedToken] = *ExistingTokenValue;
					}
				}
				if (bUserTokenAdded || UE::TakeRecorder::Private::IsExplicitlyOurToken(UndefinedToken.TokenKey))
				{
					GetNamingTokensData()->VisibleUserTokens.Add(UndefinedToken.TokenKey);
				}
			}
		}
		
		// Overwrite all user tokens with active tokens. This destroys any custom tokens not in use, preventing the token
		// suggestion dropdown from being cluttered with stale entries, since each letter typed technically adds a new token.
		ExistingUserDefinedTokens = MoveTemp(ActiveUserTokens);
	}

	// Make sure our take dir token example is up-to-date.
	EvaluateTakeSaveDirTokens();

	TSet<FString, FLocKeySetFuncs> UsedTokens;
	UserTokensBox->ClearChildren();
	
	// Display tokens in order from their field appearance, to the token location in the string, but do not duplicate.
	for (const FTakeRecorderNamingTokensFieldMapping& FieldMapping : GetNamingTokensData()->FieldToUndefinedKeys)
	{
		for (const FString& TokenString : FieldMapping.UndefinedKeys)
		{
			if (UsedTokens.Contains(TokenString) || !GetNamingTokensData()->VisibleUserTokens.Contains(TokenString))
			{
				// Skip if this was already reference in this field or a previous one.
				continue;
			}
			ensure(!TokenString.IsEmpty());

			UsedTokens.Add(TokenString);

			constexpr float MinWidth = 100.f;
			constexpr float MaxWidth = 175.f;

			// Take and Slate have special handling but still need to show up in the order the user referenced them.
			if (TokenString.Equals(UE::TakeRecorder::Private::TokenKeySlate))
			{
				UserTokensBox->AddSlot()
				.HAlign(HAlign_Fill)
				.MinWidth(MinWidth)
				.MaxWidth(MaxWidth)
				.AutoWidth()
				[
					SNew(SOverlay)

					+ SOverlay::Slot()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Left)
					.Padding(2.0f, 2.0f)
					[
						SNew(STextBlock)
						.TextStyle(FTakeRecorderStyle::Get(), "TakeRecorder.TextBox")
						.Text(LOCTEXT("SlateLabel", "slate"))
					]

					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(SEditableTextBox)
						.IsEnabled(this, &STakeRecorderCockpit::EditingMetaData)
						.Style(FTakeRecorderStyle::Get(), "TakeRecorder.EditableTextBox")
						.Font(FTakeRecorderStyle::Get().GetFontStyle("TakeRecorder.Cockpit.LargeText"))
						.HintText(LOCTEXT("EnterSlate_Hint", "<slate>"))
						.Justification(ETextJustify::Left)
						.SelectAllTextWhenFocused(true)
						.Text(this, &STakeRecorderCockpit::GetSlateText)
						.OnTextCommitted(this, &STakeRecorderCockpit::SetSlateText)
					]
				];
			}
			else if (TokenString.Equals(UE::TakeRecorder::Private::TokenKeyTake))
			{
				UserTokensBox->AddSlot()
				.HAlign(HAlign_Fill)
				.MinWidth(MinWidth)
				.MaxWidth(MaxWidth)
				.AutoWidth()
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Left)
					.Padding(2.0f, 2.0f)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.TextStyle(FTakeRecorderStyle::Get(), "TakeRecorder.TextBox")
							.Text(LOCTEXT("TakeLabel", "take"))
						]

						+ SHorizontalBox::Slot()
						.Padding(2.f, 0.f)
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "NoBorder")
							.OnClicked(this, &STakeRecorderCockpit::OnSetNextTakeNumber)
							.ForegroundColor(FSlateColor::UseForeground())
							.Visibility(this, &STakeRecorderCockpit::GetTakeWarningVisibility)
							.Content()
							[
								SNew(STextBlock)
								.ToolTipText(this, &STakeRecorderCockpit::GetTakeWarningText)
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.8"))
								.Text(FEditorFontGlyphs::Exclamation_Triangle)
							]
						]
					]

					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(SSpinBox<int32>)
						.IsEnabled(this, &STakeRecorderCockpit::EditingMetaData)
						.ContentPadding(FMargin(8.f, 0.f))
						.Style(FTakeRecorderStyle::Get(), "TakeRecorder.TakeInput")
						.Font(FTakeRecorderStyle::Get().GetFontStyle("TakeRecorder.Cockpit.GiantText"))
						.Justification(ETextJustify::Center)
						.Value(this, &STakeRecorderCockpit::GetTakeNumber)
						.Delta(1)
						.MinValue(1)
						.MaxValue(TOptional<int32>())
						.OnBeginSliderMovement(this, &STakeRecorderCockpit::OnBeginSetTakeNumber)
						.OnValueChanged(this, &STakeRecorderCockpit::SetTakeNumber)
						.OnValueCommitted(this, &STakeRecorderCockpit::SetTakeNumber_FromCommit)
						.OnEndSliderMovement(this, &STakeRecorderCockpit::OnEndSetTakeNumber)
						.TypeInterface(DigitsTypeInterface)
					]
				];
			}
			// Custom user keys
			else
			{
				UserTokensBox->AddSlot()
				.HAlign(HAlign_Fill)
				.MinWidth(MinWidth)
				.MaxWidth(MaxWidth)
				.AutoWidth()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(SOverlay)

						+ SOverlay::Slot()
						.VAlign(VAlign_Top)
						.HAlign(HAlign_Left)
						.Padding(2.0f, 2.0f)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(STextBlock)
								.TextStyle(FTakeRecorderStyle::Get(), "TakeRecorder.TextBox")
								.Text(FText::FromString(TokenString))
							]
						]
						+ SOverlay::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						[
							SNew(SEditableTextBox)
							.IsEnabled(this, &STakeRecorderCockpit::EditingMetaData)
							.Style(FTakeRecorderStyle::Get(), "TakeRecorder.EditableTextBox")
							.Font(FTakeRecorderStyle::Get().GetFontStyle("TakeRecorder.Cockpit.LargeText"))
							.HintText(LOCTEXT("EnterToken_Hint", "<value>"))
							.Justification(ETextJustify::Left)
							.SelectAllTextWhenFocused(true)
							.Text(this, &STakeRecorderCockpit::GetCustomTokenTextValue, TokenString)
							.OnTextCommitted(this, &STakeRecorderCockpit::SetCustomTokenTextValue, TokenString)
						]
					]
				];
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE