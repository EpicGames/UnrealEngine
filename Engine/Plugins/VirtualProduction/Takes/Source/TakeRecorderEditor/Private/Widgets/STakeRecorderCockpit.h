// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Recorder/TakeRecorderParameters.h"
#include "UObject/WeakFieldPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UTakeRecorderNamingTokensData;
class UTakeRecorderSubsystem;
enum class ECheckBoxState : uint8;
enum class ETakeRecordMode : uint8;

struct FFrameRate;
class ULevelSequence;
class UNamingTokens;
class UTakeMetaData;
class UTakeRecorder;
class SHorizontalBox;
class IDetailsView;
struct FDigitsTypeInterface;
struct FNamingTokenResultData;
struct FNamingTokensEvaluationData;
struct FPropertyChangedEvent;

/**
 * Cockpit UI for defining take meta-data.
 * Interacts with UTakeMetaData stored on the level sequence, if present, otherwise uses its own transient meta-data
 */
class STakeRecorderCockpit : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STakeRecorderCockpit)
		
		{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	~STakeRecorderCockpit();

	UTakeMetaData* GetMetaData() const;
	UTakeMetaData* GetMetaDataChecked() const;

	FFrameRate GetFrameRate() const;

	void SetFrameRate(FFrameRate InFrameRate, bool bFromTimecode);

	bool IsSameFrameRate(FFrameRate InFrameRate) const;

	bool Reviewing() const;

	bool Recording() const;

	TSharedRef<SWidget> MakeLockButton();

	bool CanStartRecording(FText& OutErrorText) const;

	void StartRecording();

	void StopRecording();

	void CancelRecording();

	void Refresh();

	/** Externally called when a property has been updated. */
	void NotifyPropertyUpdated(const FPropertyChangedEvent& InPropertyChangedEvent);
	/** Externally called when a details view has been added related to this take recorder. */
	void NotifyDetailsViewAdded(const TWeakPtr<IDetailsView>& InDetailsView);

private:
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void PostUndoRedo();
	
private:

	/** @return When Hitch Protection is enabled, the widget that warns the user when recorded at FPS mismatches the timecode provider's FPS */
	TSharedRef<SWidget> MakeHitchProtectionMismatchedFrameRateWarningIcon();

	EVisibility GetTakeWarningVisibility() const;
	FText GetTakeWarningText() const;

	EVisibility GetRecordErrorVisibility() const;
	FText GetRecordErrorText() const; 

	void UpdateRecordError();
	void UpdateTakeError();

	EVisibility GetCountdownVisibility() const;
	FText GetCountdownText() const;

	TSharedRef<SWidget> OnRecordingOptionsMenu();

	FText GetTimecodeText() const;

	FText GetUserDescriptionText() const;
	void SetUserDescriptionText(const FText& InNewText, ETextCommit::Type);

	FText GetFrameRateText() const;
	FText GetFrameRateTooltipText() const;
	bool IsFrameRateCompatible(FFrameRate InFrameRate) const;
	bool IsSetFromTimecode() const;

	FText GetSlateText() const;
	void SetSlateText(const FText& InNewText, ETextCommit::Type InCommitType);

	FText GetTimestampText() const;
	FText GetTimestampTooltipText() const;

	/** Fully evaluated text of the take save dir. */
	FText GetEvaluatedTakeSaveDirText() const;

	int32 GetTakeNumber() const;

	FReply OnSetNextTakeNumber();

	void OnBeginSetTakeNumber();

	void SetTakeNumber(int32 InNewTakeNumber);

	void SetTakeNumber_FromCommit(int32 InNewTakeNumber, ETextCommit::Type InCommitType);

	void OnEndSetTakeNumber(int32 InFinalValue);

	float GetEngineTimeDilation() const;

	void SetEngineTimeDilation(float InEngineTimeDilation);

	FReply OnAddMarkedFrame();

	void OnToggleRecording(ECheckBoxState);

	FReply NewRecordingFromThis();

	ECheckBoxState IsRecording() const;

	bool CanRecord() const;

	bool IsLocked() const;

	bool EditingMetaData() const;

	void BindCommands();

	void OnToggleEditPreviousRecording(ECheckBoxState CheckState);
	
	TSharedRef<SWidget> OnCreateMenu();

	/** Retrieve data relating to naming tokens. */
	UTakeRecorderNamingTokensData* GetNamingTokensData() const;
	
	/** Find a user defined text value for our custom tokens. */
	FText GetCustomTokenTextValue(FString InTokenKey) const;
	/** Set a user defined text value for a custom token. */
	void SetCustomTokenTextValue(const FText& InNewText, ETextCommit::Type InCommitType, FString InTokenKey);

	/** When a token value has been updated by the user, such as for slate, take, or custom tokens. */
	void OnTokenValueUpdated();

	/** Evaluate the take dir tokens for our example display. */
	void EvaluateTakeSaveDirTokens();
	
	/** Iterate all cached details views refreshing all undefined tokens. */
	void RefreshUndefinedTokens();
	
	/** Process all properties in the details view. */
	void UpdateUndefinedTokensFromDetailsView(const TWeakPtr<IDetailsView>& InDetailsView);

	/** Create and use temporary details view for key data for the case where no details views are available. */
	void UpdateUndefinedTokensFromTemporaryDetailsView();
	
	/**
	 * Evaluate naming tokens given a property.
	 * @param InProperty The property we need the value from.
	 * @param InContainer Container of the property which holds the value.
	 * @param bForce Whether to force an evaluation even if the property isn't flagged as a NamingToken. Property still must be a supported type.
	 */
	void EvaluateTokensFromProperty(const FProperty* InProperty, const void* InContainer, bool bForce = false);

	/** Configure any user tokens in the UI. */
	void CreateUserTokensUI();
	
private:

	/** The index of a pending transaction initiated by this widget, or INDEX_NONE if none is pending */
	int32 TransactionIndex;

	/** Text that describes why the user cannot record with the current settings */
	FText RecordErrorText;

	/** Text that describes why the user cannot record with the current settings */
	FText TakeErrorText;

	TSharedPtr<FDigitsTypeInterface> DigitsTypeInterface;

	/** All known detail views. */
	TArray<TWeakPtr<IDetailsView>> DetailViews;

	/** Details views we manage so we know what tokens are available in case the user has hidden the normal details. */
	TArray<TSharedPtr<IDetailsView>> TemporaryDetailsViews;
	
	/** Signal that undefined tokens should be refreshed on the next tick. */
	bool bRefreshUndefinedTokens = false;
	
	/** Tokens added by the user */
	TSharedPtr<SHorizontalBox> UserTokensBox;

	// Cached take numbers and slate used to UpdateTakeError() only when necessary
	int32 CachedTakeNumber;
	FString CachedTakeSlate;

	/** Our cached Take Save Dir Property. */
	TWeakFieldPtr<const FProperty> CachedTakeSaveDirProperty;
	/** Our cached object container, holding our Take Save Dir property value. */
	const void* CachedTakeSaveDirContainer = nullptr;

	/** Weak ptr to take recorder subsystem. */
	TWeakObjectPtr<UTakeRecorderSubsystem> TakeRecorderSubsystem;

	/** Handle for binding to the post undo/redo editor delegate. */
	FDelegateHandle PostUndoRedoDelegateHandle;
};
