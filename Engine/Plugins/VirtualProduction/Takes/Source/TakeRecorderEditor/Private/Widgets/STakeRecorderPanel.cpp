// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STakeRecorderPanel.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/TakeRecorderWidgetConstants.h"
#include "Widgets/SLevelSequenceTakeEditor.h"
#include "Recorder/TakeRecorder.h"
#include "Recorder/TakeRecorderPanel.h"
#include "ScopedSequencerPanel.h"
#include "ITakeRecorderModule.h"
#include "TakesCoreBlueprintLibrary.h"
#include "TakePreset.h"
#include "TakeMetaData.h"
#include "TakeRecorderSources.h"
#include "TakeRecorderStyle.h"
#include "TakeRecorderSettings.h"
#include "Widgets/STakeRecorderCockpit.h"
#include "LevelSequence.h"

// Core includes
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "Algo/Sort.h"
#include "Misc/FileHelper.h"

// AssetRegistry includes
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetData.h"

// ContentBrowser includes
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

// AssetTools includes
#include "AssetToolsModule.h"

// Slate includes
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/SlateIconFinder.h"

// Style includes
#include "Styling/AppStyle.h"
#include "EditorFontGlyphs.h"

// UnrealEd includes
#include "ScopedTransaction.h"
#include "FileHelpers.h"
#include "Misc/MessageDialog.h"

// Sequencer includes
#include "ISequencer.h"
#include "SequencerSettings.h"

#include "LevelEditor.h"
#include "Recorder/TakeRecorderSubsystem.h"
#include "TakePresetSettings.h"
#include "Editor/Transactor.h"
#include "Editor/TransBuffer.h"
#include "Misc/ITransaction.h"

#define LOCTEXT_NAMESPACE "STakeRecorderPanel"

STakeRecorderPanel::~STakeRecorderPanel()
{
	ITakeRecorderModule& TakeRecorderModule = FModuleManager::Get().LoadModuleChecked<ITakeRecorderModule>("TakeRecorder");
	TakeRecorderModule.OnForceSaveAsPreset().Unbind();
	TakeRecorderModule.GetExternalObjectAddRemoveEventDelegate().Remove(OnWidgetExternalObjectChangedHandle);
	
	if (TakeRecorderSubsystem.IsValid())
	{
		TakeRecorderSubsystem->GetOnRecordingInitializedEvent().Remove(OnRecordingInitializedHandle);
		TakeRecorderSubsystem->GetOnRecordingFinishedEvent().Remove(OnRecordingFinishedHandle);
		TakeRecorderSubsystem->GetOnRecordingCancelledEvent().Remove(OnRecordingCancelledHandle);
	}
}

UE_DISABLE_OPTIMIZATION_SHIP
void STakeRecorderPanel::Construct(const FArguments& InArgs)
{
	if (UTransBuffer* Transactor = GEditor ? Cast<UTransBuffer>(GEditor->Trans) : nullptr)
	{
		Transactor->OnBeforeRedoUndo().AddSP(this, &STakeRecorderPanel::OnBeforeRedoUndo);
	}

	TakeRecorderSubsystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>();
	
	FTakeRecorderSequenceParameters Data;
	Data.BasePreset = InArgs._BasePreset;
	Data.BaseSequence = InArgs._BaseSequence;
	Data.RecordIntoSequence = InArgs._RecordIntoSequence;
	Data.SequenceToView = InArgs._SequenceToView;
	
	TakeRecorderSubsystem->SetTargetSequence(Data);
	
	// Create the child widgets that need to know about our level sequence
	CockpitWidget = SNew(STakeRecorderCockpit);

	LevelSequenceTakeWidget = SNew(SLevelSequenceTakeEditor)
	.LevelSequence(this, &STakeRecorderPanel::GetLevelSequence)
	.OnDetailsPropertiesChanged(this, &STakeRecorderPanel::OnLevelSequenceDetailsChanged)
	.OnDetailsViewAdded(this, &STakeRecorderPanel::OnLevelSequenceDetailsViewAdded);

	// Create the sequencer panel, and open it if necessary
	SequencerPanel = MakeShared<FScopedSequencerPanel>(MakeAttributeSP(this, &STakeRecorderPanel::GetLevelSequence));

	// Bind onto the necessary delegates we need
	OnLevelSequenceChangedHandle = TakeRecorderSubsystem->GetTransientPreset()->AddOnLevelSequenceChanged(FSimpleDelegate::CreateSP(this, &STakeRecorderPanel::OnLevelSequenceChanged));
	OnRecordingInitializedHandle = TakeRecorderSubsystem->GetOnRecordingInitializedEvent().AddSP(this, &STakeRecorderPanel::OnRecordingInitialized);
	OnRecordingFinishedHandle = TakeRecorderSubsystem->GetOnRecordingFinishedEvent().AddSP(this, &STakeRecorderPanel::OnRecordingFinished);
	OnRecordingCancelledHandle = TakeRecorderSubsystem->GetOnRecordingCancelledEvent().AddSP(this, &STakeRecorderPanel::OnRecordingCancelled);

	ITakeRecorderModule& TakeRecorderModule = FModuleManager::Get().LoadModuleChecked<ITakeRecorderModule>("TakeRecorder");
	OnWidgetExternalObjectChangedHandle =
		TakeRecorderModule.GetExternalObjectAddRemoveEventDelegate().AddSP(this, &STakeRecorderPanel::ReconfigureExternalSettings);

	TakeRecorderModule.OnForceSaveAsPreset().BindRaw(this, &STakeRecorderPanel::OnSaveAsPreset);

	for(TWeakObjectPtr<> Object : TakeRecorderModule.GetExternalObjects())
	{
		if (Object.IsValid())
		{
			LevelSequenceTakeWidget->AddExternalSettingsObject(Object.Get());
		}
	}

	// Setup the preset origin for the meta-data in the cockpit if one was supplied
	if (InArgs._BasePreset)
	{
		CockpitWidget->GetMetaDataChecked()->SetPresetOrigin(InArgs._BasePreset);
	}

	// Add the settings immediately if the user preference tells us to
	UTakeRecorderUserSettings* UserSettings = GetMutableDefault<UTakeRecorderUserSettings>();
	UTakeRecorderProjectSettings* ProjectSettings = GetMutableDefault<UTakeRecorderProjectSettings>();
	if (UserSettings->bShowUserSettingsOnUI)
	{
		LevelSequenceTakeWidget->AddExternalSettingsObject(ProjectSettings);
		LevelSequenceTakeWidget->AddExternalSettingsObject(UserSettings);
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(FMargin(0.f, 1.0f))
		.AutoHeight()
		[
			MakeToolBar()
		]

		+ SVerticalBox::Slot()
		.Padding(FMargin(0.f, 1.0f))
		.AutoHeight()
		[
			CockpitWidget.ToSharedRef()
		]

		+ SVerticalBox::Slot()
		.Padding(FMargin(0.f, 1.0f, 0.f, 0.f))
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
			.BorderBackgroundColor( FLinearColor( .6f, .6f, .6f, 1.0f ) )
			.IsEnabled_Lambda([this]() { return !CockpitWidget->Reviewing() && !CockpitWidget->Recording(); })
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(TakeRecorder::ButtonOffset)
				.VAlign(VAlign_Fill)
				.AutoWidth()
				[
					LevelSequenceTakeWidget->MakeAddSourceButton()
				]

				+ SHorizontalBox::Slot()
				.Padding(TakeRecorder::ButtonOffset)
				.VAlign(VAlign_Fill)
				.AutoWidth()
				[
					SNew(SComboButton)
					.ContentPadding(TakeRecorder::ButtonPadding)
					.ComboButtonStyle(FTakeRecorderStyle::Get(), "ComboButton")
					.OnGetMenuContent(this, &STakeRecorderPanel::OnGeneratePresetsMenu)
					.ForegroundColor(FSlateColor::UseForeground())
					.ButtonContent()
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FSlateIconFinder::FindIconBrushForClass(UTakePreset::StaticClass()))
						]

						+ SHorizontalBox::Slot()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PresetsToolbarButton", "Presets"))
						]
					]
				]

				+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
				]

				+ SHorizontalBox::Slot()
				.Padding(TakeRecorder::ButtonOffset)
				.VAlign(VAlign_Fill)
				.AutoWidth()
				[
					SNew(SButton)
					.ContentPadding(TakeRecorder::ButtonPadding)
					.ToolTipText(LOCTEXT("RevertChanges_Text", "Revert all changes made to this take back its original state (either its original preset, or an empty take)."))
					.ForegroundColor(FSlateColor::UseForeground())
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.OnClicked(this, &STakeRecorderPanel::OnRevertChanges)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
						.Text(FEditorFontGlyphs::Undo)
					]
				]
			]
		]

		+ SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)
			.IsEnabled_Lambda([this]() { return !CockpitWidget->Recording(); })
			+ SHorizontalBox::Slot()
			[
				LevelSequenceTakeWidget.ToSharedRef()
			]
		]
	];
}

TSharedRef<SWidget> STakeRecorderPanel::MakeToolBar()
{
	int32 ButtonBoxSize = 28;
	TSharedPtr<SHorizontalBox> ButtonHolder;

	TSharedRef<SBorder> Border = SNew(SBorder)
	.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
	.Padding(FMargin(3.f, 3.f))
	[

		SAssignNew(ButtonHolder, SHorizontalBox)

		+ SHorizontalBox::Slot()
		.Padding(TakeRecorder::ButtonOffset)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(ButtonBoxSize)
			.HeightOverride(ButtonBoxSize)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ToolTipText(LOCTEXT("ClearPendingTake", "Clear pending take"))
				.ForegroundColor(FSlateColor::UseForeground())
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &STakeRecorderPanel::OnClearPendingTake)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
					.Text(FEditorFontGlyphs::File)
				]
			]
		]

		+ SHorizontalBox::Slot()
		.Padding(TakeRecorder::ButtonOffset)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[

			SNew(SBox)
			.WidthOverride(ButtonBoxSize)
			.HeightOverride(ButtonBoxSize)
			.Visibility_Lambda([this]() { return !CockpitWidget->Reviewing() ? EVisibility::Visible : EVisibility::Collapsed; })
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ContentPadding(TakeRecorder::ButtonPadding)
				.ToolTipText(LOCTEXT("ReviewLastRecording", "Review the last recording"))
				.ForegroundColor(FSlateColor::UseForeground())
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.IsEnabled_Lambda([this]() { return (CanReviewLastLevelSequence() && GetTakeRecorderMode() == ETakeRecorderMode::RecordNewSequence); })
				.OnClicked(this, &STakeRecorderPanel::OnReviewLastRecording)
				[
					SNew(SImage)
					.Image(FTakeRecorderStyle::Get().GetBrush("TakeRecorder.ReviewRecordingButton"))
				]
			]
		]

		+ SHorizontalBox::Slot()
		.Padding(TakeRecorder::ButtonOffset)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(ButtonBoxSize)
			.HeightOverride(ButtonBoxSize)
			.Visibility_Lambda([this]() { return CockpitWidget->Reviewing() ? EVisibility::Visible : EVisibility::Collapsed; })
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ContentPadding(TakeRecorder::ButtonPadding)
				.ToolTipText(LOCTEXT("Back", "Return back to the pending take"))
				.ForegroundColor(FSlateColor::UseForeground())
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &STakeRecorderPanel::OnBackToPendingTake)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
					.Text(FEditorFontGlyphs::Arrow_Left)
				]
			]
		]

		+ SHorizontalBox::Slot()
		.Padding(TakeRecorder::ButtonOffset)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(ButtonBoxSize)
			.HeightOverride(ButtonBoxSize)
			.Visibility_Lambda([this]() { return !CockpitWidget->Reviewing() ? EVisibility::Visible : EVisibility::Collapsed; })
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(ButtonBoxSize)
					.HeightOverride(ButtonBoxSize)
					[
						SNew(SCheckBox)
						.ToolTipText_Lambda([this]() { return GetTakeRecorderMode() == ETakeRecorderMode::RecordIntoSequence ? LOCTEXT("RecordIntoSequenceTooltip", "Recording directly into chosen sequence") : LOCTEXT("RecordFromPendingTakeTooltip", "Recording from pending take. To record into an existing sequence, choose a sequence to record into"); })
						.Style(FTakeRecorderStyle::Get(), "ToggleButtonIndicatorCheckbox")
						.IsChecked_Lambda([this]() { return GetTakeRecorderMode() == ETakeRecorderMode::RecordIntoSequence ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
						[
							SNew(SImage)
							.Image(FTakeRecorderStyle::Get().GetBrush("TakeRecorder.SequenceToRecordIntoButton"))
						]
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2)
				[
					SNew(SComboButton)
					.ContentPadding(2)
					.ForegroundColor(FSlateColor::UseForeground())
					.ComboButtonStyle(FTakeRecorderStyle::Get(), "ComboButton")
					.ToolTipText(LOCTEXT("OpenSequenceToRecordIntoTooltip", "Open sequence to record into"))
					.OnGetMenuContent(this, &STakeRecorderPanel::OnOpenSequenceToRecordIntoMenu)
					.HasDownArrow(false)
					.ButtonContent()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "NormalText.Important")
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
						.Text(FEditorFontGlyphs::Caret_Down)
					]
				]
			]
		]

		+ SHorizontalBox::Slot()
		[
			SNew(SSpacer)
		]

		+ SHorizontalBox::Slot()
		.Padding(TakeRecorder::ButtonOffset)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(ButtonBoxSize)
			.HeightOverride(ButtonBoxSize)
			[
				SNew(SCheckBox)
				.Padding(TakeRecorder::ButtonPadding)
				.ToolTipText(NSLOCTEXT("TakesBrowser", "ToggleTakeBrowser_Tip", "Show/Hide the Takes Browser"))
				.Style(FTakeRecorderStyle::Get(), "ToggleButtonCheckbox")
				.IsChecked(this, &STakeRecorderPanel::GetTakeBrowserCheckState)
				.OnCheckStateChanged(this, &STakeRecorderPanel::ToggleTakeBrowserCheckState)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
					.Text(FEditorFontGlyphs::Folder_Open)
				]
			]
		]

		+ SHorizontalBox::Slot()
		.Padding(TakeRecorder::ButtonOffset)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
				.WidthOverride(ButtonBoxSize)
				.HeightOverride(ButtonBoxSize)
				[
					MakeSimpleViewToggleButton()
				]
		]

		+ SHorizontalBox::Slot()
		.Padding(TakeRecorder::ButtonOffset)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(ButtonBoxSize)
			.HeightOverride(ButtonBoxSize)
			[

				SequencerPanel->MakeToggleButton()
			]
		]

		+ SHorizontalBox::Slot()
		.Padding(TakeRecorder::ButtonOffset)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(ButtonBoxSize)
			.HeightOverride(ButtonBoxSize)
			.Visibility_Lambda([this]() { return CockpitWidget->Reviewing() ? EVisibility::Visible : EVisibility::Collapsed; })
			[
				CockpitWidget->MakeLockButton()
			]
		]

		+ SHorizontalBox::Slot()
		.Padding(TakeRecorder::ButtonOffset)
		.VAlign(VAlign_Fill)
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(ButtonBoxSize)
			.HeightOverride(ButtonBoxSize)
			[
				SNew(SCheckBox)
				.Padding(TakeRecorder::ButtonPadding)
				.ToolTipText(LOCTEXT("ShowSettings_Tip", "Show/Hide the general user/project settings for Take Recorder"))
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ForegroundColor(FSlateColor::UseForeground())
				.IsChecked(this, &STakeRecorderPanel::GetSettingsCheckState)
				.OnCheckStateChanged(this, &STakeRecorderPanel::ToggleSettings)
				.Visibility_Lambda([this]() { return !CockpitWidget->Reviewing() ? EVisibility::Visible : EVisibility::Collapsed; })
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
					.Text(FEditorFontGlyphs::Cogs)
				]
			]
		]
	];

	ITakeRecorderModule& TakeRecorderModule = FModuleManager::Get().LoadModuleChecked<ITakeRecorderModule>("TakeRecorder");
	TArray<TSharedRef<SWidget>> OutExtensions;
	TakeRecorderModule.GetToolbarExtensionGenerators().Broadcast(OutExtensions);

	for (const TSharedRef<SWidget>& Widget : OutExtensions)
	{
		ButtonHolder->AddSlot()
		.Padding(TakeRecorder::ButtonOffset)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
			.HeightOverride(ButtonBoxSize)
			[
				Widget
			]
		];
	}

	return Border;
}
UE_ENABLE_OPTIMIZATION_SHIP

TSharedRef<SWidget> STakeRecorderPanel::MakeSimpleViewToggleButton()
{
	return SNew(SCheckBox)
		.Padding(TakeRecorder::ButtonPadding)
		.ToolTipText(NSLOCTEXT("TakeRecorder", "ToggleSequencerSimpleView_Tip", "Auto collapse to simplified sequencer while recording."))
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.IsEnabled_Lambda([this] 
			{ 
				if (TStrongObjectPtr<UTakeRecorderSubsystem> TRSubsystem = TakeRecorderSubsystem.Pin())
				{
					return !TRSubsystem->IsRecording();
				}
				// Default to enabled if no subsystem is present.
				return true; 
			})
		.IsChecked_Lambda([]()
			{
				if (const UTakeRecorderProjectSettings* Settings = GetDefault<UTakeRecorderProjectSettings>())
				{
					return Settings->Settings.bRecordWithSimpleSequencerView ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}
				return ECheckBoxState::Unchecked;
			})
		.OnCheckStateChanged_Lambda([](ECheckBoxState InNewState)
			{
				if (UTakeRecorderProjectSettings* Settings = GetMutableDefault<UTakeRecorderProjectSettings>())
				{
					Settings->Settings.bRecordWithSimpleSequencerView = InNewState == ECheckBoxState::Checked;
					Settings->SaveConfig();
				}
			})
		[
			SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Sequencer.SimpleView.ToggleToSimpleView"))
		];
}

ULevelSequence* STakeRecorderPanel::GetLevelSequence() const
{
	return TakeRecorderSubsystem->GetLevelSequence();
}

ULevelSequence* STakeRecorderPanel::GetLastRecordedLevelSequence() const
{
	return TakeRecorderSubsystem->GetLastRecordedLevelSequence();
}

bool STakeRecorderPanel::CanReviewLastLevelSequence() const
{
	return TakeRecorderSubsystem->CanReviewLastRecording();
}

ETakeRecorderMode STakeRecorderPanel::GetTakeRecorderMode() const
{
	return TakeRecorderSubsystem->GetTakeRecorderMode();
}

UTakeMetaData* STakeRecorderPanel::GetTakeMetaData() const
{
	return CockpitWidget->GetMetaDataChecked();
}

void STakeRecorderPanel::ClearPendingTake()
{
	TakeRecorderSubsystem->ClearPendingTake();
}

void STakeRecorderPanel::ResetToPendingTake()
{
	TakeRecorderSubsystem->ResetToPendingTake();
	RefreshPanel();
}

TOptional<ETakeRecorderPanelMode> STakeRecorderPanel::GetMode() const
{
	if (TakeRecorderSubsystem->GetSuppliedLevelSequence())
	{
		return ETakeRecorderPanelMode::ReviewingRecording;	
	}
	else if (TakeRecorderSubsystem->GetRecordingLevelSequence())
	{
		return ETakeRecorderPanelMode::NewRecording;	
	}
	else if (TakeRecorderSubsystem->GetRecordIntoLevelSequence())
	{
		return ETakeRecorderPanelMode::RecordingInto;	
	}

	return TOptional<ETakeRecorderPanelMode>();
}

TSharedRef<SWidget> STakeRecorderPanel::OnGeneratePresetsMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SaveAsPreset_Text", "Save As Preset"),
		LOCTEXT("SaveAsPreset_Tip", "Save the current setup as a new preset that can be imported at a later date"),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAsset"),
		FUIAction(
			FExecuteAction::CreateSP(this, &STakeRecorderPanel::OnSaveAsPreset)
		)
	);

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bShowBottomToolbar = true;
		AssetPickerConfig.bAutohideSearchBar = false;
		AssetPickerConfig.bAllowDragging = false;
		AssetPickerConfig.bCanShowClasses = false;
		AssetPickerConfig.bShowPathInColumnView = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.bSortByPathInColumnView = false;

		AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoPresets_Warning", "No Presets Found");
		AssetPickerConfig.Filter.ClassPaths.Add(UTakePreset::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &STakeRecorderPanel::OnImportPreset);
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ImportPreset_MenuSection", "Import Preset"));
	{
		TSharedRef<SWidget> PresetPicker = SNew(SBox)
			.MinDesiredWidth(400.f)
			.MinDesiredHeight(400.f)
			[
				ContentBrowser.CreateAssetPicker(AssetPickerConfig)
			];

		MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


void STakeRecorderPanel::OnImportPreset(const FAssetData& InPreset)
{
	FSlateApplication::Get().DismissAllMenus();

	TakeRecorderSubsystem->ImportPreset(InPreset);
}


static bool OpenSaveDialog(const FString& InDefaultPath, const FString& InNewNameSuggestion, FString& OutPackageName)
{
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	{
		SaveAssetDialogConfig.DefaultPath = InDefaultPath;
		SaveAssetDialogConfig.DefaultAssetName = InNewNameSuggestion;
		SaveAssetDialogConfig.AssetClassNames.Add(UTakePreset::StaticClass()->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveTakePresetDialogTitle", "Save Take Preset");
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (!SaveObjectPath.IsEmpty())
	{
		OutPackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
		return true;
	}

	return false;
}


bool STakeRecorderPanel::GetSavePresetPackageName(FString& OutName)
{
	UTakeRecorderUserSettings* ConfigSettings = GetMutableDefault<UTakeRecorderUserSettings>();

	FDateTime Today = FDateTime::Now();

	TMap<FString, FStringFormatArg> FormatArgs;
	FormatArgs.Add(TEXT("date"), Today.ToString());

	// determine default package path
	const FString DefaultSaveDirectory = FString::Format(*ConfigSettings->GetResolvedPresetSaveDir(), FormatArgs);

	FString DialogStartPath;
	FPackageName::TryConvertFilenameToLongPackageName(DefaultSaveDirectory, DialogStartPath);
	if (DialogStartPath.IsEmpty())
	{
		DialogStartPath = TEXT("/Game");
	}

	// determine default asset name
	FString DefaultName = LOCTEXT("NewTakePreset", "NewTakePreset").ToString();

	FString UniquePackageName;
	FString UniqueAssetName;

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(DialogStartPath / DefaultName, TEXT(""), UniquePackageName, UniqueAssetName);

	FString DialogStartName = FPaths::GetCleanFilename(UniqueAssetName);

	FString UserPackageName;
	FString NewPackageName;

	// get destination for asset
	bool bFilenameValid = false;
	while (!bFilenameValid)
	{
		if (!OpenSaveDialog(DialogStartPath, DialogStartName, UserPackageName))
		{
			return false;
		}

		NewPackageName = FString::Format(*UserPackageName, FormatArgs);

		FText OutError;
		bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError);
	}

	ConfigSettings->SetPresetSaveDir(FPackageName::GetLongPackagePath(UserPackageName));
	ConfigSettings->SaveConfig();
	OutName = MoveTemp(NewPackageName);
	return true;
}

void STakeRecorderPanel::OnSaveAsPreset()
{
	FString PackageName;
	if (!GetSavePresetPackageName(PackageName))
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SaveAsPreset", "Save As Preset"));

	// Saving into a new package
	const FString NewAssetName = FPackageName::GetLongPackageAssetName(PackageName);
	UPackage*     NewPackage   = CreatePackage(*PackageName);
	UTakePreset*  NewPreset    = NewObject<UTakePreset>(NewPackage, *NewAssetName, RF_Public | RF_Standalone | RF_Transactional);

	if (NewPreset)
	{
		UTakePreset* TransientPreset = TakeRecorderSubsystem->GetTransientPreset();
		
		NewPreset->CopyFrom(TransientPreset);
		if (ULevelSequence* LevelSequence = NewPreset->GetLevelSequence())
		{
			// Ensure no take meta data is saved with this preset
			LevelSequence->RemoveMetaData<UTakeMetaData>();
		}

		NewPreset->MarkPackageDirty();
		// Clear the package dirty flag on the transient preset since it was saved.
		TransientPreset->GetOutermost()->SetDirtyFlag(false);
		FAssetRegistryModule::AssetCreated(NewPreset);

		FEditorFileUtils::PromptForCheckoutAndSave({ NewPackage }, false, false);

		TakeRecorderSubsystem->GetTakeMetaData()->SetPresetOrigin(NewPreset);
	}
}

FReply STakeRecorderPanel::OnBackToPendingTake()
{
	ResetToPendingTake();
	return FReply::Handled();
}

FReply STakeRecorderPanel::OnClearPendingTake()
{
	FText WarningMessage (LOCTEXT("Warning_ClearPendingTake", "Are you sure you want to clear the pending take? Your current tracks will be discarded."));
	if (EAppReturnType::No == FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage))
	{
		return FReply::Handled();
	}

	ClearPendingTake();
	return FReply::Handled();
}

FReply STakeRecorderPanel::OnReviewLastRecording()
{
	if (TakeRecorderSubsystem->ReviewLastRecording())
	{
		RefreshPanel();
	}

	return FReply::Handled();
}

void STakeRecorderPanel::OnTakePresetSettingsChanged()
{
	// Settings may have been changed by undo / redo... in that case undo / redo will have already updated the internal level sequence.
	if (GIsTransacting)
	{
		// OnBeforeRedoUndo has closed the panel - we'll now reinitialize it with the level sequence.
		RefreshPanel();
	}
}

void STakeRecorderPanel::OnBeforeRedoUndo(const FTransactionContext& TransactionContext) const
{
	UTransBuffer* Transactor = GEditor ? Cast<UTransBuffer>(GEditor->Trans) : nullptr;
	if (!Transactor)
	{
		return;
	}
	
	const int32 Index = Transactor->FindTransactionIndex(TransactionContext.TransactionId);
	const FTransaction* Transaction = Transactor->GetTransaction(Index);
	const bool bAffectsSettings = Transaction && Transaction->ContainsObject(UTakePresetSettings::Get());

	// Sequencer does not deal well with the underlying ULevelSequence class changing, i.e. when UTakePresetSettings::RecordTargetClass is changed.
	// Checks & ensures fly. After undo / redo, it seems like FSharedPlaybackState::WeakRootSequence is not transacted properly.
	// The proper fix would be to find out why that happens. However, right now it's easier to reinitialize sequencer by closing the tab and
	// reopening it post undo in OnTakePresetSettingsChanged.
	if (bAffectsSettings)
	{
		SequencerPanel->Close();
	}
}

FReply STakeRecorderPanel::OnRevertChanges()
{
	FText WarningMessage(LOCTEXT("Warning_RevertChanges", "Are you sure you want to revert changes? Your current changes will be discarded."));
	if (EAppReturnType::No == FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage))
	{
		return FReply::Handled();
	}

	TakeRecorderSubsystem->RevertChanges();

	return FReply::Handled();
}

TSharedRef<SWidget> STakeRecorderPanel::OnOpenSequenceToRecordIntoMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bShowBottomToolbar = true;
		AssetPickerConfig.bAutohideSearchBar = false;
		AssetPickerConfig.bAllowDragging = false;
		AssetPickerConfig.bCanShowClasses = false;
		AssetPickerConfig.bShowPathInColumnView = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.bSortByPathInColumnView = false;
		AssetPickerConfig.ThumbnailScale = 0.3f;
		AssetPickerConfig.SaveSettingsName = TEXT("TakeRecorderOpenSequenceToRecordInto");

		AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoSequences_Warning", "No Level Sequences Found");
		AssetPickerConfig.Filter.ClassPaths.Add(ULevelSequence::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &STakeRecorderPanel::OnOpenSequenceToRecordInto);
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("OpenSequenceToRecordInto", "Open Sequence to Record Into"));
	{
		TSharedRef<SWidget> PresetPicker = SNew(SBox)
			.WidthOverride(300.f)
			.HeightOverride(300.f)
			[
				ContentBrowser.CreateAssetPicker(AssetPickerConfig)
			];

		MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void STakeRecorderPanel::OnOpenSequenceToRecordInto(const FAssetData& InAsset)
{
	// Close the dropdown menu that showed them the assets to pick from.
	FSlateApplication::Get().DismissAllMenus();

	// Only try to initialize level sequences, in the event they had more than a level sequence selected when drag/dropping.
	if (ULevelSequence* LevelSequence = Cast<ULevelSequence>(InAsset.GetAsset()))
	{
		TakeRecorderSubsystem->SetRecordIntoLevelSequence(LevelSequence);
		RefreshPanel();
	}
}

void STakeRecorderPanel::RefreshPanel()
{
	// Re-open the sequencer panel for the new level sequence if it should be
	if (GetDefault<UTakeRecorderUserSettings>()->bIsSequenceOpen)
	{
		SequencerPanel->Open();
	}
}

ECheckBoxState STakeRecorderPanel::GetSettingsCheckState() const
{
	return GetDefault<UTakeRecorderUserSettings>()->bShowUserSettingsOnUI ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


void STakeRecorderPanel::ToggleSettings(ECheckBoxState CheckState)
{
	UTakeRecorderUserSettings* UserSettings = GetMutableDefault<UTakeRecorderUserSettings>();
	UTakeRecorderProjectSettings* ProjectSettings = GetMutableDefault<UTakeRecorderProjectSettings>();

	if (LevelSequenceTakeWidget->RemoveExternalSettingsObject(UserSettings))
	{
		LevelSequenceTakeWidget->RemoveExternalSettingsObject(ProjectSettings);
		UserSettings->bShowUserSettingsOnUI = false;
	}
	else
	{
		LevelSequenceTakeWidget->AddExternalSettingsObject(ProjectSettings);
		LevelSequenceTakeWidget->AddExternalSettingsObject(UserSettings);
		UserSettings->bShowUserSettingsOnUI = true;
	}

	UserSettings->SaveConfig();
}

void STakeRecorderPanel::OnLevelSequenceChanged()
{
	RefreshPanel();
}

void STakeRecorderPanel::OnLevelSequenceDetailsChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (CockpitWidget.IsValid())
	{
		CockpitWidget->NotifyPropertyUpdated(InPropertyChangedEvent);
	}
}

void STakeRecorderPanel::OnLevelSequenceDetailsViewAdded(const TWeakPtr<IDetailsView>& InDetailsView)
{
	if (CockpitWidget.IsValid())
	{
		CockpitWidget->NotifyDetailsViewAdded(InDetailsView);
	}
}

void STakeRecorderPanel::OnRecordingInitialized(UTakeRecorder* Recorder)
{
	// It's important that UTakeRecorderEditorSubsystem::OnRecordingInitialized has fired before this point,
	// otherwise the refresh may invalidate a weak sequencer the subsystem relies on.
	RefreshPanel();
}

void STakeRecorderPanel::OnRecordingFinished(UTakeRecorder* Recorder)
{
	OnRecordingCancelled(Recorder);
}

void STakeRecorderPanel::OnRecordingCancelled(UTakeRecorder* Recorder)
{
	RefreshPanel();
	CockpitWidget->Refresh();
}

ECheckBoxState STakeRecorderPanel::GetTakeBrowserCheckState() const
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<SDockTab> TakesBrowserTab = LevelEditorModule.GetLevelEditorTabManager()->FindExistingLiveTab(ITakeRecorderModule::TakesBrowserTabName);
	if (TakesBrowserTab.IsValid())
	{
		return TakesBrowserTab->IsForeground() ? ECheckBoxState::Checked : ECheckBoxState::Undetermined;
	}
	return ECheckBoxState::Unchecked;
}

void STakeRecorderPanel::ToggleTakeBrowserCheckState(ECheckBoxState CheckState)
{
	// If it is up, but not visible, then bring it forward
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<SDockTab> TakesBrowserTab = LevelEditorModule.GetLevelEditorTabManager()->FindExistingLiveTab(ITakeRecorderModule::TakesBrowserTabName);
	if (TakesBrowserTab.IsValid())
	{
		if (!TakesBrowserTab->IsForeground())
		{
			TakesBrowserTab->ActivateInParent(ETabActivationCause::SetDirectly);
			TakesBrowserTab->FlashTab();
		}
		else
		{
			TakesBrowserTab->RequestCloseTab();
		}
	}
	else 
	{
		TakesBrowserTab = LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(ITakeRecorderModule::TakesBrowserTabName);

		bool bAllowLockedBrowser =  true;
		bool bFocusContentBrowser = false;

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName);
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		FString TakesPath;
		if (GetTakeMetaData()->TryGenerateRootAssetPath(GetDefault<UTakeRecorderProjectSettings>()->Settings.GetTakeAssetPath(), TakesPath))
		{
			TakesPath = FPaths::GetPath(*TakesPath);

			while(!TakesPath.IsEmpty())
			{
				if (AssetRegistry.HasAssets(FName(*TakesPath), true))
				{
					break;
				}
				TakesPath = FPaths::GetPath(TakesPath);
			}

			TArray<FString> TakesFolder;
			TakesFolder.Push(TakesPath);
			if (AssetRegistry.HasAssets(FName(*TakesPath), true) )
			{
				FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().SyncBrowserToFolders(TakesFolder, bAllowLockedBrowser, bFocusContentBrowser, ITakeRecorderModule::TakesBrowserInstanceName );
			}
		}

		TakesBrowserTab->FlashTab();
	}
}

void STakeRecorderPanel::ReconfigureExternalSettings(UObject* InExternalObject, bool bIsAdd)
{
	if (LevelSequenceTakeWidget.IsValid())
	{
		if (bIsAdd)
		{
			LevelSequenceTakeWidget->AddExternalSettingsObject(InExternalObject);
		}
		else
		{
			LevelSequenceTakeWidget->RemoveExternalSettingsObject(InExternalObject);
		}
	}
}

#undef LOCTEXT_NAMESPACE
