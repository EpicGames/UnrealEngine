// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "SLevelSequenceTakeEditor.h"
#include "UObject/GCObject.h"
#include "Recorder/TakeRecorderParameters.h"

enum class ECheckBoxState : uint8;
enum class ETakeRecorderMode : uint8;
enum class ETakeRecorderPanelMode : uint8;

struct FAssetData;
struct ITakeRecorderSourceTreeItem;
struct FScopedSequencerPanel;

class SScrollBox;
class UTakePreset;
class IDetailsView;
class UTakeMetaData;
class UTakeRecorder;
class ULevelSequence;
class UTakeRecorderSource;
class STakeRecorderCockpit;
class SLevelSequenceTakeEditor;
class UTakeRecorderSubsystem;

/**
 * Outermost widget that is used for setting up a new take recording. Operates on a transient UTakePreset that is internally owned and maintained 
 */
class STakeRecorderPanel : public SCompoundWidget
{
public:

	~STakeRecorderPanel();

	SLATE_BEGIN_ARGS(STakeRecorderPanel)
		: _BasePreset(nullptr)
		, _BaseSequence(nullptr)
		, _RecordIntoSequence(nullptr)
		, _SequenceToView(nullptr)
		{}

		/*~ All following arguments are mutually-exclusive */
		/*-------------------------------------------------*/
		/** A preset asset to base the recording off */
		SLATE_ARGUMENT(UTakePreset*, BasePreset)

		/** A level sequence asset to base the recording off */
		SLATE_ARGUMENT(ULevelSequence*, BaseSequence)

		/** A level sequence asset to record into */
		SLATE_ARGUMENT(ULevelSequence*, RecordIntoSequence)

		/** A sequence that should be shown directly on the take recorder UI */
		SLATE_ARGUMENT(ULevelSequence*, SequenceToView)
		/*-------------------------------------------------*/

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	ULevelSequence* GetLevelSequence() const;

	ULevelSequence* GetLastRecordedLevelSequence() const;

	ETakeRecorderMode GetTakeRecorderMode() const;

	UTakeMetaData* GetTakeMetaData() const;

	TSharedPtr<STakeRecorderCockpit> GetCockpitWidget() const { return CockpitWidget; }

	void ClearPendingTake();
	void ResetToPendingTake();

	TOptional<ETakeRecorderPanelMode> GetMode() const;

private:

	/**
	 * Refresh this panel after a change to its preset or levelsequence
	 */
	void RefreshPanel();

	/**
	 * Prompt for a package name to save the current setup as a preset
	 */
	bool GetSavePresetPackageName(FString& OutName);

private:

	TSharedRef<SWidget> OnGeneratePresetsMenu();

	void OnImportPreset(const FAssetData& InPreset);

	void OnSaveAsPreset();

	FReply OnRevertChanges();

	FReply OnBackToPendingTake();

	FReply OnClearPendingTake();

	FReply OnReviewLastRecording();

	/** Handles the UTakePresetSettings::RecordTargetClass changing. Recreates the object if recording a transaction or refreshes sequencer if undo / redoing. */
	void OnTakePresetSettingsChanged();
	/** If the transaction changes the UTakePresetSettings::RecordTargetClass, temporarily closes the sequencer in order to refresh it. */
	void OnBeforeRedoUndo(const FTransactionContext& TransactionContext) const;

	TSharedRef<SWidget> OnOpenSequenceToRecordIntoMenu();

	void OnOpenSequenceToRecordInto(const FAssetData& InAsset);

	ECheckBoxState GetSettingsCheckState() const;
	void ToggleSettings(ECheckBoxState CheckState);

	void OnLevelSequenceChanged();
	/** When properties in a level sequence details have been changed. */
	void OnLevelSequenceDetailsChanged(const FPropertyChangedEvent& InPropertyChangedEvent);
	/** When a level sequence has a details view added. */
	void OnLevelSequenceDetailsViewAdded(const TWeakPtr<IDetailsView>& InDetailsView);

	ECheckBoxState GetTakeBrowserCheckState() const;
	void ToggleTakeBrowserCheckState(ECheckBoxState CheckState);

private:
	void ReconfigureExternalSettings(UObject* InExternalObject, bool bIsAdd);

	void OnRecordingInitialized(UTakeRecorder* Recorder);

	void OnRecordingFinished(UTakeRecorder* Recorder);

	void OnRecordingCancelled(UTakeRecorder* Recorder);

	TSharedRef<SWidget> MakeToolBar();

	TSharedRef<SWidget> MakeSimpleViewToggleButton();

private:
	bool CanReviewLastLevelSequence() const;

	/** Weak ptr to subsystem. */
	TWeakObjectPtr<UTakeRecorderSubsystem> TakeRecorderSubsystem;

	/** The main level sequence take editor widget */
	TSharedPtr<SLevelSequenceTakeEditor> LevelSequenceTakeWidget;
	/** The recorder cockpit */
	TSharedPtr<STakeRecorderCockpit> CockpitWidget;
	/** Scoped panel that handles opening and closing the sequencer pane for this preset */
	TSharedPtr<FScopedSequencerPanel> SequencerPanel;

	FDelegateHandle OnWidgetExternalObjectChangedHandle;
	FDelegateHandle OnLevelSequenceChangedHandle;

	FDelegateHandle OnRecordingInitializedHandle, OnRecordingFinishedHandle, OnRecordingCancelledHandle;
};
