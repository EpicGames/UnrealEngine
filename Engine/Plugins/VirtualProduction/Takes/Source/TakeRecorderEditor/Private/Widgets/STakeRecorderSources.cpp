// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STakeRecorderSources.h"
#include "TakeRecorderSource.h"
#include "TakeRecorderSources.h"
#include "TakeRecorderStyle.h"
#include "ITakeRecorderDropHandler.h"
#include "Recorder/TakeRecorderBlueprintLibrary.h"

#include "TakeMetaData.h"
#include "LevelSequence.h"

// Core includes
#include "Algo/Sort.h"
#include "UObject/UObjectIterator.h"

// AssetRegistry includes
#include "AssetRegistry/AssetRegistryModule.h"

// Slate includes
#include "Widgets/SBoxPanel.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Commands/GenericCommands.h"
#include "Recorder/TakeRecorderSubsystem.h"

// EditorWidgets includes
#include "SDropTarget.h"

// UnrealEd includes
#include "DragAndDrop/ActorDragDropOp.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "STakeRecorderSources"

struct FTakeRecorderSourceCategory;
struct FTakeRecorderSourceTreeItem;

struct ITakeRecorderSourceTreeItem : TSharedFromThis<ITakeRecorderSourceTreeItem>
{
	virtual ~ITakeRecorderSourceTreeItem() {}

	virtual void Delete(UTakeRecorderSources* Owner) = 0;

	virtual TSharedPtr<FTakeRecorderSourceCategory> AsCategory() { return nullptr; }
	virtual TSharedPtr<FTakeRecorderSourceTreeItem> AsSource()   { return nullptr; }

	virtual TSharedRef<SWidget> ConstructWidget(TWeakPtr<STakeRecorderSources> SourcesWidget) = 0;
};

struct FTakeRecorderSourceTreeItem : ITakeRecorderSourceTreeItem
{
	/** Weak pointer to the source that this tree item represents */
	TWeakObjectPtr<UTakeRecorderSource> WeakSource;

	explicit FTakeRecorderSourceTreeItem(UTakeRecorderSource* InSource)
		: WeakSource(InSource)
	{}

	FText GetLabel() const
	{
		UTakeRecorderSource* Source = WeakSource.Get();
		return Source ? Source->GetDisplayText() : FText();
	}

	virtual void Delete(UTakeRecorderSources* Owner) override
	{
		UTakeRecorderSource* Source = WeakSource.Get();
		if (Source)
		{
			Owner->RemoveSource(Source);
		}
	}

	virtual TSharedPtr<FTakeRecorderSourceTreeItem> AsSource()
	{
		return SharedThis(this);
	}

	virtual TSharedRef<SWidget> ConstructWidget(TWeakPtr<STakeRecorderSources> SourcesWidget) override
	{
		return SNew(SOverlay)

		+SOverlay::Slot()
		[
		
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(32)
				.HeightOverride(32)
				.Visibility(this, &FTakeRecorderSourceTreeItem::EditableVisibility)
				[
					SNew(SWidgetSwitcher)
		  			.WidgetIndex(this, &FTakeRecorderSourceTreeItem::GetIndicatorIndex)

		  			+ SWidgetSwitcher::Slot()
		  			.VAlign(VAlign_Center)
		  			.HAlign(HAlign_Center)
		  			[
						SNew(SCheckBox)
						.Style(FTakeRecorderStyle::Get(), "TakeRecorder.Source.Switch")
						.IsFocusable(false)
						.IsChecked(this, &FTakeRecorderSourceTreeItem::GetCheckState)
						.OnCheckStateChanged(this, &FTakeRecorderSourceTreeItem::SetCheckState, SourcesWidget)
					]

		  			+ SWidgetSwitcher::Slot()
		  			.VAlign(VAlign_Center)
		  			.HAlign(HAlign_Center)
		  			[
						SNew(SThrobber)
						.NumPieces(1)
						.Animate(SThrobber::Opacity)
						.PieceImage(FTakeRecorderStyle::Get().GetBrush("TakeRecorder.Source.RecordingImage"))
						.Visibility(this, &FTakeRecorderSourceTreeItem::RecordingVisibility)
					]
				]	
			]

			+ SHorizontalBox::Slot()
			.Padding(8, 4, 8, 4)
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(32)
				.HeightOverride(32)
				[
					SNew(SImage)
					.Image(this, &FTakeRecorderSourceTreeItem::GetIcon)
					.ColorAndOpacity(this, &FTakeRecorderSourceTreeItem::GetImageColorAndOpacity)
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8, 0)
			[
				SNew(STextBlock)
				.Text(this, &FTakeRecorderSourceTreeItem::GetLabel)
				.TextStyle(FTakeRecorderStyle::Get(), "TakeRecorder.Source.Label")
				.ColorAndOpacity(this, &FTakeRecorderSourceTreeItem::GetColorAndOpacity)
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SBox)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8, 0, 24, 0)
			[
				SNew(STextBlock)
				.Text(this, &FTakeRecorderSourceTreeItem::GetDescription)
				.TextStyle(FTakeRecorderStyle::Get(), "TakeRecorder.Source.Label")
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				.ColorAndOpacity(this, &FTakeRecorderSourceTreeItem::GetColorAndOpacity)
			]
		]

		+SOverlay::Slot()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SBox)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 0, 0)
			[
				SNew(SColorBlock)
				.Size(FVector2D(6.0, 38.0))
				.Color(this, &FTakeRecorderSourceTreeItem::GetSourceTintColor) 
				.OnMouseButtonDown(this, &FTakeRecorderSourceTreeItem::SetSourceTintColor)
			]

		];

	}

private:

	const FSlateBrush* GetIcon() const
	{
		UTakeRecorderSource* Source = WeakSource.Get();
		return Source ? Source->GetDisplayIcon() : nullptr;
	}

	FText GetTakeLabel() const
	{
		UTakeRecorderSource* Source = WeakSource.Get();
		return Source && Source->SupportsTakeNumber() ? FText::Format(FText::FromString("TAKE {0}"), Source->TakeNumber) : FText();
	}

	FText GetDescription() const
	{
		UTakeRecorderSource* Source = WeakSource.Get();
		return Source ? Source->GetDescriptionText() : FText();
	}

	ECheckBoxState GetCheckState() const
	{
		UTakeRecorderSource* Source = WeakSource.Get();
		return Source && Source->bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void SetCheckState(const ECheckBoxState NewState, TWeakPtr<STakeRecorderSources> WeakSourcesWidget)
	{
		const bool bEnable = NewState == ECheckBoxState::Checked;

		TSharedPtr<STakeRecorderSources> SourcesWidget = WeakSourcesWidget.Pin();
		UTakeRecorderSource*             ThisSource    = WeakSource.Get();

		if (ThisSource && SourcesWidget.IsValid())
		{
			TArray<UTakeRecorderSource*> SelectedSources;
			SourcesWidget->GetSelectedSources(SelectedSources);

			FText TransactionFormat = bEnable
				? LOCTEXT("EnableSources", "Enable Recording {0}|plural(one=Source, other=Sources)")
				: LOCTEXT("DisableSources", "Disable Recording {0}|plural(one=Source, other=Sources)");

			FProperty* EnabledProperty = UTakeRecorderSource::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTakeRecorderSource, bEnabled));
			
			if (!SelectedSources.Contains(ThisSource))
			{
				FScopedTransaction Transaction(FText::Format(TransactionFormat, 1));

				ThisSource->Modify();
				ThisSource->bEnabled = bEnable;
				
				FPropertyChangedEvent PropertyChangedEvent(EnabledProperty, EPropertyChangeType::ValueSet);
				ThisSource->PostEditChangeProperty(PropertyChangedEvent);
			}
			else 
			{
				FScopedTransaction Transaction(FText::Format(TransactionFormat, SelectedSources.Num()));

				for (UTakeRecorderSource* SelectedSource : SelectedSources)
				{
					SelectedSource->Modify();
					SelectedSource->bEnabled = bEnable;
					
					FPropertyChangedEvent PropertyChangedEvent(EnabledProperty, EPropertyChangeType::ValueSet);
					SelectedSource->PostEditChangeProperty(PropertyChangedEvent);
				}
			}
		}
	}

	EVisibility RecordingVisibility() const
	{
		UTakeRecorderSource* Source = WeakSource.Get();
		return Source && Source->bEnabled ? EVisibility::Visible : EVisibility::Hidden;
	}

	int32 GetIndicatorIndex() const
	{
		return UTakeRecorderBlueprintLibrary::IsRecording() ? 1 : 0;
	}

	FSlateColor GetColorAndOpacity() const
	{
		UTakeRecorderSource* Source = WeakSource.Get();
		return Source && Source->bEnabled ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground();
	}

	FSlateColor GetImageColorAndOpacity() const
	{
		UTakeRecorderSource* Source = WeakSource.Get();
		return Source && Source->bEnabled ? FLinearColor::White : FLinearColor::White.CopyWithNewOpacity(0.3f);
	}

	EVisibility EditableVisibility() const
	{
		if (UTakeRecorderSource* Source = WeakSource.Get())
		{
			if (ULevelSequence* OwningSequence = Source->GetTypedOuter<ULevelSequence>())
			{
				if (UTakeMetaData* TakeMetaData = OwningSequence->FindMetaData<UTakeMetaData>())
				{
					const UTakeRecorderSubsystem* Subsystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>();
					const bool bRecordIsIntoSequence = Subsystem != nullptr ? Subsystem->GetTakeRecorderMode() == ETakeRecorderMode::RecordIntoSequence : false;
					if (TakeMetaData->Recorded() && !bRecordIsIntoSequence)
					{
						return EVisibility::Hidden;
					}
				}
			}
		}	
		return EVisibility::Visible;
	}

	FReply SetSourceTintColor(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		FColorPickerArgs PickerArgs;
		PickerArgs.bUseAlpha = false;
		PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
		PickerArgs.InitialColor = GetSourceTintColor();
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FTakeRecorderSourceTreeItem::OnColorPickerPicked);

		OpenColorPicker(PickerArgs);

		return FReply::Handled();
	}

	FLinearColor GetSourceTintColor() const 
	{
		UTakeRecorderSource* Source = WeakSource.Get();
		return Source ? Source->TrackTint.ReinterpretAsLinear() : FLinearColor::White;
	}

	void OnColorPickerPicked(FLinearColor NewColor)
	{
		UTakeRecorderSource* Source = WeakSource.Get();
		if (Source != nullptr)
		{
			Source->TrackTint = NewColor.ToFColor(true);
		}
	}
};

struct FTakeRecorderSourceCategory : ITakeRecorderSourceTreeItem
{
	/** The title of this category */
	FText Category;

	/** Sorted list of this category's children */
	TArray<TSharedPtr<FTakeRecorderSourceTreeItem>> Children;

	explicit FTakeRecorderSourceCategory(const FString& InCategory)
		: Category(FText::FromString(InCategory))
	{}

	virtual void Delete(UTakeRecorderSources* Owner) override
	{
		for (TSharedPtr<FTakeRecorderSourceTreeItem> Child : Children)
		{
			Child->Delete(Owner);
		}
	}

	virtual TSharedPtr<FTakeRecorderSourceCategory> AsCategory()
	{
		return SharedThis(this);
	}

	virtual TSharedRef<SWidget> ConstructWidget(TWeakPtr<STakeRecorderSources> SourcesWidget) override
	{
		return SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(20, 4)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &FTakeRecorderSourceCategory::GetLabel)
			];
	}

private:

	FText GetLabel() const
	{
		return FText::Format(LOCTEXT("CategoryFormatString", "{0} ({1})"),
			Category, Children.Num());
	}
};


void STakeRecorderSources::Construct(const FArguments& InArgs)
{
	CachedSourcesSerialNumber = uint32(-1);

	TreeView = SNew(STreeView<TSharedPtr<ITakeRecorderSourceTreeItem>>)
		.TreeItemsSource(&RootNodes)
		.OnSelectionChanged(InArgs._OnSelectionChanged)
		.OnGenerateRow(this, &STakeRecorderSources::OnGenerateRow)
		.OnGetChildren(this, &STakeRecorderSources::OnGetChildren);

	CommandList = MakeShared<FUICommandList>();
	CommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &STakeRecorderSources::OnDeleteSelected)
	);

	ChildSlot
	[
		SNew(SDropTarget)
		.OnDropped(this, &STakeRecorderSources::OnDragDropTarget)
		.OnAllowDrop(this, &STakeRecorderSources::CanDragDropTarget)
		.OnIsRecognized(this, &STakeRecorderSources::CanDragDropTarget)
		[
			TreeView.ToSharedRef()
		]
	];
}

void STakeRecorderSources::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	UTakeRecorderSources* Sources = WeakSources.Get();

	// If we have a sources ptr, we expect its serial number to match our cached one, if not, we rebuild the tree
	if (Sources)
	{
		if (CachedSourcesSerialNumber != Sources->GetSourcesSerialNumber())
		{
			ReconstructTree();
		}
	}
	// The sources are no longer valid, so we expect our cached serial number to be -1. If not, we haven't reset the tree yet.
	else if (CachedSourcesSerialNumber != uint32(-1))
	{
		ReconstructTree();
	}
}

FReply STakeRecorderSources::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void STakeRecorderSources::GetSelectedSources(TArray<UTakeRecorderSource*>& OutSources) const
{
	TArray<TSharedPtr<ITakeRecorderSourceTreeItem>> SelectedItems;

	TreeView->GetSelectedItems(SelectedItems);
	for (TSharedPtr<ITakeRecorderSourceTreeItem> Item : SelectedItems)
	{
		TSharedPtr<FTakeRecorderSourceTreeItem> SourceItem = Item->AsSource();
		UTakeRecorderSource* SourcePtr = SourceItem.IsValid() ? SourceItem->WeakSource.Get() : nullptr;
		if (SourcePtr)
		{
			OutSources.Add(SourcePtr);
		}
	}
}

void STakeRecorderSources::SetSourceObject(UTakeRecorderSources* InSources)
{
	WeakSources = InSources;
	ReconstructTree();
}

void STakeRecorderSources::ReconstructTree()
{
	UTakeRecorderSources* Sources = WeakSources.Get();
	if (!Sources)
	{
		CachedSourcesSerialNumber = uint32(-1);
		RootNodes.Reset();
		return;
	}

	CachedSourcesSerialNumber = Sources->GetSourcesSerialNumber();

	TSortedMap<FString, TSharedPtr<FTakeRecorderSourceCategory>> RootCategories;
	for (TSharedPtr<ITakeRecorderSourceTreeItem> RootItem : RootNodes)
	{
		TSharedPtr<FTakeRecorderSourceCategory> RootCategory = RootItem->AsCategory();
		if (RootCategory.IsValid())
		{
			RootCategory->Children.Reset();
			RootCategories.Add(RootCategory->Category.ToString(), RootCategory);
		}
	}

	RootNodes.Reset();

	// We attempt to re-use tree items in order to maintain selection states on them
	TMap<FObjectKey, TSharedPtr<FTakeRecorderSourceTreeItem>> OldSourceToTreeItem;
	Swap(SourceToTreeItem, OldSourceToTreeItem);


	static const FName CategoryName = "Category";

	for (UTakeRecorderSource* Source : Sources->GetSources())
	{
		if (!Source)
		{
			continue;
		}

		// The category in the UI is taken from the class itself
		FString Category = Source->GetCategoryText().ToString();
		if (Category.IsEmpty())
		{
			Category = Source->GetClass()->GetMetaData(CategoryName);
		}

		// Attempt to find an existing category node, creating one if necessary
		TSharedPtr<FTakeRecorderSourceCategory> CategoryNode = RootCategories.FindRef(Category);
		if (!CategoryNode.IsValid())
		{
			CategoryNode = RootCategories.Add(Category, MakeShared<FTakeRecorderSourceCategory>(Category));

			TreeView->SetItemExpansion(CategoryNode, true);
		}

		// Attempt to find an existing source item node from the previous data, creating one if necessary
		FObjectKey SourceKey(Source);
		TSharedPtr<FTakeRecorderSourceTreeItem> SourceItem = SourceToTreeItem.FindRef(SourceKey);
		if (!SourceItem.IsValid())
		{
			SourceItem = OldSourceToTreeItem.FindRef(Source);
			if (SourceItem.IsValid())
			{
				SourceToTreeItem.Add(SourceKey, SourceItem);
			}
			else
			{
				SourceItem = MakeShared<FTakeRecorderSourceTreeItem>(Source);
				SourceToTreeItem.Add(SourceKey, SourceItem);

				TreeView->SetItemExpansion(SourceItem, true);
			}
		}

		check(SourceItem.IsValid());
		CategoryNode->Children.Add(SourceItem);
	}


	RootNodes.Reset(RootCategories.Num());
	for(TTuple<FString, TSharedPtr<FTakeRecorderSourceCategory>>& Pair : RootCategories)
	{
		if (Pair.Value->Children.Num() == 0)
		{
			continue;
		}

		// Sort children by name. Work with a tuple of index and string to avoid excessively calling GetLabel().ToString()
		TArray<TTuple<int32, FString>> SortData;
		SortData.Reserve(Pair.Value->Children.Num());
		for (TSharedPtr<FTakeRecorderSourceTreeItem> Item : Pair.Value->Children)
		{
			SortData.Add(MakeTuple(SortData.Num(), Item->GetLabel().ToString()));
		}

		auto SortPredicate = [](TTuple<int32, FString>& A, TTuple<int32, FString>& B)
		{
			return A.Get<1>() <= B.Get<1>();
		};

		Algo::Sort(SortData, SortPredicate);

		// Create a new sorted list of the child entries
		TArray<TSharedPtr<FTakeRecorderSourceTreeItem>> NewChildren;
		NewChildren.Reserve(SortData.Num());
		for (const TTuple<int32, FString>& Item : SortData)
		{
			NewChildren.Add(Pair.Value->Children[Item.Get<0>()]);
		}

		Swap(Pair.Value->Children, NewChildren);

		// Add the category
		RootNodes.Add(Pair.Value);
	}

	TreeView->RequestTreeRefresh();
}

TSharedRef<ITableRow> STakeRecorderSources::OnGenerateRow(TSharedPtr<ITakeRecorderSourceTreeItem> Item, const TSharedRef<STableViewBase>& Tree)
{
	return
		SNew(STableRow<TSharedPtr<ITakeRecorderSourceTreeItem>>, Tree)
		[
			Item->ConstructWidget(SharedThis(this))
		];
}

void STakeRecorderSources::OnGetChildren(TSharedPtr<ITakeRecorderSourceTreeItem> Item, TArray<TSharedPtr<ITakeRecorderSourceTreeItem>>& OutChildItems)
{
	TSharedPtr<FTakeRecorderSourceCategory> Category = Item->AsCategory();
	if (Category.IsValid())
	{
		OutChildItems.Append(Category->Children);
	}
}

FReply STakeRecorderSources::OnDragDropTarget(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> DragDropOperation = InDragDropEvent.GetOperation();
	if (!DragDropOperation)
	{
		return FReply::Unhandled();
	}

	UTakeRecorderSources* Sources = WeakSources.Get();
	if (Sources)
	{
		for (ITakeRecorderDropHandler* Handler : ITakeRecorderDropHandler::GetDropHandlers())
		{
			if (Handler->CanHandleOperation(DragDropOperation, Sources))
			{
				Handler->HandleOperation(DragDropOperation, Sources);
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

bool STakeRecorderSources::CanDragDropTarget(TSharedPtr<FDragDropOperation> InOperation)
{
	if (IsLocked())
	{
		return false;
	}

	UTakeRecorderSources* Sources = WeakSources.Get();
	if (Sources)
	{
		for (ITakeRecorderDropHandler* Handler : ITakeRecorderDropHandler::GetDropHandlers())
		{
			if (Handler->CanHandleOperation(InOperation, Sources))
			{
				return true;
			}
		}
	}

	return false;
}

void STakeRecorderSources::OnDeleteSelected()
{
	// Still capture the delete so that it doesn't trickle through to delete selected actors for example
	if (IsLocked())
	{
		return;
	}

	UTakeRecorderSources* Sources = WeakSources.Get();
	if (Sources)
	{
		TArray<TSharedPtr<ITakeRecorderSourceTreeItem>> Items = TreeView->GetSelectedItems();

		FScopedTransaction Transaction(FText::Format(LOCTEXT("DeleteSelection", "Delete Selected {0}|plural(one=Source, other=Sources)"), Items.Num()));
		Sources->Modify();

		for (TSharedPtr<ITakeRecorderSourceTreeItem> Item : Items)
		{
			Item->Delete(Sources);
		}
	}
}

bool STakeRecorderSources::IsLocked() const
{
	UTakeRecorderSources* Sources = WeakSources.Get();
	if (Sources)
	{
		if (ULevelSequence* OwningSequence = Sources->GetTypedOuter<ULevelSequence>())
		{
			if (UTakeMetaData* TakeMetaData = OwningSequence->FindMetaData<UTakeMetaData>())
			{
				return TakeMetaData->IsLocked();
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE