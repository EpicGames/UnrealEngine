// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/ObjectKey.h"
#include "Styling/SlateTypes.h" 

class ITableRow;
class FUICommandList;
class STableViewBase;
class UTakeRecorderSource;
class UTakeRecorderSources;
struct ITakeRecorderSourceTreeItem;
struct FTakeRecorderSourceTreeItem;
template<typename> class STreeView;

DECLARE_DELEGATE_TwoParams(
	FOnSourcesSelectionChanged,
	TSharedPtr<ITakeRecorderSourceTreeItem>,
	ESelectInfo::Type
)

/** Main widget for the take recorder sources panel */
class STakeRecorderSources : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STakeRecorderSources){}
		SLATE_EVENT(FOnSourcesSelectionChanged, OnSelectionChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SetSourceObject(UTakeRecorderSources* InSources);

	void GetSelectedSources(TArray<UTakeRecorderSource*>& OutSources) const;

private:

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	void ReconstructTree();

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<ITakeRecorderSourceTreeItem> Item, const TSharedRef<STableViewBase>& Tree);

	void OnGetChildren(TSharedPtr<ITakeRecorderSourceTreeItem> Item, TArray<TSharedPtr<ITakeRecorderSourceTreeItem>>& OutChildItems);

	FReply OnDragDropTarget(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);

	bool CanDragDropTarget(TSharedPtr<FDragDropOperation> InOperation);

	void OnDeleteSelected();

	bool IsLocked() const;

private:

	uint32 CachedSourcesSerialNumber;
	TWeakObjectPtr<UTakeRecorderSources> WeakSources;

	TArray<TSharedPtr<ITakeRecorderSourceTreeItem>> RootNodes;
	TMap<FObjectKey, TSharedPtr<FTakeRecorderSourceTreeItem>> SourceToTreeItem;
	TSharedPtr<STreeView<TSharedPtr<ITakeRecorderSourceTreeItem>>> TreeView;

	TSharedPtr<FUICommandList> CommandList;
};