// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STakeRecorderTabContent.h"
#include "Widgets/STakeRecorderPanel.h"
#include "Widgets/STakeRecorderCockpit.h"
#include "Widgets/STakePresetAssetEditor.h"
#include "Widgets/Input/SHyperlink.h"
#include "TakeMetaData.h"
#include "TakeRecorderSources.h"
#include "ITakeRecorderModule.h"
#include "TakeRecorderStyle.h"
#include "TakePresetToolkit.h"
#include "Styling/SlateIconFinder.h"
#include "LevelSequence.h"
#include "Recorder/TakeRecorderPanel.h"
#include "Recorder/TakeRecorderBlueprintLibrary.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "ITakeRecorderSourcesManager.h"

// Analytics
#include "EngineAnalytics.h"
#include "Interfaces/IAnalyticsProvider.h"

#define LOCTEXT_NAMESPACE "STakeRecorderTabContent"


void STakeRecorderTabContent::Construct(const FArguments& InArgs)
{
	TitleAttribute = ITakeRecorderModule::TakeRecorderTabLabel;
	IconAttribute  = FTakeRecorderStyle::Get().GetBrush("TakeRecorder.TabIcon");

	// Delay one tick before opening the default recording setup panel
	// this allows anything that just invoked the tab to customize it without the default UI being created
	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &STakeRecorderTabContent::OnActiveTimer));

	UTakeRecorderPanel* NewTakeRecorderPanel = NewObject<UTakeRecorderPanel>();

	NewTakeRecorderPanel->InitializePanel(SharedThis(this));
	UTakeRecorderBlueprintLibrary::SetTakeRecorderPanel(NewTakeRecorderPanel);
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("TakeRecorder.PanelOpened"));
	}
}

STakeRecorderTabContent::~STakeRecorderTabContent()
{
	UTakeRecorderBlueprintLibrary::SetTakeRecorderPanel(nullptr);
}

EActiveTimerReturnType STakeRecorderTabContent::OnActiveTimer(double InCurrentTime, float InDeltaTime)
{
	if (!GetMode().IsSet())
	{
		SetupForRecording((UTakePreset*)nullptr);
	}
	return EActiveTimerReturnType::Stop;
}

FText STakeRecorderTabContent::GetTitle() const
{
	return TitleAttribute.Get();
}

const FSlateBrush* STakeRecorderTabContent::GetIcon() const
{
	return IconAttribute.Get();
}

TOptional<ETakeRecorderPanelMode> STakeRecorderTabContent::GetMode() const
{
	if (WeakPanel.IsValid())
	{
		return WeakPanel.Pin()->GetMode();
	}
	else if (WeakAssetEditor.IsValid())
	{
		return ETakeRecorderPanelMode::EditingPreset;
	}
	else
	{
		return TOptional<ETakeRecorderPanelMode>();
	}
}

void STakeRecorderTabContent::SetupForRecording(UTakePreset* BasePreset)
{
	WeakAssetEditor = nullptr;

	TitleAttribute = ITakeRecorderModule::TakeRecorderTabLabel;
	IconAttribute  = FAppStyle::Get().GetBrush("SequenceRecorder.TabIcon");

	// Null out the tab content to ensure that all references have been cleaned up before constructing the new one
	ChildSlot [ SNullWidget::NullWidget ];

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SAssignNew(WeakPanel, STakeRecorderPanel)
		.BasePreset(BasePreset)
	];

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("TakeRecorder.SetupForRecordingFromPreset"));
	}
}

void STakeRecorderTabContent::SetupForRecording(ULevelSequence* LevelSequenceAsset)
{
	WeakAssetEditor = nullptr;

	TitleAttribute = ITakeRecorderModule::TakeRecorderTabLabel;
	IconAttribute  = FTakeRecorderStyle::Get().GetBrush("TakeRecorder.TabIcon");

	// Null out the tab content to ensure that all references have been cleaned up before constructing the new one
	ChildSlot [ SNullWidget::NullWidget ];

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SAssignNew(WeakPanel, STakeRecorderPanel)
		.BaseSequence(LevelSequenceAsset)
	];

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("TakeRecorder.SetupForRecordingFromLevelSequence"));
	}
}

void STakeRecorderTabContent::SetupForRecordingInto(ULevelSequence* LevelSequenceAsset)
{
	WeakAssetEditor = nullptr;

	TitleAttribute = ITakeRecorderModule::TakeRecorderTabLabel;
	IconAttribute  = FTakeRecorderStyle::Get().GetBrush("TakeRecorder.TabIcon");

	// Null out the tab content to ensure that all references have been cleaned up before constructing the new one
	ChildSlot [ SNullWidget::NullWidget ];

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SAssignNew(WeakPanel, STakeRecorderPanel)
		.RecordIntoSequence(LevelSequenceAsset)
	];

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("TakeRecorder.SetupForRecordingIntoLevelSequence"));
	}
}

void STakeRecorderTabContent::OnPresetClosed()
{
	WeakPanel = nullptr;
	WeakAssetEditor = nullptr;

	TitleAttribute = ITakeRecorderModule::TakeRecorderTabLabel;
	IconAttribute = FTakeRecorderStyle::Get().GetBrush("TakeRecorder.TabIcon");
}

void STakeRecorderTabContent::SetupForEditing(UTakePreset* TakePreset)
{
	TSharedPtr<ILevelEditor> OpenedFromLevelEditor = FModuleManager::LoadModuleChecked< FLevelEditorModule >("LevelEditor").GetFirstLevelEditor();
	TSharedPtr<FTakePresetToolkit> Toolkit = MakeShared<FTakePresetToolkit>();
	Toolkit->Initialize(EToolkitMode::WorldCentric, OpenedFromLevelEditor, TakePreset);

	SetupForEditing(Toolkit);
}

void STakeRecorderTabContent::SetupForEditing(TSharedPtr<FTakePresetToolkit> InToolkit)
{
	WeakPanel = nullptr;

	TitleAttribute = MakeAttributeSP(InToolkit.Get(), &FTakePresetToolkit::GetToolkitName);
	IconAttribute = MakeAttributeSP(InToolkit.Get(), &FTakePresetToolkit::GetTabIcon);

	InToolkit->OnClosed().AddRaw(this, &STakeRecorderTabContent::OnPresetClosed);

	// Null out the tab content to ensure that all references have been cleaned up before constructing the new one
	ChildSlot[SNullWidget::NullWidget];

	ChildSlot
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(WeakAssetEditor, STakePresetAssetEditor, InToolkit, SharedThis(this))
		];

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("TakeRecorder.SetupForEditingPreset"));
	}
}

void STakeRecorderTabContent::SetupForViewing(ULevelSequence* LevelSequence)
{
	WeakAssetEditor = nullptr;

	TitleAttribute = FText::FromString(LevelSequence->GetName());
	IconAttribute  = FSlateIconFinder::FindIconBrushForClass(ULevelSequence::StaticClass());

	// Null out the tab content to ensure that all references have been cleaned up before constructing the new one
	ChildSlot [ SNullWidget::NullWidget ];

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SAssignNew(WeakPanel, STakeRecorderPanel)
		.SequenceToView(LevelSequence)
	];

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("TakeRecorder.SetupForReviewing"));
	}
}

ULevelSequence* STakeRecorderTabContent::GetLevelSequence() const
{
	if (TSharedPtr<STakePresetAssetEditor> AssetEditor = WeakAssetEditor.Pin())
	{
		return AssetEditor->GetLevelSequence();
	}
	else if (TSharedPtr<STakeRecorderPanel> Panel = WeakPanel.Pin())
	{
		return Panel->GetLevelSequence();
	}
	else
	{
		return nullptr;
	}
}

ULevelSequence* STakeRecorderTabContent::GetLastRecordedLevelSequence() const
{
	if (TSharedPtr<STakeRecorderPanel> Panel = WeakPanel.Pin())
	{
		return Panel->GetLastRecordedLevelSequence();
	}
	else
	{
		return nullptr;
	}
}

UTakeMetaData* STakeRecorderTabContent::GetTakeMetaData() const
{
	if (TSharedPtr<STakePresetAssetEditor> AssetEditor = WeakAssetEditor.Pin())
	{
		ULevelSequence* LevelSequence = AssetEditor->GetLevelSequence();
		return LevelSequence ? LevelSequence->FindMetaData<UTakeMetaData>() : nullptr;
	}
	else if (TSharedPtr<STakeRecorderPanel> Panel = WeakPanel.Pin())
	{
		return Panel->GetTakeMetaData();
	}
	else
	{
		return nullptr;
	}
}

FFrameRate STakeRecorderTabContent::GetFrameRate() const
{
	TSharedPtr<STakeRecorderPanel>   Panel = WeakPanel.Pin();
	TSharedPtr<STakeRecorderCockpit> Cockpit = Panel.IsValid() ? Panel->GetCockpitWidget() : nullptr;

	return Cockpit.IsValid() ? Cockpit->GetFrameRate() : FFrameRate();
}

void STakeRecorderTabContent::SetFrameRate(FFrameRate InFrameRate)
{
	TSharedPtr<STakeRecorderPanel>   Panel = WeakPanel.Pin();
	TSharedPtr<STakeRecorderCockpit> Cockpit = Panel.IsValid() ? Panel->GetCockpitWidget() : nullptr;

	if (Cockpit.IsValid())
	{
		Cockpit->SetFrameRate(InFrameRate,false);
	}
}

void STakeRecorderTabContent::SetFrameRateFromTimecode(bool bInFromTimecode)
{
	TSharedPtr<STakeRecorderPanel>   Panel = WeakPanel.Pin();
	TSharedPtr<STakeRecorderCockpit> Cockpit = Panel.IsValid() ? Panel->GetCockpitWidget() : nullptr;

	if (Cockpit.IsValid())
	{
		Cockpit->SetFrameRate(FApp::GetTimecodeFrameRate(), true);
	}
}


UTakeRecorderSources* STakeRecorderTabContent::GetSources() const
{
	ULevelSequence* LevelSequence = nullptr;

	if (TSharedPtr<STakePresetAssetEditor> AssetEditor = WeakAssetEditor.Pin())
	{
		LevelSequence = AssetEditor->GetLevelSequence();
	}
	else if (TSharedPtr<STakeRecorderPanel> Panel = WeakPanel.Pin())
	{
		LevelSequence = Panel->GetLevelSequence();
	}

	return ITakeRecorderSourcesManager::GetChecked().FindSources(LevelSequence);
}

void STakeRecorderTabContent::StartRecording() const
{
	FText ErrorText;

	TSharedPtr<STakeRecorderPanel>   Panel   = WeakPanel.Pin();
	TSharedPtr<STakeRecorderCockpit> Cockpit = Panel.IsValid() ? Panel->GetCockpitWidget() : nullptr;

	if (!Cockpit.IsValid())
	{
		FFrame::KismetExecutionMessage(TEXT("It is not currently possible to start recording on this panel."), ELogVerbosity::Error);
	}
	else if (Cockpit->Recording())
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot start a new recording while one is already in progress."), ELogVerbosity::Error);
	}
	else if (Cockpit->Reviewing())
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot start a new recording while reviewing a take."), ELogVerbosity::Error);
	}
	else if (!Cockpit->CanStartRecording(ErrorText))
	{
		FFrame::KismetExecutionMessage(*ErrorText.ToString(), ELogVerbosity::Error);
	}
	else
	{
		Cockpit->StartRecording();

		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("TakeRecorder.StartRecording"));
		}
	}
}

void STakeRecorderTabContent::StopRecording() const
{
	TSharedPtr<STakeRecorderPanel>   Panel   = WeakPanel.Pin();
	TSharedPtr<STakeRecorderCockpit> Cockpit = Panel.IsValid() ? Panel->GetCockpitWidget() : nullptr;

	if (!Cockpit.IsValid())
	{
		FFrame::KismetExecutionMessage(TEXT("It is not currently possible to stop recording on this panel."), ELogVerbosity::Error);
	}
	else if (!Cockpit->Recording())
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot stop a recording when one is not in progress."), ELogVerbosity::Error);
	}
	else
	{
		Cockpit->StopRecording();

		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("TakeRecorder.StopRecording"));
		}
	}
}


void STakeRecorderTabContent::ClearPendingTake()
{
	if (TSharedPtr<STakeRecorderPanel> Panel = WeakPanel.Pin())
	{
		Panel->ClearPendingTake();
	}
}

bool STakeRecorderTabContent::CanStartRecording(FText& ErrorText) const
{
	TSharedPtr<STakeRecorderPanel>   Panel   = WeakPanel.Pin();
	TSharedPtr<STakeRecorderCockpit> Cockpit = Panel.IsValid() ? Panel->GetCockpitWidget() : nullptr;
	
	if (Cockpit)
	{
		return Cockpit->CanStartRecording(ErrorText);
	}
	else
	{
		ErrorText = LOCTEXT("ErrorWidget_NoTakeRecorderPanel", "Take Recorder panel is not set up yet. Please open Take Recorder.");
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
