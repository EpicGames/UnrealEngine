// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "ITakeRecorderTabContent.h"
#include "Recorder/TakeRecorderPanel.h"

class UTakePreset;
class UTakeMetaData;
class ULevelSequence;
class STakeRecorderPanel;
class UTakeRecorderSources;
class STakePresetAssetEditor;

class STakeRecorderTabContent : public ITakeRecorderTabContent
{
public:

	SLATE_BEGIN_ARGS(STakeRecorderTabContent){}
	SLATE_END_ARGS()

	virtual ~STakeRecorderTabContent() override;

	void Construct(const FArguments& InArgs);

	//~ Begin ITakeRecorderTabContent
	virtual FText GetTitle() const override;
	virtual const FSlateBrush* GetIcon() const override;
	virtual TOptional<ETakeRecorderPanelMode> GetMode() const override;
	virtual void SetupForRecording(ULevelSequence* LevelSequenceAsset) override;
	virtual void SetupForRecording(UTakePreset* BasePreset) override;
	virtual void SetupForRecordingInto(ULevelSequence* LevelSequenceAsset) override;
	virtual void SetupForEditing(UTakePreset* Preset) override;
	// WITH_EDITOR
	virtual void SetupForEditing(TSharedPtr<FTakePresetToolkit> InToolkit) override;
	// WITH_EDITOR
	virtual void SetupForViewing(ULevelSequence* LevelSequence) override;
	virtual ULevelSequence* GetLevelSequence() const override;
	virtual ULevelSequence* GetLastRecordedLevelSequence() const override;
	virtual UTakeMetaData* GetTakeMetaData() const override;
	virtual FFrameRate GetFrameRate() const override;
	virtual void SetFrameRate(FFrameRate InFrameRate) override;
	virtual void SetFrameRateFromTimecode(bool bInFromTimecode) override;
	virtual UTakeRecorderSources* GetSources() const override;
	virtual void StartRecording() const override;
	virtual void StopRecording() const override;
	virtual void ClearPendingTake() override;
	virtual bool CanStartRecording(FText& ErrorText) const override;
	virtual const TWeakPtr<STakeRecorderPanel>& GetTakeRecorderPanel() const override { return WeakPanel; };
	//~ End ITakeRecorderTabContent

private:

	EActiveTimerReturnType OnActiveTimer(double InCurrentTime, float InDeltaTime);

	void OnPresetClosed();

private:

	TAttribute<FText> TitleAttribute;
	TAttribute<const FSlateBrush*> IconAttribute;
	TWeakPtr<STakeRecorderPanel> WeakPanel;
	TWeakPtr<STakePresetAssetEditor> WeakAssetEditor;
};
