// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Margin.h"

namespace TakeRecorder
{
	/** Button offset to apply to buttons on take recorder panel toolbars */
	static FMargin ButtonOffset(FMargin(0.f, 0.f, 4.f, 0.f));

	/** Button padding to apply to buttons inside take recorder panel toolbars */
	static FMargin ButtonPadding(4.f);

	/** Outer padding to used for toolbars on the take preset asset editor and take recorder UI */
	static FMargin ToolbarPadding(FMargin(0.f, 2.f));
}