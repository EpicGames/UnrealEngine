// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TakeRecorderEditor : ModuleRules
{
	public TakeRecorderEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AppFramework",
				"AssetDefinition",
				"AssetRegistry",
				"TakeTrackRecorders",
				"ClassViewer",
				"ContentBrowser",
				"Core",
				"CoreUObject",
				"Engine",
				"EditorStyle",
				"EditorWidgets",
				"LevelEditor",
				"MovieSceneTools",
				"PropertyEditor",
				"EditorFramework",
				"MovieScene",
				"MovieSceneTracks",
				"LevelSequence",
				"InputCore",
				"MessageLog",
				"UnrealEd",
				"WorkspaceMenuStructure",
				"SequencerCore",
				"Slate",
				"SlateCore",
				"NamingTokens",
				"NamingTokensUI",
				"TakeRecorder",
				"TakesCore",
				"TakeMovieScene",
				"ToolWidgets",
				"ToolMenus",
				"TimeManagement",
				"Projects"
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"EditorSubsystem",
				"Sequencer"
			}
		);
	}
}
