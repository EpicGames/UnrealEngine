// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderNamingTokens.h"

#include "NamingTokens/TakeRecorderNamingTokensContext.h"

#include "ITakeRecorderNamingTokensModule.h"
#include "TakeRecorderNamingTokensLog.h"

#include "Recorder/TakeRecorderSubsystem.h"
#include "TakeMetaData.h"
#include "TakeRecorderSettings.h"
#include "TakesCoreBlueprintLibrary.h"

#include "Engine/Engine.h"
#include "LevelSequence.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"

#if WITH_EDITOR
#include "Editor.h"
#include "MovieSceneToolsProjectSettings.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderNamingTokens)

#define LOCTEXT_NAMESPACE "TakeRecorderNamingTokens"

UTakeRecorderNamingTokens::UTakeRecorderNamingTokens()
{
	Namespace = ITakeRecorderNamingTokensModule::GetTakeRecorderNamespace();
	NamespaceDisplayName = LOCTEXT("NamespaceDisplayName", "Take Recorder");
}

UTakeRecorderNamingTokens::~UTakeRecorderNamingTokens()
{
}

void UTakeRecorderNamingTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens)
{
	Super::OnCreateDefaultTokens(Tokens);

	Tokens.Add({
	    TEXT("day"),
	    LOCTEXT("TokenDay", "Day"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]{
	        return FText::FromString(FString::Printf(TEXT("%02i"), GetCurrentDateTime().GetDay()));
	    })
	});

	Tokens.Add({
	    TEXT("month"),
	    LOCTEXT("TokenMonth", "Month"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]{
	        return FText::FromString(FString::Printf(TEXT("%02i"), GetCurrentDateTime().GetMonth()));
	    })
	});

	Tokens.Add({
	    TEXT("year"),
	    LOCTEXT("TokenYear", "Year"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]{
	        return FText::FromString(FString::Printf(TEXT("%04i"), GetCurrentDateTime().GetYear()));
	    })
	});

	Tokens.Add({
	    TEXT("hour"),
	    LOCTEXT("TokenHour", "Hour"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]{
	        return FText::FromString(FString::Printf(TEXT("%02i"), GetCurrentDateTime().GetHour()));
	    })
	});

	Tokens.Add({
	    TEXT("minute"),
	    LOCTEXT("TokenMinute", "Minute"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]{
	        return FText::FromString(FString::Printf(TEXT("%02i"), GetCurrentDateTime().GetMinute()));
	    })
	});

	Tokens.Add({
	    TEXT("second"),
	    LOCTEXT("TokenSecond", "Second"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]{
	        return FText::FromString(FString::Printf(TEXT("%02i"), GetCurrentDateTime().GetSecond()));
	    })
	});

	Tokens.Add({
	    TEXT("take"),
	    LOCTEXT("TokenTake", "Take Number"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]{
		#if WITH_EDITOR
	        const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();
	        const int32 TakeNumDigits = ProjectSettings->TakeNumDigits;
		#else // WITH_EDITOR
			// This loses the settings aspect - to consider if we want to try to bring this to runtime as well.
			const int32 TakeNumDigits = 2;
		#endif // WITH_EDITOR
	        return FText::FromString(FString::Printf(TEXT("%0*d"), TakeNumDigits, TakeMetaData.IsValid() ? TakeMetaData->GetTakeNumber() : 0));
	    })
	});

	Tokens.Add({
	    TEXT("slate"),
	    LOCTEXT("TokenSlate", "Slate"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]{
	        return FText::FromString(TakeMetaData.IsValid() ? TakeMetaData->GetSlate() : TEXT("Invalid"));
	    })
	});

	Tokens.Add({
		TEXT("takeName"),
		LOCTEXT("TokenTakeName", "Take Name"),
		LOCTEXT("TokenTakeNameDescription",
			"The sequencer asset's name. This is only completely accurate when read at the time of recording or the subsystem has a valid, non-transient, level sequence."),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]{
			if (const UTakeRecorderSubsystem* Subsystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>())
			{
				if (const ULevelSequence* Sequence = Subsystem->GetLevelSequence())
				{
					FString AssetName = Sequence->GetName();
					
					const UTakePreset* TransientPreset = Subsystem->GetTransientPreset();
					if (TransientPreset && Sequence == TransientPreset->GetLevelSequence() && TakeMetaData.IsValid())
					{
						// Working with the transient preset... try to generate the most correct takeName.
						// Depending on when this is read, it may not match the actual takeName (Sequencer asset's name)
						// when saving the asset.
						
						const FString TakeAssetPath = GetDefault<UTakeRecorderProjectSettings>()->Settings.GetTakeAssetPath();
						
						FString GeneratedPath;
						if (TakeMetaData->TryGenerateRootAssetPath(TakeAssetPath, GeneratedPath))
						{
							AssetName = FPaths::GetBaseFilename(GeneratedPath);
						}
					}

					return FText::FromString(AssetName);
				}
			}
			return FText::GetEmpty();
		})
	});

	Tokens.Add({
	    TEXT("map"),
	    LOCTEXT("TokenMap", "Map"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([]{
	        FString MapName;
		#if WITH_EDITOR
	        if (GIsEditor)
	        {
	            MapName = FPackageName::GetShortFName(
	                GEditor->GetEditorWorldContext().World()->PersistentLevel->GetOutermost()->GetFName()
	            ).GetPlainNameString();
	        }
		#endif // WITH_EDITOR
	        return FText::FromString(MapName);
	    })
	});

	Tokens.Add({
		TEXT("actor"),
		LOCTEXT("TokenActor", "Actor"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]{
			if (Context && Context->Actor.IsValid())
			{
			#if WITH_EDITOR
				// GetActorLabel is an Editor only function.
				return FText::FromString(Context->Actor->GetActorLabel());
			#else // WITH_EDITOR
				return FText::FromString(Context->Actor->GetName());
			#endif // WITH_EDITOR
			}

			UE_LOGF(LogTakeRecorderNamingTokens, Verbose, "Attempted to use 'actor' naming token but no context is available.");

			return FText::GetEmpty();
		})
	});
	
	Tokens.Add({
		TEXT("channel"),
		LOCTEXT("TokenChannel", "Channel"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]{
			if (Context)
			{
				return FText::FromString(FString::FromInt(Context->AudioInputDeviceChannel));
			}

			UE_LOGF(LogTakeRecorderNamingTokens, Verbose, "Attempted to use 'channel' naming token but no context is available.");

			return FText::GetEmpty();
		})
	});
}

void UTakeRecorderNamingTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
	Super::OnPreEvaluate_Implementation(InEvaluationData);

	UTakeRecorderNamingTokensContext* MatchingContext = nullptr;
	InEvaluationData.Contexts.FindItemByClass<UTakeRecorderNamingTokensContext>(&MatchingContext);
	Context = MatchingContext;

	// Use either provided take meta-data (manually evaluated from take recorder), or locate the most recent (global operation)
	TakeMetaData = (Context && Context->TakeMetaData != nullptr) ?
		Context->TakeMetaData : TWeakObjectPtr<UTakeMetaData>(UTakeMetaData::GetMostRecentMetaData());
	if (TakeMetaData == nullptr)
	{
		// Create metadata which computes its data from available information.
		// This is how STakeRecorderCockpit handles cases when metadata isn't available. Ideally this
		// method would be available for both us and the slate widget, but UTakeRecorderProjectSettings creates
		// a circular dependency we should avoid.
		
		UTakeMetaData* TransientTakeMetaData = UTakeMetaData::CreateFromDefaults(GetTransientPackage(), NAME_None);
		TransientTakeMetaData->SetFlags(RF_Transactional | RF_Transient);

		const FString DefaultSlate = GetDefault<UTakeRecorderProjectSettings>()->Settings.DefaultSlate;
		if (TransientTakeMetaData->GetSlate() != DefaultSlate)
		{
			TransientTakeMetaData->SetSlate(DefaultSlate, false);
		}

		// Compute the correct starting take number
		const int32 NextTakeNumber = UTakesCoreBlueprintLibrary::ComputeNextTakeNumber(TransientTakeMetaData->GetSlate());
		if (TransientTakeMetaData->GetTakeNumber() != NextTakeNumber)
		{
			TransientTakeMetaData->SetTakeNumber(NextTakeNumber, false);
		}

		TakeMetaData = TransientTakeMetaData;
	}
}

void UTakeRecorderNamingTokens::OnPostEvaluate_Implementation()
{
	Super::OnPostEvaluate_Implementation();

	// Make sure we don't keep a strong reference in case we're linking to active metadata.
    TakeMetaData = nullptr;
	Context = nullptr;
}

FDateTime UTakeRecorderNamingTokens::GetCurrentDateTime_Implementation() const
{
	const FDateTime TakeTimestamp = TakeMetaData.IsValid() ? TakeMetaData->GetTimestamp() : FDateTime(0);
	return TakeTimestamp == FDateTime(0) ? Super::GetCurrentDateTime_Implementation() : TakeTimestamp;
}

#undef LOCTEXT_NAMESPACE
