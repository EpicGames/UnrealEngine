// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NamingTokens.h"

#include "TakeRecorderNamingTokens.generated.h"

class UTakeMetaData;
class UTakeRecorderNamingTokensContext;

/**
 * Naming Tokens for Take Recorder.
 */
UCLASS(NotBlueprintable)
class UTakeRecorderNamingTokens final: public UNamingTokens
{
	GENERATED_BODY()

public:
	UTakeRecorderNamingTokens();
	virtual ~UTakeRecorderNamingTokens() override;

protected:
	// ~Begin UNamingTokens
	virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens) override;
	virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData) override;
	virtual void OnPostEvaluate_Implementation() override;
	virtual FDateTime GetCurrentDateTime_Implementation() const override;
	// ~End UNamingTokens
	
private:
	/** Cached metadata for this run. */
	UPROPERTY(Transient)
	TWeakObjectPtr<const UTakeMetaData> TakeMetaData;

	/** Cached context for this run. This isn't available globally and requires a context passed to it. */
	UPROPERTY(Transient)
	TObjectPtr<UTakeRecorderNamingTokensContext> Context;
};
