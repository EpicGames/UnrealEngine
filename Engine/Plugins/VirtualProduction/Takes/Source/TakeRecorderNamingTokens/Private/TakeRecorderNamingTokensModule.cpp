// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITakeRecorderNamingTokensModule.h"
#include "TakeRecorderNamingTokensLog.h"

class FTakeRecorderNamingTokensModule : public ITakeRecorderNamingTokensModule
{
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FTakeRecorderNamingTokensModule, TakeRecorderNamingTokens);
DEFINE_LOG_CATEGORY(LogTakeRecorderNamingTokens);