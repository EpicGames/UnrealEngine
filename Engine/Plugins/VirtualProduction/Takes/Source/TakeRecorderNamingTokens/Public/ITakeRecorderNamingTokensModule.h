// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class ITakeRecorderNamingTokensModule : public IModuleInterface
{
public:
	/** The Take Recorder namespace used for processing naming tokens. */
	static FString GetTakeRecorderNamespace()
	{
		return TEXT("tr");
	}
};
