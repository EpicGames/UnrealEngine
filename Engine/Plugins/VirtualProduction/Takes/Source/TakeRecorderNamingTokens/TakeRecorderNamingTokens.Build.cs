// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TakeRecorderNamingTokens : ModuleRules
{
	public TakeRecorderNamingTokens(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Engine",
				"TakeRecorder",
				"TakesCore",
				"NamingTokens"
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"UnrealEd",
					"MovieSceneTools"
				}
			);
		}
	}
}
