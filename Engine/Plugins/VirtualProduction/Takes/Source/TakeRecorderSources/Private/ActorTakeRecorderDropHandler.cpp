// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorTakeRecorderDropHandler.h"

#if WITH_EDITOR
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/CompositeDragDropOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "EditorActorFolders.h"
#endif // WITH_EDITOR

#include "TakeRecorderSourceHelpers.h"
#include "TakeRecorderSourcesUtils.h"

namespace UE::TakeRecorderSources
{
void FActorTakeRecorderDropHandler::HandleOperation(TSharedPtr<FDragDropOperation> InOperation, UTakeRecorderSources* Sources)
{
	TArray<AActor*> ActorsToAdd = GetValidDropActors(InOperation, Sources);
	TakeRecorderSourceHelpers::AddActorSources(Sources, ActorsToAdd);
}

bool FActorTakeRecorderDropHandler::CanHandleOperation(TSharedPtr<FDragDropOperation> InOperation, UTakeRecorderSources* Sources)
{
#if WITH_EDITOR
	bool bCanHandle = false;
	if (InOperation)
	{		
		TSharedPtr<FActorDragDropOp>  ActorDrag = nullptr;
		TSharedPtr<FFolderDragDropOp> FolderDrag = nullptr;

		if (!InOperation.IsValid())
		{
			return false;
		}
		if (InOperation->IsOfType<FActorDragDropOp>())
		{
			ActorDrag = StaticCastSharedPtr<FActorDragDropOp>(InOperation);
		}
		else if (InOperation->IsOfType<FFolderDragDropOp>())
		{
			FolderDrag = StaticCastSharedPtr<FFolderDragDropOp>(InOperation);
		}
		else if (InOperation->IsOfType<FCompositeDragDropOp>())
		{
			if (const TSharedPtr<FCompositeDragDropOp> CompositeDrag = StaticCastSharedPtr<FCompositeDragDropOp>(InOperation))
			{
				ActorDrag = CompositeDrag->GetSubOp<FActorDragDropOp>();
				FolderDrag = CompositeDrag->GetSubOp<FFolderDragDropOp>();
			}
		}

		if (ActorDrag)
		{
			for (TWeakObjectPtr<AActor> WeakActor : ActorDrag->Actors)
			{
				if (AActor* Actor = WeakActor.Get())
				{
					if (TakeRecorderSourcesUtils::IsActorRecordable(Actor))
					{
						bCanHandle = true;
						break;
					}
				}
			}
		}

		if (FolderDrag && !bCanHandle)
		{
			TArray<AActor*> FolderActors;
			FActorFolders::GetActorsFromFolders(*GWorld, FolderDrag->Folders, FolderActors);

			for (AActor* ActorInFolder : FolderActors)
			{
				if (TakeRecorderSourcesUtils::IsActorRecordable(ActorInFolder))
				{
					bCanHandle = true;
					break;
				}
			}
		}
	}

	return bCanHandle;
#else // WITH_EDITOR
	return false;
#endif // WITH_EDITOR
}

TArray<AActor*> FActorTakeRecorderDropHandler::GetValidDropActors(TSharedPtr<FDragDropOperation> InOperation, UTakeRecorderSources* Sources)
{
#if WITH_EDITOR
	TSharedPtr<FActorDragDropOp>  ActorDrag = nullptr;
	TSharedPtr<FFolderDragDropOp> FolderDrag = nullptr;

	if (!InOperation.IsValid())
	{
		return TArray<AActor*>();
	}
	if (InOperation->IsOfType<FActorDragDropOp>())
	{
		ActorDrag = StaticCastSharedPtr<FActorDragDropOp>(InOperation);
	}
	else if (InOperation->IsOfType<FFolderDragDropOp>())
	{
		FolderDrag = StaticCastSharedPtr<FFolderDragDropOp>(InOperation);
	}
	else if (InOperation->IsOfType<FCompositeDragDropOp>())
	{
		if (const TSharedPtr<FCompositeDragDropOp> CompositeOp = StaticCastSharedPtr<FCompositeDragDropOp>(InOperation))
		{
			ActorDrag = CompositeOp->GetSubOp<FActorDragDropOp>();
			FolderDrag = CompositeOp->GetSubOp<FFolderDragDropOp>();
		}
	}

	TArray<AActor*> DraggedActors;

	if (ActorDrag)
	{
		DraggedActors.Reserve(ActorDrag->Actors.Num());
		for (TWeakObjectPtr<AActor> WeakActor : ActorDrag->Actors)
		{
			if (AActor* Actor = WeakActor.Get())
			{
				if (TakeRecorderSourcesUtils::IsActorRecordable(Actor))
				{
					DraggedActors.Add(Actor);
				}
			}
		}
	}

	if (FolderDrag)
	{
		TArray<AActor*> FolderActors;
		FActorFolders::GetActorsFromFolders(*GWorld, FolderDrag->Folders, FolderActors);

		for (AActor* ActorInFolder : FolderActors)
		{
			if (TakeRecorderSourcesUtils::IsActorRecordable(ActorInFolder))
			{
				DraggedActors.Add(ActorInFolder);
			}
		}
	}

	TArray<AActor*> ExistingActors;
	for (UTakeRecorderSource* Source : Sources->GetSources())
	{
		UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source);
		AActor* ExistingActor = ActorSource ? ActorSource->Target.Get() : nullptr;
		if (ExistingActor)
		{
			ExistingActors.Add(ExistingActor);
		}
	}

	if (ExistingActors.Num() && DraggedActors.Num())
	{
		// Remove any actors that are already added as a source. We do this by sorting both arrays,
		// then iterating them together, removing any that are the same
		Algo::Sort(ExistingActors);
		Algo::Sort(DraggedActors);

		for (int32 DragIndex = 0, PredIndex = 0;
			DragIndex < DraggedActors.Num() && PredIndex < ExistingActors.Num();
			/** noop*/)
		{
			AActor* Dragged   = DraggedActors[DragIndex];
			AActor* Predicate = ExistingActors[PredIndex];

			if (Dragged < Predicate)
			{
				++DragIndex;
			}
			else if (Dragged == Predicate)
			{
				DraggedActors.RemoveAt(DragIndex, EAllowShrinking::No);
			}
			else // (Dragged > Predicate)
			{
				++PredIndex;
			}
		}
	}

	return DraggedActors;
#else // WITH_EDITOR
	return TArray<AActor*>();
#endif // WITH_EDITOR
}
}