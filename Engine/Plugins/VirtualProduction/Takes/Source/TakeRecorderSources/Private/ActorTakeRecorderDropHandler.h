// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "ITakeRecorderDropHandler.h"
#include "Templates/SharedPointer.h"

class AActor;

namespace UE::TakeRecorderSources
{
struct FActorTakeRecorderDropHandler : ITakeRecorderDropHandler
{
	virtual void HandleOperation(TSharedPtr<FDragDropOperation> InOperation, UTakeRecorderSources* Sources) override;
	virtual bool CanHandleOperation(TSharedPtr<FDragDropOperation> InOperation, UTakeRecorderSources* Sources) override;

	TArray<AActor*> GetValidDropActors(TSharedPtr<FDragDropOperation> InOperation, UTakeRecorderSources* Sources);
};
}
