// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderActorSource.h"
#include "Styling/SlateIconFinder.h"
#include "MovieScene.h"
#include "MovieSceneBindingReferences.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSignedObject.h"
#include "MovieSceneTimeHelpers.h"
#include "LevelSequence.h"
#include "Features/IModularFeatures.h"
#include "Misc/ScopedSlowTask.h"
#include "TakeRecorderSource.h"
#include "TakeRecorderSources.h"
#include "TakeRecorderSourcesUtils.h"
#include "Recorder/TakeRecorderParameters.h"
#include "TakeRecorderSettings.h"
#include "TakesUtils.h"
#include "TakeMetaData.h"
#include "MovieSceneFolder.h"
#include "Serializers/MovieSceneManifestSerialization.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "ActorSequenceInformation.h"
#include "Templates/Function.h"
#include "MovieSceneSpawnableAnnotation.h"

#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/MovementComponent.h"
#include "Camera/CameraComponent.h"
#include "Components/SkeletalMeshComponent.h"

#include "TrackRecorders/IMovieSceneTrackRecorderFactory.h"
#include "TrackRecorders/MovieSceneTrackRecorder.h"
#include "TrackRecorders/MovieScenePropertyTrackRecorder.h"
#include "TrackRecorders/MovieScene3DAttachTrackRecorder.h"
#include "TrackRecorders/MovieSceneAnimationTrackRecorder.h"
#include "TrackRecorders/MovieSceneAnimationTrackRecorderSettings.h"
#include "TrackRecorders/MovieScene3DTransformTrackRecorder.h"
#include "TrackRecorders/MovieSceneTrackRecorderSettings.h"

#include "MovieSceneTakeTrack.h"
#include "MovieSceneTakeSettings.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Tracks/MovieScene3DAttachTrack.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Sections/MovieSceneSpawnSection.h"
#include "Sections/MovieScene3DAttachSection.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "MovieSceneBinding.h"

#include "NamingTokens/TakeRecorderNamingTokensContext.h"
#include "Recorder/TakeRecorderSubsystem.h"
#include "Containers/Ticker.h"
#include "TakeRecorderSourcesModule.h"
#include "TakeRecorderTimeProcessing.h"
#include "Compilation/MovieSceneCompiledDataManager.h"

#if WITH_EDITOR
#include "AnimationRecorder.h"
#include "SequencerUtilities.h"
#include "ILevelSequenceEditorToolkit.h"
#include "ISequencer.h"
#include "LevelEditorSequencerIntegration.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderActorSource)

DEFINE_LOG_CATEGORY(ActorSerialization);

#define LOCTEXT_NAMESPACE "UTakeRecorderActorSource"
static const FName MovieSceneSectionRecorderFactoryName("MovieSceneTrackRecorderFactory");
static const FName SequencerTrackClassMetadataName("SequencerTrackClass");
static const FName DoNotRecordTag("DoNotRecord");

static TAutoConsoleVariable<bool> CVarTakeRecorderAllowPossessablePIEObjects(
	TEXT("TakeRecorder.AllowPossessablePIEObjects"),
	0,
	TEXT("0: Don't allow recording PIE objects as possessable\n1: Override to allow recording PIE Objects as possessable (note the bindings will be broken!)"),
	ECVF_Default);

namespace UE::TakeRecorderActorSource::Private
{
#if WITH_EDITOR

static AActor* TryLocateBoundActorByGuid(UAssetEditorSubsystem* AssetEditorSubsystem, const ULevelSequence* SequenceToSkip, const ULevelSequence* RootSequenceToSkip, const FGuid& BindingGuid)
{
	for (ULevelSequence* OpenSequence : TObjectRange<ULevelSequence>())
	{
		if (!OpenSequence || OpenSequence == SequenceToSkip || OpenSequence == RootSequenceToSkip)
		{
			continue;
		}
		IAssetEditorInstance* EditorInstance = AssetEditorSubsystem->FindEditorForAsset(OpenSequence, false);
		if (const ILevelSequenceEditorToolkit* Toolkit = static_cast<ILevelSequenceEditorToolkit*>(EditorInstance))
		{
			if (const TSharedPtr<ISequencer> FoundSequencer = Toolkit->GetSequencer())
			{
				for (TWeakObjectPtr<> WeakObj : FoundSequencer->FindBoundObjects(BindingGuid, MovieSceneSequenceID::Root))
				{
					if (AActor* BoundActor = Cast<AActor>(WeakObj.Get()))
					{
						return BoundActor;
					}
				}
			}
		}
	}
	return nullptr;
}

static AActor* TryLocateBoundActorBySpawnableLabel(UAssetEditorSubsystem* AssetEditorSubsystem, const ULevelSequence* SequenceToSkip, const ULevelSequence* RootSequenceToSkip, const FString& ActorLabel)
{
	for (ULevelSequence* OpenSequence : TObjectRange<ULevelSequence>())
	{
		if (!OpenSequence || OpenSequence == SequenceToSkip || OpenSequence == RootSequenceToSkip)
		{
			continue;
		}
		IAssetEditorInstance* EditorInstance = AssetEditorSubsystem->FindEditorForAsset(OpenSequence, false);
		if (const ILevelSequenceEditorToolkit* Toolkit = static_cast<ILevelSequenceEditorToolkit*>(EditorInstance))
		{
			if (const TSharedPtr<ISequencer> FoundSequencer = Toolkit->GetSequencer())
			{
				UMovieScene* MovieScene = OpenSequence->GetMovieScene();
				for (int32 Idx = 0; Idx < MovieScene->GetSpawnableCount(); ++Idx)
				{
					const FMovieSceneSpawnable& Spawnable = MovieScene->GetSpawnable(Idx);
					if (Spawnable.GetName() != ActorLabel)
					{
						continue;
					}
					for (TWeakObjectPtr<> WeakObj : FoundSequencer->FindBoundObjects(Spawnable.GetGuid(), MovieSceneSequenceID::Root))
					{
						if (AActor* BoundActor = Cast<AActor>(WeakObj.Get()))
						{
							return BoundActor;
						}
					}
				}
			}
		}
	}
	return nullptr;
}

static AActor* TryLocateBoundActorByWorldLabel(const FString& ActorLabel)
{
	if (const UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		for (TActorIterator<AActor> ActorIt(World); ActorIt; ++ActorIt)
		{
			if (ActorIt->GetActorLabel() == ActorLabel)
			{
				return *ActorIt;
			}
		}
	}
	return nullptr;
}

static AActor* TryLocateBoundActorInSequencerAndWorld(const ULevelSequence* SequenceToSkip, const ULevelSequence* RootSequenceToSkip, const FGuid& BindingGuid, const FString& ActorLabel)
{
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (BindingGuid.IsValid())
	{
		if (AActor* Found = TryLocateBoundActorByGuid(AssetEditorSubsystem, SequenceToSkip, RootSequenceToSkip, BindingGuid))
		{
			return Found;
		}
	}

	if (AActor* Found = TryLocateBoundActorBySpawnableLabel(AssetEditorSubsystem, SequenceToSkip, RootSequenceToSkip, ActorLabel))
	{
		return Found;
	}

	return TryLocateBoundActorByWorldLabel(ActorLabel);
}

bool IsActorSpawnedInActiveSequence(AActor* InActor)
{
	UTakeRecorderSubsystem* TRSubsystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>();
	ULevelSequence* PendingSequence = TRSubsystem ? TRSubsystem->GetLevelSequence() : nullptr;
	if (!PendingSequence || !PendingSequence->GetMovieScene() || !GEditor)
	{
		return false;
	}

	UE::TakesCore::FActorSequenceInformation ActorInfo(PendingSequence, InActor);
	return ActorInfo.IsSpawnable();
}

TSharedPtr<ISequencer> FindSequencerForRootSequence(const ULevelSequence* RootSequence)
{
	for (const TWeakPtr<ISequencer>& Weak : FLevelEditorSequencerIntegration::Get().GetSequencers())
	{
		TSharedPtr<ISequencer> Pinned = Weak.Pin();
		if (Pinned.IsValid() && Pinned->GetRootMovieSceneSequence() == RootSequence)
		{
			return Pinned;
		}
	}
	return nullptr;
}

/*
 * This may get called frequently and from a UI callback, so the child may already be attached to ExpectedParent
 * in the level by the time we get here. If we leave that attachment in place, the sequencer's
 * pre-animated state capture records "attached to ExpectedParent" as the restore value when our attach
 * track first evaluates - turning the track into a no-op (scrubbing out of range or deleting it
 * won't detach, because the restore state matches the animated state).
 *
 * Detach with KeepWorldTransform so nothing moves visibly - the attach track's first evaluation
 * then re-attaches the child as authored, with "detached" correctly captured as pre-animated state.
 *
 * Returns if the detach was successful.
 */
bool DetachChildPreservingWorldTransform(AActor* InChild, AActor* ExpectedParent)
{
	USceneComponent* ChildRoot = InChild->GetRootComponent();
	if (!ChildRoot)
	{
		return false;
	}

	USceneComponent* CurrentAttachParent = ChildRoot->GetAttachParent();
	if (!CurrentAttachParent)
	{
		// If we didn't find a parent, this is only a success if the expected parent is also null
		return ExpectedParent == nullptr;
	}

	// Guard against unexpected state: the child should currently be attached to ExpectedParent,
	// since that's the attachment we're being asked to mirror into the sequence.
	if (CurrentAttachParent->GetOwner() != ExpectedParent)
	{
		return false;
	}
	ChildRoot->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
	return true;
}

bool HasExistingAttachSectionBinding(const UE::TakesCore::FActorSequenceInformation& ChildInfo)
{
	// As we expect parent to be a spawnable, both of these must exist in the sequence for an attach track to exist.
	if (!ChildInfo.IsControlledBySequence())
	{
		return false;
	}

	// If ChildInfo is controlled, it must have a controlling sequence.
	check(ChildInfo.GetControllingSequence().IsValid());
	UMovieScene* ControllingMovieScene = ChildInfo.GetControllingSequence()->GetMovieScene();
	for (UMovieSceneTrack* ExistingTrack : ControllingMovieScene->FindTracks(UMovieScene3DAttachTrack::StaticClass(), ChildInfo.GetCachedObjectBindingGuid()))
	{
		if (!ExistingTrack)
		{
			continue;
		}

		for (UMovieSceneSection* ExistingSection : ExistingTrack->GetAllSections())
		{
			if (UMovieScene3DAttachSection* ExistingAttachSection = Cast<UMovieScene3DAttachSection>(ExistingSection);
				ExistingAttachSection != nullptr && ExistingAttachSection->GetConstraintBindingID().GetGuid().IsValid())
			{
				// we consider any existing binding to be valid, and we should not recreate another attachment if there is already an active one.
				// We will let the Sequencer take care of that.
				return true;
			}
		}
	}
	return false;
}

FGuid ResolveOrCreateRootBinding(TSharedPtr<ISequencer> Sequencer, ULevelSequence* RootSequence,
	const UE::TakesCore::FActorSequenceInformation& ActorInfo)
{
	if (ActorInfo.IsControlledBySequence())
	{
		return ActorInfo.GetCachedObjectBindingGuid();
	}

	AActor* TargetActor = ActorInfo.GetTargetActor();
	if (!TargetActor)
	{
		return FGuid();
	}

	UMovieScene* RootMovieScene = RootSequence->GetMovieScene();
	if (!RootMovieScene)
	{
		return FGuid();
	}

	UE::Sequencer::FCreateBindingParams Params;
	Params.bSetupDefaults = false;
	Params.BindingNameOverride = TargetActor->GetActorLabel();
	const FGuid NewGuid = FSequencerUtilities::CreateOrReplaceBinding(Sequencer, RootSequence, TargetActor, Params);
	if (!NewGuid.IsValid())
	{
		return FGuid();
	}

	return NewGuid;
}

void DefaultTransformSectionToRelativeTransform(UMovieScene3DTransformSection& Section, FFrameNumber StartFrame, FFrameNumber EndFrame, const FTransform& RelativeTransform)
{
	const FVector Translation = RelativeTransform.GetTranslation();
	const FVector EulerRotation = RelativeTransform.GetRotation().Rotator().Euler();
	const FVector Scale = RelativeTransform.GetScale3D();

	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = Section.GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	if (DoubleChannels.Num() < 9)
	{
		return;
	}

	Section.Modify();

	DoubleChannels[0]->SetDefault(Translation.X);
	DoubleChannels[1]->SetDefault(Translation.Y);
	DoubleChannels[2]->SetDefault(Translation.Z);

	DoubleChannels[3]->SetDefault(EulerRotation.X);
	DoubleChannels[4]->SetDefault(EulerRotation.Y);
	DoubleChannels[5]->SetDefault(EulerRotation.Z);

	DoubleChannels[6]->SetDefault(Scale.X);
	DoubleChannels[7]->SetDefault(Scale.Y);
	DoubleChannels[8]->SetDefault(Scale.Z);
}

bool TryAuthorAttachTrackWithConstraint(UMovieScene* MovieScene, const FGuid& ChildGuid, const FGuid& ParentGuid,
	FMovieSceneSequenceID ParentSequenceID, FName SocketName, FName ComponentName,
	UTakeRecorderActorSource* AuthoringSource)
{
	if (!MovieScene)
	{
		return false;
	}

	MovieScene->Modify();
	UMovieScene3DAttachTrack* AttachTrack = Cast<UMovieScene3DAttachTrack>(
		MovieScene->AddTrack(UMovieScene3DAttachTrack::StaticClass(), ChildGuid));
	if (!AttachTrack)
	{
		return false;
	}
	AttachTrack->Modify();

	const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
	const FFrameNumber StartFrame = UE::MovieScene::DiscreteInclusiveLower(PlaybackRange);
	const int32 Duration = UE::MovieScene::DiscreteSize(PlaybackRange);
	const FFrameNumber EndFrame = StartFrame + FFrameNumber(Duration);
	const TRange<FFrameNumber> AttachRange(StartFrame, EndFrame);

	UE::MovieScene::FRelativeObjectBindingID BindingID(ParentGuid, ParentSequenceID);
	UMovieSceneSection* NewSection = AttachTrack->AddConstraint(StartFrame, Duration, SocketName, ComponentName, BindingID);

	if (!NewSection)
	{
		return false;
	}
	NewSection->SetRange(TRange<FFrameNumber>::All());

	Cast<UMovieScene3DAttachSection>(AttachTrack->GetAllSections().Top())->bFullRevertOnDetach = true;

	// RestoreState completion mode + decoration on the attach section. Setting it explicitly
	// avoids depending on the project default, which determines whether the child returns to
	// its pre-track world position when scrubbing past the attach section's range.
	if (UMovieScene3DAttachSection* AttachSection = Cast<UMovieScene3DAttachSection>(NewSection))
	{
		AttachSection->Modify();
		AttachSection->EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::RestoreState);
	}

	return true;
}

/**
 * Author the default transform track that was added for a new attach track.
 * This function assumes that the transform track that we will find was added through a new binding and is not an existing
 * transform track.
 */
bool TryAuthorTransformTrack(UMovieScene* MovieScene, const FGuid& BindingGuid,
	const FTransform& ChildRelativeTransform, UTakeRecorderActorSource* AuthoringSource)
{
	if (!MovieScene)
	{
		return false;
	}

	MovieScene->Modify();

	const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
	const FFrameNumber StartFrame = UE::MovieScene::DiscreteInclusiveLower(PlaybackRange);
	const FFrameNumber EndFrame = UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange);

	TArray<UMovieSceneTrack*> FoundTracks = MovieScene->FindTracks(UMovieScene3DTransformTrack::StaticClass(), BindingGuid);

	UMovieScene3DTransformTrack* TransformTrack = !FoundTracks.IsEmpty()
		? Cast<UMovieScene3DTransformTrack>(FoundTracks.Top())
		: nullptr;

	if (TransformTrack)
	{
		// We need to check if there are are any sections with keys - if there are, we do not want to replace information to avoid data loss.
		if (TakesUtils::DoesTrackHaveAnyKeys(*TransformTrack))
		{
			return false;
		}
	}
	else
	{
		TransformTrack = MovieScene->AddTrack<UMovieScene3DTransformTrack>(BindingGuid);
	}

	if (!TransformTrack)
	{
		return false;
	}

	// Remove any and all existing data. We will build a new section.
	TransformTrack->RemoveAllAnimationData();

	UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(TransformTrack->CreateNewSection());

	if (!TransformSection)
	{
		return false;
	}

	TransformSection->Modify();
	TransformSection->SetRange(TRange<FFrameNumber>::All());
	DefaultTransformSectionToRelativeTransform(*TransformSection, StartFrame, EndFrame, ChildRelativeTransform);
	TransformSection->EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::RestoreState);

	TransformTrack->AddSection(*TransformSection);

	return true;
}

bool AddAttachTrackInActiveSequence(AActor* InChild, AActor* InParent, UTakeRecorderActorSource* InAuthoringSource, FName SocketName = NAME_None, FName ComponentName = NAME_None)
{
	if (!InChild || !InParent || !GEngine)
	{
		return false;
	}

	UTakeRecorderSubsystem* TakeRecorder = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>();
	ULevelSequence* RootSequence = TakeRecorder ? TakeRecorder->GetLevelSequence() : nullptr;
	UMovieScene* MovieScene = RootSequence ? RootSequence->GetMovieScene() : nullptr;
	if (!MovieScene || MovieScene->IsReadOnly())
	{
		return false;
	}

	TSharedPtr<ISequencer> Sequencer = FindSequencerForRootSequence(RootSequence);
	if (!Sequencer.IsValid())
	{
		return false;
	}

	const UE::TakesCore::FActorSequenceInformation ChildInfo(RootSequence, InChild);
	const UE::TakesCore::FActorSequenceInformation ParentInfo(RootSequence, InParent);

	if (HasExistingAttachSectionBinding(ChildInfo))
	{
		return true;
	}

	/*
	 * Capture the child's relative-to-parent transform NOW, while the child is still attached.
	 * We need this to seed the editor's default-track creation in TryAuthorTransformTrack. Captured before the detach so the relative reflects
	 * the actual offset to the parent, not the post-detach value (where the child has no parent and relative == world).
	 */
	FTransform PreDetachCapturedChildRelativeTransform = FTransform::Identity;
	if (USceneComponent* ChildRoot = InChild->GetRootComponent())
	{
		PreDetachCapturedChildRelativeTransform = ChildRoot->GetRelativeTransform();
	}

	{
		// Defer signed-object change broadcasts across all our mutations, then force-flush on scope
		// exit so the sequencer picks up the new binding + attach track before we notify it below.
		UE::MovieScene::FScopedSignedObjectModifyDefer Defer(/*bInForceFlush=*/true);

		RootSequence->Modify();

		// Detach before the bindings have captured their defaults if we are already attached. Detach with KeepWorldTransform
		// so the child stays put visibly; the attach section's first evaluation re-attaches it.
		// Pre-animated transform state for the attach gets cached at first eval - because the
		// child is detached at that moment with world == relative == W, the cached pre-track
		// value is W. RestoreState + KeepWorld attach rules then restore the child to W when
		// scrubbing past the attach section.
		if (!DetachChildPreservingWorldTransform(InChild, InParent))
		{
			return false;
		}

		// Create bindings after detach so the editor's default-track creation
		// captures the correct world value into the transform section's channels.
		// This prevents any movement.
		const FGuid ChildGuid = ResolveOrCreateRootBinding(Sequencer, RootSequence, ChildInfo);
		const FGuid ParentGuid = ResolveOrCreateRootBinding(Sequencer, RootSequence, ParentInfo);
		if (!ChildGuid.IsValid() || !ParentGuid.IsValid())
		{
			return false;
		}

		if (!TryAuthorAttachTrackWithConstraint(MovieScene, ChildGuid, ParentGuid,
			ParentInfo.GetControllingSequenceID(), SocketName, ComponentName, InAuthoringSource))
		{
			return false;
		}

		// Key the transform track
		if (!TryAuthorTransformTrack(MovieScene, ChildGuid, PreDetachCapturedChildRelativeTransform, InAuthoringSource))
		{
			return false;
		}
	} // Defer flushes here, ensuring compiled data sees our changes before the refresh below.

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshAllImmediately);
	return true;
}

#endif // WITH_EDITOR

} // namespace UE::TakeRecorderActorSource::Private

UTakeRecorderSource* UTakeRecorderActorSource::AddSourceForActor(AActor* InActor, UTakeRecorderSources* InSources)
{
	if (InSources == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The Source is invalid."), ELogVerbosity::Error);
		return nullptr;
	}
	
	if (InActor == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The Actor is invalid."), ELogVerbosity::Error);
		return nullptr;
	}

	if (!TakeRecorderSourcesUtils::IsActorRecordable(InActor))
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("The Actor: %s is not recordable."), *InActor->GetPathName()), ELogVerbosity::Error);
		return nullptr;
	}

#if WITH_EDITOR
	//Look through our sources and see if one actor matches the incoming one either from editor or PIE world.
	{
		//Cache  InputActor comparison data
		const bool bIsAlreadyPIEActor = InActor->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor);
		const AActor* InputActorEditor = EditorUtilities::GetEditorWorldCounterpartActor(InActor);
		const AActor* InputActorPIE = EditorUtilities::GetSimWorldCounterpartActor(InActor);

		TArray<UTakeRecorderSource*> SourceArray = InSources->GetSourcesCopy();
		for (UTakeRecorderSource* CurrentSource : SourceArray)
		{
			UTakeRecorderActorSource* CurrentActorSource = Cast<UTakeRecorderActorSource>(CurrentSource);
			if (CurrentActorSource != nullptr)
			{
				AActor* CurrentActor = CurrentActorSource->Target.Get();
				if (CurrentActor == nullptr)
				{
					continue;
				}

				if (InActor == CurrentActor)
				{
					return CurrentActorSource;
				}
				else 
				{
					if (bIsAlreadyPIEActor)
					{
						//The input actor is from PIE -> Bring it into Editor world and compare. 
						if (InputActorEditor == CurrentActor)
						{
							return CurrentActorSource;
						}
					}
					else
					{
						//The input actor is from Editor -> Bring it into PIE world and compare. 
						if (InputActorPIE == CurrentActor)
						{
							return CurrentActorSource;
						}
					}
				}
			}
		}
	}
#endif // WITH_EDITOR

	UTakeRecorderActorSource* NewSource = InSources->AddSource<UTakeRecorderActorSource>();
	NewSource->SetSourceActor(InActor);
	return NewSource;
}

void UTakeRecorderActorSource::RemoveActorFromSources(AActor* InActor, UTakeRecorderSources* InSources)
{
	if (InSources == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The Source to remove from is invalid."), ELogVerbosity::Error);
		return;
	}

	if (InActor == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("The Actor to remove is invalid."), ELogVerbosity::Error);
		return;
	}

#if WITH_EDITOR
	//Cache  InputActor comparison data
	const bool bIsAlreadyPIEActor = InActor->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor);
	const AActor* InputActorEditor = EditorUtilities::GetEditorWorldCounterpartActor(InActor);
	const AActor* InputActorPIE = EditorUtilities::GetSimWorldCounterpartActor(InActor);

	TArray<UTakeRecorderSource*> SourceArray = InSources->GetSourcesCopy();
	for (UTakeRecorderSource* CurrentSource : SourceArray)
	{
		UTakeRecorderActorSource* CurrentActorSource = Cast<UTakeRecorderActorSource>(CurrentSource);
		if (CurrentActorSource != nullptr)
		{
			const AActor* CurrentActor = CurrentActorSource->Target.Get();
			if (CurrentActor == nullptr)
			{
				continue;
			}

			if (InActor == CurrentActor)
			{
				InSources->RemoveSource(CurrentSource);
			}
			else
			{
				if (bIsAlreadyPIEActor)
				{
					//The input actor is from PIE -> Bring it into Editor world and compare. 
					if (InputActorEditor == CurrentActor)
					{
						InSources->RemoveSource(CurrentSource);
					}
				}
				else
				{
					//The input actor is from Editor -> Bring it into PIE world and compare. 
					if (InputActorPIE == CurrentActor)
					{
						InSources->RemoveSource(CurrentSource);
					}
				}
			}
		}
	}
#endif // WITH_EDITOR
}

namespace TakeRecorderActorSource
{
static bool bAllowsSpawnableObjects = true;
}

bool UTakeRecorderActorSource::AllowsSpawnableObjects() { return TakeRecorderActorSource::bAllowsSpawnableObjects; }
void UTakeRecorderActorSource::SetAllowsSpawnableObjects(bool bInAllowsSpawnableObjects) { TakeRecorderActorSource::bAllowsSpawnableObjects  = bInAllowsSpawnableObjects; }

UTakeRecorderActorSource::UTakeRecorderActorSource(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	RecordType = ETakeRecorderActorRecordType::ProjectDefault;
	bReduceKeys = true;
	AttachRecordBehaviour = ETakeRecorderAttachRecordBehaviour::ProjectDefault;
	bShowProgressDialog = true;

	TargetSequenceID = MovieSceneSequenceID::Invalid;

	// Build the property map on initialization so that sources created at runtime have a default map
	RebuildRecordedPropertyMap();
}

bool UTakeRecorderActorSource::TickRecovery(float /*DeltaTime*/)
{
#if WITH_EDITOR
	if (!Target.IsValid() && !CachedActorLabel.IsEmpty() && bTargetIsSequencerActor)
	{
		TryRecoverTarget();
	}
#endif
	return true;
}

void UTakeRecorderActorSource::TryRecoverTarget(const ULevelSequence* InSequenceToSkip, const ULevelSequence* InRootSequenceToSkip)
{
#if WITH_EDITOR
	if (CachedActorLabel.IsEmpty() || !GEditor)
	{
		return;
	}

	if (AActor* RecoveredActor = UE::TakeRecorderActorSource::Private::TryLocateBoundActorInSequencerAndWorld(
		InSequenceToSkip, InRootSequenceToSkip, CachedObjectBindingGuid, CachedActorLabel))
	{
		Target = RecoveredActor;
		if (UTakeRecorderActorSource* Original = WeakOriginalSource.Get())
		{
			Original->Target = RecoveredActor;
		}
	}
#endif
}

void UTakeRecorderActorSource::Initialize()
{
#if WITH_EDITOR
	if (!HasAnyFlags(RF_ClassDefaultObject) && GEngine)
	{
		GEngine->OnLevelActorAttached().AddUObject(this, &UTakeRecorderActorSource::HandleActorAttached);
		GEngine->OnLevelActorDetached().AddUObject(this, &UTakeRecorderActorSource::HandleActorDetached);
	}
#endif // WITH_EDITOR
}

bool UTakeRecorderActorSource::IsValid() const
{
	return (Target.IsValid());
}

TArray<UTakeRecorderSource*> UTakeRecorderActorSource::PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer)
{
	// Don't bother doing anything if we don't have a valid actor to record.
	if (!Target.IsValid())
	{
		// Only attempt target recovery for Sequencer-managed spawnables.
		if (CachedActorLabel.IsEmpty() || !bTargetIsSequencerActor)
		{
			return TArray<UTakeRecorderSource*>();
		}

		TryRecoverTarget(InSequence, InRootSequence);

		if (Target.IsValid())
		{
			UE_LOG(LogTakesCore, Warning, TEXT("Take Recorder: Actor source target '%s' was invalid (likely a Sequencer spawnable replaced when the recording sequence was opened). Recovered successfully."), *CachedActorLabel);
		}
		else
		{
			UE_LOG(LogTakesCore, Warning, TEXT("Take Recorder: Actor source target '%s' was invalid and could not be recovered. This source will be skipped."), *CachedActorLabel);
			return TArray<UTakeRecorderSource*>();
		}
	}
	// We used to do this at the PostRecording but other Actor sources may need to check you recorded an animation so we keep it around
	TrackRecorders.Empty();

	// Resolve which actor we wish to record 
	AActor* ActorToRecord = Target.Get();
	TargetLevelSequence = InSequence;
	RootLevelSequence = InRootSequence;
	TargetSequenceID = InSequenceID;

	if (ResolveAttachRecordBehaviour() != ETakeRecorderAttachRecordBehaviour::None)
	{
		// Ensure our sequence is compiled so that we can find any potential parent bindings that exist.
		if (UMovieSceneCompiledDataManager* CompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData())
		{
			// This will take a fast exit if it is already compiled and not dirty.
			CompiledDataManager->Compile(InRootSequence);
		}
	}

	// Resolve which movie scene our data should go into
	UMovieScene* MovieScene = TargetLevelSequence->GetMovieScene();

	FString ObjectBindingName = ActorToRecord->GetName();

	UE::TakesCore::FActorSequenceInformation SequenceInfo(RootLevelSequence, ActorToRecord);
	bTargetIsSequencerActor = SequenceInfo.IsControlledBySequence();

	// Look to see if the movie scene already has our object binding in it (which is common if we're recording a new take) so we can
	// replace the data that was already there...
	CachedObjectBindingGuid = SequenceInfo.GetCachedObjectBindingGuid();

	// Fallback as a last attempt to find binding guid if not found and we are trying to overwrite a spawnable.
	if (!CachedObjectBindingGuid.IsValid() && !GetRecordToPossessable() && ResolveSpawnableOverwriteBehaviour() == ETakeRecorderSpawnableOverwriteBehavior::OverwriteLegacy)
	{
	#if WITH_EDITOR
		const FString ActorLabel = ActorToRecord->GetActorLabel();
	#else
		const FString ActorLabel = ActorToRecord->GetName();
	#endif
		for (int32 i = 0; i < MovieScene->GetPossessableCount(); ++i)
		{
			const FMovieScenePossessable& Possessable = MovieScene->GetPossessable(i);
			if (Possessable.GetName() == ActorLabel)
			{
				CachedObjectBindingGuid = Possessable.GetGuid();
				break;
			}
		}
	}

	bool bReuseExistingBinding = CheckIfBindingCanBeReused(SequenceInfo);

	// Rengenerate the binding if this is a spawnable or recording into subsequences.
	bool bRegenerateBindingGuid = SequenceInfo.IsSpawnable();

	if (UTakeRecorderSources* Sources = GetTypedOuter<UTakeRecorderSources>();
		Sources != nullptr && Sources->GetSettings().bRecordSourcesIntoSubSequences)
	{
		bRegenerateBindingGuid = true;
		bReuseExistingBinding = false;
	}

	// Recording operates on a DuplicateObject copy of each source; the Take Recorder panel
	// shows the original. Store a weak pointer to the original now so FinalizeRecording can
	// update its Target after recording ends. The original is identified by matching
	// CachedActorLabel and living in a different level sequence than the recording copy
	// (TargetLevelSequence). We cannot use Target validity as a filter because the per-tick
	// recovery check may have already restored the original's Target before PreRecording runs.
	if (bTargetIsSequencerActor)
	{
		for (UTakeRecorderActorSource* OtherSource : TObjectRange<UTakeRecorderActorSource>())
		{
			if (OtherSource != this &&
				OtherSource->CachedActorLabel == CachedActorLabel &&
				OtherSource->GetTypedOuter<ULevelSequence>() != TargetLevelSequence)
			{
				WeakOriginalSource = OtherSource;
				break;
			}
		}
		// Also update the original immediately so the panel shows the correct actor
		// during the window between initialization and when recording starts.
		if (UTakeRecorderActorSource* Original = WeakOriginalSource.Get())
		{
			Original->Target = ActorToRecord;
			Original->CachedObjectBindingGuid = CachedObjectBindingGuid;
		}
	}
	// Don't clear existing data if we're recording into an existing take as a spawnable. We want the original object, and its recorded data,
	// to persist throughout this new recording. Sequencer actors also need to have their data preserved.
	if (!bTargetIsSequencerActor && OverwriteExistingTrackData())
	{
		CleanExistingDataFromSequence(CachedObjectBindingGuid, *TargetLevelSequence);
		// CleanExistingDataFromSequence removed the binding that CachedObjectBindingGuid referred to.
		// Reset it so the possessable/spawnable paths below always create a fresh binding rather than
		// treating a stale (now-removed) possessable GUID as valid.
		CachedObjectBindingGuid.Invalidate();
	}
	
	// Initialize the header for this actor in the Manifest Serializer for streaming data capture.
#if WITH_EDITOR
	const FString ActorId = ActorToRecord->GetActorLabel();
#else // WITH_EDITOR
	const FString ActorId = ActorToRecord->GetName();
#endif // WITH_EDITOR

	FName SerializedType("Actor");
	FActorFileHeader Header(ObjectBindingName, ActorId, SerializedType, ActorToRecord->GetClass()->GetPathName(), false);

	if (GetRecordToPossessable())
	{
		// If a user adds a PIE-only instance as a recordable object, they can't record this to a possessable (because the binding will be broken when they exit PIE).
		if (!CVarTakeRecorderAllowPossessablePIEObjects->GetBool() &&
			Target->GetWorld()->WorldType != EWorldType::Editor 
		#if WITH_EDITOR
			&& GEditor && !GEditor->ObjectsThatExistInEditorWorld.Get(Target.Get())
		#endif // WITH_EDITOR
			)
		{
			UE_LOGF(LogTakesCore, Warning, "Attempted to record an actor that does not exist in the editor world to a possessable. Forcing recording of %ls as a Spawnable so that the resulting binding is not broken!", *Target->GetName());
			RecordType = ETakeRecorderActorRecordType::Spawnable;
		}
		else
		{
			// Create a Possessable object binding in the Sequence and then bind it to our actor
			if (!bTargetIsSequencerActor || bRegenerateBindingGuid)
			{
			#if WITH_EDITOR
				const FString ActorID = ActorToRecord->GetActorLabel();
			#else // WITH_EDITOR
				const FString ActorID = ActorToRecord->GetName();
			#endif // WITH_EDITOR
				CachedObjectBindingGuid = MovieScene->AddPossessable(ActorID, ActorToRecord->GetClass());
				check(CachedObjectBindingGuid.IsValid());
				TargetLevelSequence->BindPossessableObject(CachedObjectBindingGuid, *ActorToRecord, ActorToRecord->GetWorld());
			}
			else
			{
				// The binding already exists in the recording sequence, inherited from the source sequence
				// via duplication, or owned by Sequencer for spawnable actors.
				ULevelSequence* TemplateSequence = InRootSequence ? InRootSequence : InSequence;
				CachedObjectTemplate = Cast<AActor>(MovieSceneHelpers::GetObjectTemplate(TemplateSequence, CachedObjectBindingGuid, MovieSceneHelpers::CreateTransientSharedPlaybackState(ActorToRecord, TemplateSequence)));
			}

			Header.bRecordToPossessable = true;
		}
	}
	
	if (!GetRecordToPossessable())
	{
		// We need to store the object template in the Movie Scene (because it's a complex UObject)
		// instead of trying to place this data into the non-UObject safe data stream.
		// Note: Sequencer-managed actors (bTargetIsSequencerActor) are handled via the possessable path
		// above, with their inherited spawnable GUID already resolved before we reach this block.

		if (SequenceInfo.IsSpawnable())
		{
			if (TStrongObjectPtr<UMovieSceneSequence> StrongControllingSequence = SequenceInfo.GetControllingSequence().Pin())
			{
				// Grab the object template before we do any kind of binding replacement to ensure we have a valid template for spawnables.
				CachedObjectTemplate = Cast<AActor>(MovieSceneHelpers::GetObjectTemplate(StrongControllingSequence.Get(), CachedObjectBindingGuid, MovieSceneHelpers::CreateTransientSharedPlaybackState(ActorToRecord, StrongControllingSequence.Get())));
			}
		}

		// Always create or replace binding when not dealing with a spawnable actor.
		if (!SequenceInfo.IsSpawnable() || !CachedObjectBindingGuid.IsValid() || bRegenerateBindingGuid)
		{
		#if WITH_EDITOR
			UE::Sequencer::FCreateBindingParams CreateBindingParams;
			CreateBindingParams.bSpawnable = true;
			CreateBindingParams.bAllowCustomBinding = true;
			CreateBindingParams.BindingNameOverride = ActorToRecord->GetActorLabel();
			CachedObjectBindingGuid = FSequencerUtilities::CreateOrReplaceBinding(nullptr, InSequence, ActorToRecord, CreateBindingParams);
		#else // WITH_EDITOR
			CachedObjectBindingGuid = MovieSceneHelpers::TryCreateCustomSpawnableBinding(InSequence, ActorToRecord);
		#endif // WITH_EDITOR

			// SetupDefaults (called inside CreateOrReplaceBinding) adds a spawn track so the
			// actor is immediately spawned when the recording sequence is evaluated by the
			// open Sequencer. This produces a duplicate instance alongside the original level
			// actor. Remove the spawn track now (before Sequencer's RefreshTree evaluates it).
			// We re-add it in FinalizeRecording so the saved take plays back correctly.
			if (UMovieScene* RecordingMovieScene = InSequence->GetMovieScene())
			{
				if (UMovieSceneTrack* SpawnTrack = RecordingMovieScene->FindTrack(
					UMovieSceneSpawnTrack::StaticClass(), CachedObjectBindingGuid))
				{
					RecordingMovieScene->RemoveTrack(*SpawnTrack);
					bRemovedSpawnTrack = true;
				}
			}
		}

		if (!CachedObjectTemplate.IsValid())
		{
			// Try again to grab the template for anything we missed.
			CachedObjectTemplate = Cast<AActor>(MovieSceneHelpers::GetObjectTemplate(InSequence, CachedObjectBindingGuid, MovieSceneHelpers::CreateTransientSharedPlaybackState(ActorToRecord, InSequence)));
		}
		
		if (CachedObjectTemplate.IsValid())
		{
			Header.TemplateName = CachedObjectTemplate->GetName();
			PostProcessCreatedObjectTemplateImpl(CachedObjectTemplate.Get());
		}
	}

	Header.Guid = CachedObjectBindingGuid;
	if (InManifestSerializer)
	{
		FManifestProperty  ManifestProperty(ObjectBindingName, FName("Actor"), CachedObjectBindingGuid);
		InManifestSerializer->WriteFrameData(InManifestSerializer->FramesWritten, ManifestProperty);
	
		FString AssetPath = InManifestSerializer->GetLocalCaptureDir();  

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		if (!PlatformFile.DirectoryExists(*AssetPath))
		{
			PlatformFile.CreateDirectory(*AssetPath);
		}

		AssetPath = AssetPath / ObjectBindingName;
		if (!PlatformFile.DirectoryExists(*AssetPath))
		{
			PlatformFile.CreateDirectory(*AssetPath);
		}

		ActorSerializer.SetLocalCaptureDir(AssetPath);

		FText Error;

		FString FileName = FString::Printf(TEXT("%s_%s"), *SerializedType.ToString(), *ObjectBindingName);

		if (!ActorSerializer.OpenForWrite(FileName, Header, Error))
		{
			//UE_LOGF(LogFrameTransport, Error, "Cannot open frame debugger cache %ls. Failed to create archive.", *InFilename);
			UE_LOGF(ActorSerialization, Warning, "Error Opening Actor Sequencer File: Subject '%ls' Error '%ls'", *ObjectBindingName, *Error.ToString());
		}

	}

	// Now we need to create the section recorders for each of the enabled properties based on the property map.
	// Any components spawned at runtime will get picked up on Tick and have section recorders created for them mid-recording.
	// For Sequencer-managed actors in preserve mode, skip section recorder creation only when the binding already
	// has existing tracks in the recording sequence. If there are no tracks yet, we still need to record them.
	// When bReuseExistingBinding is set, we must still run section recorders to capture the take into the reused binding.
	bool bSkipSectionRecorders = false;
	if (bTargetIsSequencerActor && !OverwriteExistingTrackData() && !bReuseExistingBinding)
	{
		const UMovieScene* RecordingMovieScene = TargetLevelSequence ? TargetLevelSequence->GetMovieScene() : nullptr;
		const FMovieSceneBinding* ExistingBinding = RecordingMovieScene ? RecordingMovieScene->FindBinding(CachedObjectBindingGuid) : nullptr;

		// if the existing binding is null, check in the originating controlling sequence from the Sequence Information
		if (ExistingBinding == nullptr)
		{
			if (TStrongObjectPtr<UMovieSceneSequence> StrongControllingSequence = SequenceInfo.GetControllingSequence().Pin())
			{
				RecordingMovieScene = StrongControllingSequence->GetMovieScene();
				ExistingBinding = RecordingMovieScene ? RecordingMovieScene->FindBinding(CachedObjectBindingGuid) : nullptr;
			}
		}

		bSkipSectionRecorders = ExistingBinding && ExistingBinding->GetTracks().Num() > 0;
	}

	// Update our cached list of components so that we don't detect them all as new components on the first Tick
	GetAllComponents(CachedComponentList, false);

	// Walk our parent chain until we get to the root and make sure all of our parent actors are recorded. This allows our transforms
	// to always be in local space (conversion to world space can be done in Sequencer via baking transforms) and attach tracks
	EnsureParentHierarchyIsReferenced(TargetLevelSequence);

	// Create any new Actor Sources for Actors that we reference (either parents or attached components that belong to other actors)
	CreateNewActorSourceForReferencedActors();

	if (!bSkipSectionRecorders)
	{
		TArray<UObject*> TraversedObjects;
		CreateSectionRecordersRecursive(ActorToRecord, RecordedProperties, TraversedObjects);
	}

	// We might have generated new Actors to be recorded so that the current actor can be recorded.
	// We may have added our parents (so that transforms work) or we might have added an actor who
	// has a component that is currently attached to us.
	return AddedActorSources;
}

void UTakeRecorderActorSource::OnAllSourcesPreRecordingFinished()
{
	DoCreatePendingTrackRecorders();
}

void UTakeRecorderActorSource::EnsureParentHierarchyIsReferenced()
{
	EnsureParentHierarchyIsReferenced(TargetLevelSequence);
}

void UTakeRecorderActorSource::EnsureParentHierarchyIsReferenced(ULevelSequence* InLevelSequence)
{
	if (ResolveAttachRecordBehaviour() != ETakeRecorderAttachRecordBehaviour::Hierarchy || InLevelSequence == nullptr)
	{
		return;
	}

	if (!Target.IsValid())
	{
		return;
	}

	if (!Target->GetRootComponent())
	{
		return;
	}

	// We need to start with our parent so that we don't try to add another recording for ourself
	// as we're already in the process of creating a recording for ourself!
	USceneComponent* CurrentComponent = Target.IsValid()
		? Target->GetRootComponent()->GetAttachParent()
		: nullptr;

	while (CurrentComponent)
	{
		AActor* Owner = CurrentComponent->GetOwner();
		NewReferencedActors.Add(Owner);
		// We can skip forward to the root component for that Actor as it'll end up adding all of its children
		// which will include the component we may be attached to.
		CurrentComponent = Owner->GetRootComponent()->GetAttachParent();
	}
}

void UTakeRecorderActorSource::StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence)
{
	// Now that every source has had PreRecording called on it and we're about to start recording, iterate through each one and set
	// their starts to the most up to date time sample.
	for (UMovieSceneTrackRecorder* Recorder : TrackRecorders)
	{
		if (!Recorder)
		{
			UE_LOGF(LogTakesCore, Error, "Start Recording: Actor source %ls has an invalid recorder.", *Target.GetAssetName());
			continue;
		}
		// The Frame Number is the sequence time we should record into, but we want to get
		// a Timecode stamp that reflects the real time it was recorded... This kind of conflicts
		// with 
		Recorder->SetSectionStartTimecode(InSectionStartTimecode, InSectionFirstFrame);
	}
}

void UTakeRecorderActorSource::CreateSectionRecordersRecursive(UObject* ObjectToRecord, UActorRecorderPropertyMap* PropertyMap, TArray<UObject*>& TraversedObjects)
{
	if (TraversedObjects.Contains(ObjectToRecord))
	{
		return;
	}

	TraversedObjects.Add(ObjectToRecord);
	
	// We asked external modules whether this object can be recorded when we generated PropertyMap. They might have changed their mind,
	// e.g. if CanRecord depends on some dynamic state of the object, which the user has toggled since the last call.
	using namespace UE::TakeRecorderSources;
	if (!FTakeRecorderSourcesModule::Get().CanRecord(FCanRecordArgs(ObjectToRecord)))
	{
		return;
	}

	FGuid Guid = CachedObjectBindingGuid;
	if (ObjectToRecord->IsA<UActorComponent>())
	{
		UActorComponent* Component = Cast<UActorComponent>(ObjectToRecord);
		// Components are duplicated into the Object Template that belongs inside of the owning Movie Scene.
		// A Spawnable Object Binding is created tied to it to re-create the actor itself, but to record
		// properties about components on that object we create Possessable Object Bindings instead and bind the 
		// Possessable to the object inside the template that gets spawned.

		
		// This can be called even on Possessables (and is encouraged to do so as it does a sanity check and a warning if a dynamically added component is put on a Possessable)
		// This will update the Object Template with the given component if it does not already have a component with the same relative path.
		UActorComponent* NewlyDuplicatedComponent = nullptr;
		bool bNewComponentAdded = EnsureObjectTemplateHasComponent(Component, NewlyDuplicatedComponent);

		
		UMovieScene* MovieScene = TargetLevelSequence->GetMovieScene();
		Guid = FGuid();
		for (int32 PossessableCount = 0; PossessableCount < MovieScene->GetPossessableCount(); ++PossessableCount)
		{
			const FMovieScenePossessable& Possessable = MovieScene->GetPossessable(PossessableCount);
			if (Possessable.GetParent() == CachedObjectBindingGuid && Possessable.GetName() == Component->GetName()
			#if WITH_EDITOR
				&& Possessable.GetPossessedObjectClass() == Component->GetClass()
			#endif // WITH_EDITOR
				)
			{
				Guid = Possessable.GetGuid();
				break;
			}
		}

		UActorComponent* ComponentToRecord = bNewComponentAdded ? NewlyDuplicatedComponent : Component;
		// cbb: Not sure what this is accomplishing
		if (!Guid.IsValid())
		{
			Guid = MovieScene->AddPossessable(ComponentToRecord->GetName(), ComponentToRecord->GetClass());
		}

		// Set up parent/child guids for possessables within spawnables
		FMovieScenePossessable* ChildPossessable = MovieScene->FindPossessable(Guid);
		if (ensure(ChildPossessable))
		{
			ChildPossessable->SetParent(CachedObjectBindingGuid, MovieScene);
		}

		// Bindings are stored relative to their context outer. Newly duplicated components have a different outer
		TargetLevelSequence->BindPossessableObject(Guid, *ComponentToRecord, bNewComponentAdded ? CachedObjectTemplate.Get() : Target.Get());

		FActorProperty  ActorCompFrame(ComponentToRecord->GetName(), FName("Component"), Guid);
		ActorCompFrame.Type = EActoryPropertyType::ComponentType;
		ActorCompFrame.BindingName = Target.Get()->GetName();
		ActorCompFrame.ClassName = ComponentToRecord->GetClass()->GetName();
		ActorSerializer.WriteFrameData(ActorSerializer.FramesWritten, ActorCompFrame);
	}

	auto AddSectionRecorder = [this, ObjectToRecord, Guid](UMovieSceneTrackRecorder* SectionRecorder,
		const TSubclassOf<UMovieSceneTrackRecorderSettings>& SettingsClass)
		{
			TrackRecorders.Add(SectionRecorder);
			
			FCreateTrackArgs CreateTrackArgs
			{
				SectionRecorder,
				ObjectToRecord,
				GetSettingsObjectForFactory(SettingsClass),
				Guid
			};
			PendingTrackRecorders.Add(MoveTemp(CreateTrackArgs));
		};
	
	// We need to iterate through the Property Map to see if the user wants to record this property or not
	// We store the name of the Factory that can do the recording in the Property Map so for now we shortcut
	// and just look up Factories by name instead of replicating all of the logic that goes into initializing
	// the property map.
	for (FActorRecordedProperty& Property : PropertyMap->Properties)
	{
		// Skip any properties that the user doesn't want to record
		if (!Property.bEnabled)
		{
			continue;
		}

		TArray<IMovieSceneTrackRecorderFactory*> ModularFactories = IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneTrackRecorderFactory>(MovieSceneSectionRecorderFactoryName);
		TArray<IMovieSceneTrackRecorderFactory*> PropertyInstanceFactories;
		bool bFoundRecorder = false;
		for (IMovieSceneTrackRecorderFactory* Factory : ModularFactories)
		{
			// Check this is the right factory for the property, and that it can be created with the current settings.
			// cbb: Can't FText == FText?
			if (Factory->GetDisplayName().ToString() == Property.RecorderName.ToString() && Factory->CanCreateTrackRecorderForHost(this, Target.Get()))
			{
				if (UMovieSceneTrackRecorder* SectionRecorder = Factory->CreateTrackRecorderForObject())
				{
					if (Factory->IsSerializable())
					{
						AActor* ActorToRecord = Target.Get();
						FString Name =  ObjectToRecord->GetName();
						FActorProperty  ActorFrame(Name, Factory->GetSerializedType(), Guid);
						ActorSerializer.WriteFrameData(ActorSerializer.FramesWritten, ActorFrame);
						SectionRecorder->SetSavedRecordingDirectory(ActorSerializer.GetLocalCaptureDir());
					}
					
					AddSectionRecorder(SectionRecorder, Factory->GetSettingsClass());

					bFoundRecorder = true;
					break;
				}
			}
			
			if (Factory->CanHandlePropertyInstance())
			{
				PropertyInstanceFactories.Add(Factory);
			}
		}

		if (!bFoundRecorder)
		{
			// Our current fallback property recorder isn't registered in the modular factories list so that it always goes last.
			TArray<FString> PropertyNames;
			Property.PropertyName.ToString().ParseIntoArray(PropertyNames, TEXT("."));

			FProperty* PropertyInstance = nullptr;
			UStruct* SearchStruct = ObjectToRecord->GetClass();
			for (auto PropertyStringName : PropertyNames)
			{
				PropertyInstance = SearchStruct ? SearchStruct->FindPropertyByName(FName(*PropertyStringName)) : nullptr;
				SearchStruct = nullptr;
				if (PropertyInstance)
				{
					if (FStructProperty* AsStructProperty = CastField<FStructProperty>(PropertyInstance))
					{
						SearchStruct = AsStructProperty->Struct;
					}
				}
				if (!PropertyInstance) break;
			}

			if (PropertyInstance)
			{
				auto CreateTrackRecorderForProperty = [&](const IMovieSceneTrackRecorderFactory& TrackRecorderFactory)
				{
					if (UMovieSceneTrackRecorder* SectionRecorder = TrackRecorderFactory.CreateTrackRecorderForProperty(ObjectToRecord, Property.PropertyName))
					{
						if (TrackRecorderFactory.IsSerializable())
						{
							AActor* ActorToRecord = Target.Get();
							FString Name =  ObjectToRecord->GetName();
							FActorProperty  ActorFrame(Name, TrackRecorderFactory.GetSerializedType(), Guid);
							ActorFrame.Type = EActoryPropertyType::PropertyType;
							ActorFrame.PropertyName = Property.PropertyName.ToString();
							ActorSerializer.WriteFrameData(ActorSerializer.FramesWritten, ActorFrame);
							SectionRecorder->SetSavedRecordingDirectory(ActorSerializer.GetLocalCaptureDir());
						}
						
						AddSectionRecorder(SectionRecorder, TrackRecorderFactory.GetSettingsClass());
					}
				};
				
				auto FindPropertyInstanceFactoryLambda = [ObjectToRecord, PropertyInstance](const IMovieSceneTrackRecorderFactory* Factory)
				{
					return Factory->CanRecordProperty(ObjectToRecord, PropertyInstance);
				};
				
				// First check if there's a dedicated recorder to handle a property instance.
				if (IMovieSceneTrackRecorderFactory** PropertyInstanceFactoryPtr = PropertyInstanceFactories.FindByPredicate(FindPropertyInstanceFactoryLambda))
				{
					IMovieSceneTrackRecorderFactory* PropertyInstanceFactory = *PropertyInstanceFactoryPtr;
					check(PropertyInstanceFactory->CanHandlePropertyInstance());
					CreateTrackRecorderForProperty(*PropertyInstanceFactory);
				}
				else
				{
					// Fall back to the generic property recorder factory.
					FMovieScenePropertyTrackRecorderFactory TrackRecorderFactory;
					if (TrackRecorderFactory.CanRecordProperty(ObjectToRecord, PropertyInstance))
					{
						CreateTrackRecorderForProperty(TrackRecorderFactory);
					}
				}
			}
			else 
			{
				UE_LOGF(LogTakesCore, Warning, "Unable to find property %ls. Cannot record.", *ObjectToRecord->GetName()); 
			}
		}
	}

	// Now iterate through any children and repeat.
	for (UActorRecorderPropertyMap* Child : PropertyMap->Children)
	{
		UObject* ChildObject = Child->RecordedObject.Get();
		if (ChildObject)
		{
			CreateSectionRecordersRecursive(ChildObject, Child, TraversedObjects);
		}
		else
		{
			UE_LOGF(LogTakesCore, Warning, "Attempted to resolve soft object path %ls but failed, skipping entire child hierarchy for recording. This is likely because the object only exists at edit time. Ideally we would filter out these and not create entries in the Property Map, but they may want to record editor-only objects at edit time.", 
				*Child->RecordedObject.ToString());
		}
	}
}

void UTakeRecorderActorSource::TickRecording(const FQualifiedFrameTime& CurrentSequenceTime)
{
	// Each frame we want to compare against the list of components we were recording last frame. 
	// This will allow us to detect newly added components and components that were removed at runtime,
	// which allows us to properly update their resulting spawn track.
	TSet<TWeakObjectPtr<UActorComponent>> CurrentComponentList;
	GetAllComponents(CurrentComponentList, false);

	TArray<TWeakObjectPtr<UActorComponent>> NewComponentsAdded;
	TArray<TWeakObjectPtr<UActorComponent>> NewComponentsRemoved;

	for (TWeakObjectPtr<UActorComponent> CurrentComponent : CurrentComponentList)
	{
		// Track any components added to our list this frame
		if (CurrentComponent.IsValid() && !CachedComponentList.Contains(CurrentComponent))
		{
			NewComponentsAdded.Add(CurrentComponent);
		}
	}

	for (TWeakObjectPtr<UActorComponent> OldComponent : CachedComponentList)
	{
		// Now do the reverse and mark any components that have been removed.
		if (OldComponent.IsValid() && !CurrentComponentList.Contains(OldComponent))
		{
			NewComponentsRemoved.Add(OldComponent);
		}
	}

	TArray<UObject*> TraversedObjects;
	for (TWeakObjectPtr<UActorComponent> AddedComponent : NewComponentsAdded)
	{
		if (Target.IsValid() && AddedComponent.IsValid())
		{
			UE_LOGF(LogTakesCore, Log, "Detected newly added component %ls on Actor %ls, beginning to record component's properties now.", *AddedComponent->GetReadableName(), *Target->GetName());
			TSet<UMovieSceneTrackRecorder*> PreviousTrackRecorders = TSet<UMovieSceneTrackRecorder*>(TrackRecorders);

			// We should create a new property map attached to the right parent, and then initialize it using existing flow. This works for Possessables too as it will throw a warning that
			// the binding will be broken.
			UActorRecorderPropertyMap* ComponentPropertyMap = NewObject<UActorRecorderPropertyMap>(this, MakeUniqueObjectName(this, UActorRecorderPropertyMap::StaticClass(), AddedComponent->GetFName()), RF_Transactional);
			ComponentPropertyMap->RecordedObject = AddedComponent.Get();

			// Add the new property map as a child of the correct parent, otherwise recursion doesn't work when we try to update the cached number of recorded properties.
			UActorRecorderPropertyMap* ParentPropertyMap = GetParentPropertyMapForComponent(AddedComponent.Get());
			if (ParentPropertyMap)
			{
				ParentPropertyMap->Children.Add(ComponentPropertyMap);
			}
			else
			{
				UE_LOGF(LogTakesCore, Log, "A component %ls was added to actor %ls at runtime but we couldn't find the property map for the parent. Is the parent no longer valid?", *AddedComponent->GetName(), *Target->GetName());
			}

			// Create the Property Map
			RebuildRecordedPropertyMapRecursive(AddedComponent.Get(), ComponentPropertyMap);

			// Create the Section Recorders required
			CreateSectionRecordersRecursive(AddedComponent.Get(), ComponentPropertyMap, TraversedObjects);

			// Ensure our track recorders are fully created after creating the initial sections.
			DoCreatePendingTrackRecorders();
			
			// Update our numbers on the display
			UpdateCachedNumberOfRecordedProperties();

			// We need to call StartRecording on only the Track Recorders created in this situation.
			for (UMovieSceneTrackRecorder* TrackRecorder : TrackRecorders)
			{
				// If the track recorder existed before we added this component then it has already had StartRecording called on it.
				if (!TrackRecorder || PreviousTrackRecorders.Contains(TrackRecorder))
				{
					continue;
				}
				
				// cbb: This should match the logic in TakeRecorderSources if changed.
				FFrameNumber FirstFrameOfSequence = CurrentSequenceTime.ConvertTo(TargetLevelSequence->GetMovieScene()->GetTickResolution()).FloorToFrame();
				TrackRecorder->SetSectionStartTimecode(FApp::GetTimecode(), FirstFrameOfSequence);
			}
		}
	}

	for (TWeakObjectPtr<UActorComponent> RemovedComponent : NewComponentsRemoved)
	{
		if (Target.IsValid() && RemovedComponent.IsValid())
		{
			UE_LOGF(LogTakesCore, Log, "Detected removed component %ls on Actor %ls, stopping recording of component's properties now.", *RemovedComponent->GetReadableName(), *Target->GetName());
			// sequencer-todo: notify the spawn track that no more data is needed for this without actually removing the object from the template/cdo
		}
	}

	// Now that we've initialized any new components we can tick all of our recordings to get the last frame's data.
	for (UMovieSceneTrackRecorder* Recorder : TrackRecorders)
	{
		if (Recorder)
		{
			Recorder->RecordSample(CurrentSequenceTime);
		}
	}

	// Update our cached list
	CachedComponentList = CurrentComponentList;
}

void UTakeRecorderActorSource::StopRecording(ULevelSequence* InSequence)
{
	for (UMovieSceneTrackRecorder* TrackRecorder : TrackRecorders)
	{
		if (!TrackRecorder)
		{
			UE_LOGF(LogTakesCore, Error, "Stop Recording: Actor source %ls has an invalid recorder.", *Target.GetAssetName());
			continue;
		}
		TrackRecorder->StopRecording();
	}
	
	ActorSerializer.Close();
}

namespace UE::TakeRecorderActorSource::Private
{
TOptional<TRange<FFrameNumber>> GetFrameRange(UMovieScene* MovieScene, FMovieSceneBinding* Binding)
{
	TOptional<TRange<FFrameNumber>> FrameRange;
	check(Binding);

	// In case we need it later, get the earliest timecode source *before* we
	// add the take section, since its timecode source will be default
	// constructed as all zeros and might accidentally compare as earliest.
	const FMovieSceneTimecodeSource EarliestTimecodeSource = MovieScene->GetEarliestTimecodeSource();

	for (UMovieSceneTrack* Track : Binding->GetTracks())
	{
		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (Section->HasStartFrame() && Section->HasEndFrame())
			{
				if (!FrameRange.IsSet())
				{
					FrameRange = Section->GetRange();
				}
				else
				{
					FrameRange = TRange<FFrameNumber>::Hull(FrameRange.GetValue(), Section->GetRange());
				}
			}
		}
	}
	return FrameRange;
}

UMovieSceneTakeTrack* FindOrAddTakeTrack(UMovieScene* MovieScene, const FGuid& CachedObjectBindingGuid)
{
	UMovieSceneTakeTrack* TakeTrack = MovieScene->FindTrack<UMovieSceneTakeTrack>(CachedObjectBindingGuid);
	if (!TakeTrack)
	{
		TakeTrack = MovieScene->AddTrack<UMovieSceneTakeTrack>(CachedObjectBindingGuid);
	}
	return TakeTrack;
}

}
void UTakeRecorderActorSource::ProcessRecordedTimes(ULevelSequence* InSequence)
{
	UMovieScene* MovieScene = InSequence->GetMovieScene();

	FMovieSceneBinding* Binding = MovieScene->FindBinding(CachedObjectBindingGuid);
	if (!Binding)
	{
		return;
	}

	// In case we need it later, get the earliest timecode source *before* we
	// add the take section, since its timecode source will be default
	// constructed as all zeros and might accidentally compare as earliest.
	TOptional<TRange<FFrameNumber>> FrameRange =
		UE::TakeRecorderActorSource::Private::GetFrameRange(MovieScene, Binding);

	// Create a new take track or reuse the existing one based on binding.
	UMovieSceneTakeTrack* TakeTrack = UE::TakeRecorderActorSource::Private::FindOrAddTakeTrack(MovieScene, CachedObjectBindingGuid);

	// Add the recorded times to the take track.
	UE::TakesCore::ProcessRecordedTimes(InSequence, TakeTrack, FrameRange, UTakeRecorderSources::RecordedTimes);
}

UMovieSceneAnimationTrackRecorderEditorSettings* UTakeRecorderActorSource::GetBestTrackRecorderEditorSettings() const
{
	// This is called when the instance settings most likely don't exist, but we should still check them.
	// The animation track recorder editor settings are shared with normal actor (non-animation) usage for asset and track name.
	if (UMovieSceneAnimationTrackRecorderEditorSettings* InstanceSettings
		= Cast<UMovieSceneAnimationTrackRecorderSettings>(GetSettingsObjectForFactory(UMovieSceneAnimationTrackRecorderSettings::StaticClass())))
	{
		return InstanceSettings;
	}
	// Fallback to global.
	return GetMutableDefault<UMovieSceneAnimationTrackRecorderEditorSettings>();
}

void UTakeRecorderActorSource::DoCreatePendingTrackRecorders()
{
	for (const FCreateTrackArgs& PendingTrack : PendingTrackRecorders)
	{
		check(PendingTrack.TrackRecorder.IsValid());
		check(PendingTrack.ObjectToRecord.IsValid());
		check(PendingTrack.ObjectGuid.IsValid());
		
		PendingTrack.TrackRecorder->CreateTrack(this, PendingTrack.ObjectToRecord.Get(),
			TargetLevelSequence->GetMovieScene(), PendingTrack.FactorySettings.Get(), PendingTrack.ObjectGuid);
	}
	
	PendingTrackRecorders.Empty();
}

TArray<UTakeRecorderSource*> UTakeRecorderActorSource::PostRecording(ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled)
{
	FTakeRecorderParameters Parameters;
	Parameters.User = GetDefault<UTakeRecorderUserSettings>()->Settings;
	Parameters.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;

	FString TargetId;

	if (Target.IsValid())
	{
	#if WITH_EDITOR
		TargetId = Target.Get()->GetActorLabel();
	#else // WITH_EDITOR
		TargetId = Target.Get()->GetName();
	#endif // WITH_EDITOR
	}

	FScopedSlowTask SlowTask((float)TrackRecorders.Num() + 1.0f, FText::Format(LOCTEXT("ProcessingActor", "Generating MovieScene Data for Actor {0}"), !TargetId.IsEmpty() ? FText::FromString(TargetId) : FText::GetEmpty()));
	SlowTask.MakeDialog(false, bShowProgressDialog);

	// We need to do some post-processing tasks on the Track Recorders (such as animation motion source fixup) so we do this now before finalizing
	{
		SlowTask.EnterProgressFrame(0.1f, LOCTEXT("PostProcessingTrackRecorder", "Post Processing Track Recorders"));
		PostProcessTrackRecorders(InSequence);
	}

	// Finalize each Section Recorder and allow it to write data into the Level Sequence.
	int32 SectionRecorderIndex = 0;
	for (UMovieSceneTrackRecorder* SectionRecorder : TrackRecorders)
	{
		if (!SectionRecorder)
		{
			UE_LOGF(LogTakesCore, Error, "Post Recording: Actor source %ls has an invalid section recorder.", *Target.GetAssetName());
			continue;
		}
		
		// Increment before entering the progress frame so we get "1/1" instead of "0/1"
		SectionRecorderIndex++;

		// takerecorder-todo: Section Recorders should have display names, update this to use those.
		SlowTask.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("FinalizingTrackRecorder", "Finalizing Section Recorder {0}/{1}"), SectionRecorderIndex, TrackRecorders.Num()));
		if (bCancelled)
		{
			SectionRecorder->CancelTrack();
		}
		else
		{
			SectionRecorder->FinalizeTrack();
		}
	}

	if (!bCancelled)
	{
		if (Parameters.Project.bRecordTimecode)
		{
			ProcessRecordedTimes(InSequence);
		}

		if (Target.IsValid() && !ParentSource)
		{
			// Automatically add or update the camera cut track if there is a camera component
			AActor* TargetActor = Target.Get();

			if (TargetActor->GetComponentByClass(UCameraComponent::StaticClass()))
			{
				FGuid RecordedCameraGuid = GetRecordedActorGuid(Target.Get());
				FMovieSceneSequenceID RecordedCameraSequenceID = GetLevelSequenceID(Target.Get());
				TakesUtils::CreateCameraCutTrack(InRootSequence, RecordedCameraGuid, RecordedCameraSequenceID, InRootSequence->GetMovieScene()->GetPlaybackRange());
			}

		#if WITH_EDITOR
			// Swap our target actor to the Editor actor (in case the recording was added while in PIE)
			if (AActor* EditorActor = EditorUtilities::GetEditorWorldCounterpartActor(Target.Get()))
			{
				Target = EditorActor;
			}
		#endif // WITH_EDITOR
		}
	}

	// Force to authority role in case of capturing replicated actors
	if (CachedObjectTemplate.IsValid() && !CachedObjectTemplate->HasAuthority())
	{
		CachedObjectTemplate->SetRole(ROLE_Authority);
	}

	// No longer need to track the Object Template that was created inside the level sequence.
	if (!bTargetIsSequencerActor)
	{
		CachedObjectBindingGuid.Invalidate();
	}
	CachedObjectTemplate = nullptr;
	CachedComponentList.Empty();

	if (bIsTemporarySource)
	{
		// So this source can be cleaned up later.
		AddedActorSources.Add(this);
	}
	
	// We may have generated some temporary recording sources
	return AddedActorSources;
}

void UTakeRecorderActorSource::FinalizeRecording()
{
#if WITH_EDITOR

	// For Sequencer-managed actors the recording instance is destroyed when the recording
	// sequence closes. Re-point Target at the new live instance so this source remains
	// usable (e.g. for display in the Take Recorder panel or the next recording).
	// We defer the lookup by a configurable number of ticks so that the source sequence's
	// Sequencer has time to evaluate and respawn the actor before we search for it.
	if (!CachedActorLabel.IsEmpty() && bTargetIsSequencerActor)
	{
		// Number of ticks to wait before resolving the new Target. Increase if the source
		// sequence's Sequencer needs more evaluation cycles to respawn the actor.
		static const int32 RetargetTickDelay = 2;

		// Clear the original's Target so the per-tick recovery check fires on the original.
		// Without this, Target still points at the live recording-time actor and the check's
		// !Target.IsValid() condition is false, leaving the panel pointing at a soon-to-be-
		// destroyed actor rather than triggering a fresh search.
		if (UTakeRecorderActorSource* Original = WeakOriginalSource.Get())
		{
			Original->Target = nullptr;
		}

		TWeakObjectPtr<UTakeRecorderActorSource> WeakThis(this);
		TWeakObjectPtr<UTakeRecorderActorSource> WeakOriginal = WeakOriginalSource;
		FString LabelToFind = CachedActorLabel;
		FGuid GuidToFind = CachedObjectBindingGuid;
		int32 TicksRemaining = RetargetTickDelay;

		FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateLambda([WeakThis, WeakOriginal, LabelToFind, GuidToFind, TicksRemaining](float) mutable -> bool
			{
				if (--TicksRemaining >= 0)
				{
					return true; // Keep ticking until delay has elapsed
				}

				UTakeRecorderActorSource* This = WeakThis.Get();
				if (!This || !GEditor)
				{
					return false;
				}

				AActor* NewTarget = nullptr;

				// Primary: ISequencer binding cache - authoritative once the source sequence
				// has completed at least one evaluation cycle after the recording closed.
				UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
				for (ULevelSequence* OpenSequence : TObjectRange<ULevelSequence>())
				{
					if (!OpenSequence)
					{
						continue;
					}
					IAssetEditorInstance* EditorInstance = AssetEditorSubsystem->FindEditorForAsset(OpenSequence, false);
					if (ILevelSequenceEditorToolkit* LevelSequenceToolkit = static_cast<ILevelSequenceEditorToolkit*>(EditorInstance))
					{
						if (TSharedPtr<ISequencer> FoundSequencer = LevelSequenceToolkit->GetSequencer())
						{
							for (TWeakObjectPtr<> WeakObj : FoundSequencer->FindBoundObjects(GuidToFind, MovieSceneSequenceID::Root))
							{
								if (AActor* BoundActor = Cast<AActor>(WeakObj.Get()))
								{
									NewTarget = BoundActor;
									break;
								}
							}
						}
					}
					if (NewTarget)
					{
						break;
					}
				}

				// Fallback: world label search.
				if (!NewTarget)
				{
					if (UWorld* World = GEditor->GetEditorWorldContext().World())
					{
						for (TActorIterator<AActor> ActorIt(World); ActorIt; ++ActorIt)
						{
							if (ActorIt->GetActorLabel() == LabelToFind)
							{
								NewTarget = *ActorIt;
								break;
							}
						}
					}
				}

				if (NewTarget)
				{
					// Update the copy (this source) and also the original source that the Take
					// Recorder panel is displaying - recording runs on a DuplicateObject copy, so
					// only the original is visible to the user after recording completes.
					// Also cache the GUID on the original so TryRecoverTarget()'s GUID-based
					// path can use it when the per-tick recovery check fires.
					This->Target = NewTarget;
					if (UTakeRecorderActorSource* Original = WeakOriginal.Get())
					{
						Original->Target = NewTarget;
						Original->CachedObjectBindingGuid = GuidToFind;
					}
				}

				return false; // Done - remove this ticker
			})
		);
		WeakOriginalSource = nullptr;
	}

	// Re-add the spawn track that was removed in PreRecording to prevent a duplicate actor
	// during recording. Without it the take would have a spawnable with no spawn track and
	// would never spawn during playback.
	if (bRemovedSpawnTrack && CachedObjectBindingGuid.IsValid() && TargetLevelSequence)
	{
		if (UMovieScene* RecordingMovieScene = TargetLevelSequence->GetMovieScene())
		{
			UMovieSceneSpawnTrack* SpawnTrack = Cast<UMovieSceneSpawnTrack>(
				RecordingMovieScene->FindTrack(UMovieSceneSpawnTrack::StaticClass(), CachedObjectBindingGuid));
			if (!SpawnTrack)
			{
				SpawnTrack = Cast<UMovieSceneSpawnTrack>(
					RecordingMovieScene->AddTrack(UMovieSceneSpawnTrack::StaticClass(), CachedObjectBindingGuid));
			}
			if (SpawnTrack && SpawnTrack->GetAllSections().IsEmpty())
			{
				SpawnTrack->Modify();
				if (UMovieSceneSpawnSection* SpawnSection = Cast<UMovieSceneSpawnSection>(SpawnTrack->CreateNewSection()))
				{
					SpawnSection->GetChannel().SetDefault(true);
					SpawnSection->SetRange(TRange<FFrameNumber>::All());
					SpawnTrack->AddSection(*SpawnSection);
					SpawnTrack->SetObjectId(CachedObjectBindingGuid);
				}
			}
		}
		bRemovedSpawnTrack = false;
	}

	CachedObjectBindingGuid.Invalidate();

#endif
	
	// Null these out there and NOT in PostRecording because they are used for cross sequence object binding via GetLevelSequenceID in PostRecording
	TargetLevelSequence = nullptr;
	RootLevelSequence = nullptr;

	ParentSource = nullptr;
}

namespace UE::TakeRecorderActorSource::Private
{

FProcessRecordedTimeParams GetTimecodeRecordingParameters()
{
	FString HoursName = GetDefault<UMovieSceneTakeSettings>()->HoursName;
	FString MinutesName = GetDefault<UMovieSceneTakeSettings>()->MinutesName;
	FString SecondsName = GetDefault<UMovieSceneTakeSettings>()->SecondsName;
	FString FramesName = GetDefault<UMovieSceneTakeSettings>()->FramesName;
	FString SubFramesName = GetDefault<UMovieSceneTakeSettings>()->SubFramesName;
	FString RateName = GetDefault<UMovieSceneTakeSettings>()->RateName;
	FString SlateName = GetDefault<UMovieSceneTakeSettings>()->SlateName;

	return FProcessRecordedTimeParams {
		.HoursName = HoursName,
		.MinutesName = MinutesName,
		.SecondsName = SecondsName,
		.FramesName = FramesName,
		.SubFramesName = SubFramesName,
		.RateName = RateName,
		.SlateName = SlateName
	};
}

}
void UTakeRecorderActorSource::PostProcessTrackRecorders(ULevelSequence* InSequence)
{
	FTakeRecorderParameters Parameters;
	Parameters.User = GetDefault<UTakeRecorderUserSettings>()->Settings;
	Parameters.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;

	FProcessRecordedTimeParams RecordedTimeParams = UE::TakeRecorderActorSource::Private::GetTimecodeRecordingParameters();
#if WITH_EDITOR
	if (UTakeMetaData* TakeMetaData = InSequence->FindMetaData<UTakeMetaData>())
	{
		RecordedTimeParams.Slate = FString::Printf(TEXT("%s_%d"), *TakeMetaData->GetSlate(), TakeMetaData->GetTakeNumber());
	}
#endif // WITH_EDITOR

	// We want to look at all Animation Track recorders and remove root motion if the transform
	// for that component is being recorded. We copy the animation out of the Animation Track
	// so that we accurately capture the original motion.
	UMovieScene3DTransformTrackRecorder* RootTransformRecorder = nullptr;
	UMovieSceneAnimationTrackRecorder* FirstAnimationRecorder = nullptr;

	for (UMovieSceneTrackRecorder* TrackRecorder : TrackRecorders)
	{
		if (!TrackRecorder)
		{
			UE_LOGF(LogTakesCore, Error, "Post Process Track Recorders: Actor source %ls has an invalid recorder.", *Target.GetAssetName());
			continue;
		}
		AActor* SourceActor = Cast<AActor>(TrackRecorder->GetSourceObject());
		AActor* SourceEditorActor = 
		#if WITH_EDITOR
			SourceActor ? EditorUtilities::GetEditorWorldCounterpartActor(SourceActor) : 
		#endif // WITH_EDITOR
			nullptr;
		AActor* TargetActor = Target.Get();

		if (!RootTransformRecorder && TrackRecorder->IsA<UMovieScene3DTransformTrackRecorder>() && (TargetActor == SourceActor || TargetActor == SourceEditorActor))
		{
			RootTransformRecorder = Cast<UMovieScene3DTransformTrackRecorder>(TrackRecorder);
		}
		if (!FirstAnimationRecorder && TrackRecorder->IsA<UMovieSceneAnimationTrackRecorder>())
		{
			FirstAnimationRecorder = Cast<UMovieSceneAnimationTrackRecorder>(TrackRecorder);
		}

		// Early out once we have both of them.
		if (RootTransformRecorder && FirstAnimationRecorder)
		{
			break;
		}
	}
	// We need to take the root motion data from the animation and override the data the Transform Track had originally captured if we are removing root
	if (RootTransformRecorder && FirstAnimationRecorder && FirstAnimationRecorder->RootWasRemoved())
	{
		RootTransformRecorder->PostProcessAnimationData(FirstAnimationRecorder);
		FirstAnimationRecorder->RemoveRootMotion();
	}

	// Remove root motion on all other animation track recorders
	for (UMovieSceneTrackRecorder* TrackRecorder : TrackRecorders)
	{
		if (!TrackRecorder)
		{
			UE_LOGF(LogTakesCore, Error, "Post Process Track Recorders: Actor source %ls has an invalid recorder.", *Target.GetAssetName());
			continue;
		}
		
		if (TrackRecorder->IsA<UMovieSceneAnimationTrackRecorder>())
		{
			UMovieSceneAnimationTrackRecorder* AnimationTrackRecorder = Cast<UMovieSceneAnimationTrackRecorder>(TrackRecorder);
			
			if (TrackRecorder != FirstAnimationRecorder && AnimationTrackRecorder->RootWasRemoved())
			{
				AnimationTrackRecorder->RemoveRootMotion();
			}
			
			if (Parameters.Project.bRecordTimecode)
			{
				AnimationTrackRecorder->ProcessRecordedTimes(RecordedTimeParams);
			}
		}
	}

	// Reset transform for recorded spawnable actors when their skeletal animation is recorded in world space 
	// but a transform track is not being recorded.
	// 
	// This is a very specific case with the following parameters:
	// 1. Skeletal mesh actor is recorded as a spawnable.
	// 2. Skeletal mesh actor is a child of another actor which is not being recorded (this results in the bones being recorded in world space)
	// 3. Transform track is set to not record.
	//
	// When this occurs, the transform of the spawnable is doubled up because the root bone is in world space
	// and the spawnable template is also in world space.The fix here is to reset the spawnable template's 
	// transform to identity.	
	if (FirstAnimationRecorder && !RootTransformRecorder)
	{
		if (CachedObjectTemplate.IsValid())
		{
			AActor* ActorToRecord = Target.Get();
			if (ActorToRecord)
			{
				USceneComponent* RootComponent = ActorToRecord->GetRootComponent();
				USceneComponent* AttachParent = RootComponent ? RootComponent->GetAttachParent() : nullptr;
				if (AttachParent && !IsOtherActorBeingRecorded(AttachParent->GetOwner()))
				{
					// The object template is marked as bComponentToWorldUpdated=true while the ComponentToWorld doesn't 
					// match the relative location and rotation. So, calling SetRelativeTransform() doesn't work. 
					// Set it directly here.
					CachedObjectTemplate->GetRootComponent()->SetRelativeTransform_Direct(FTransform::Identity);
				}
			}
		}
	}
}

bool UTakeRecorderActorSource::EnsureObjectTemplateHasComponent(UActorComponent* InComponent, UActorComponent*& OutComponent)
{
	check(InComponent);

	//If it's coming from a component that is created from a template defined in the Components section of the Blueprint
	//it will NOT be found as a Component in AllChildren below but it will exist when created so we also exit here.
	//So we only do the check if UserConstructionScript or Instance, with the latter may not be needed.. not sure
	if (InComponent->CreationMethod == EComponentCreationMethod::SimpleConstructionScript)
	{
		return false;
	}
	// Attempt to find the component in our Object Template by comparing relative paths. This might fail in complex dynamic hierarchies if a component
	// is added and removed multiple times at runtime if they don't all end up with unique names but is pretty straightforward logic for now.
	FString NewComponentRelativePath = InComponent->GetFullName(InComponent->GetTypedOuter<AActor>());

	// Sequencer actors are recorded via the possessable path (bTargetIsSequencerActor), so treat them
	// the same as true possessables: compare against the CDO rather than the spawnable template.
	// The spawnable template lives in the source sequence and must not be modified during recording.
	AActor* DestinationActor = (GetRecordToPossessable()) ? Target->GetClass()->GetDefaultObject<AActor>() : CachedObjectTemplate.Get();
	check(DestinationActor);

	TInlineComponentArray<UActorComponent*> AllChildren;
	DestinationActor->GetComponents(AllChildren);

	bool bFoundComponent = false;
	for (UActorComponent* Child : AllChildren)
	{
		if (Child == nullptr)
		{
			continue;
		}
			
		FString ChildRelativePath = Child->GetFullName(DestinationActor);

		if (NewComponentRelativePath == ChildRelativePath)
		{
			bFoundComponent = true;
			break;
		}
	}

	// If we found the component with the same relative path on either the CDO (for Possessables) or the Object Template (for Spawnables) then
	// there's nothing we need to do.
	if (bFoundComponent)
	{
		return false;
	}

	// Possessables (and sequencer actors on the possessable path) can't have objects dynamically
	// added so if this is a new object and they don't have them, warn the user.
	if (GetRecordToPossessable())
	{
		UE_LOGF(LogTakesCore, Warning, "Actor %ls had dynamically added component at runtime (%ls) but this cannot be saved because we are recording to a possessable, component binding will be broken!",
			*Target->GetName(), *InComponent->GetName());
		return false;
	}

	// Now we'll go through the process of duplicating the new component and updating our Object Template with it so that the bindings work after the fact.
	USceneComponent* TemplateRoot = CachedObjectTemplate->GetRootComponent();
	USceneComponent* AttachToComponent = nullptr;

	// If the new component is a Scene Component then we'll attach it to the correct parent.
	USceneComponent* SceneComponent = Cast<USceneComponent>(InComponent);
	if (SceneComponent)
	{
		USceneComponent* AttachParent = SceneComponent->GetAttachParent();
		if (AttachParent != nullptr)
		{
			// Get the path to the parent component so we can find the matching path in the template.
			FString ParentRelativePath = AttachParent->GetFullName(Target.Get());

			TInlineComponentArray<USceneComponent*> AllTemplateChildren;
			CachedObjectTemplate->GetComponents(AllTemplateChildren);

			for (USceneComponent* Child : AllTemplateChildren)
			{
				if (Child == nullptr)
				{
					continue;
				}

				FString ChildRelativePath = Child->GetFullName(CachedObjectTemplate.Get());

				if (ParentRelativePath == ChildRelativePath)
				{
					AttachToComponent = Child;
					break;
				}
			}

			if (!AttachToComponent)
			{
				UE_LOGF(LogTakesCore, Warning, "Dynamically added component %ls failed to find attach parent %ls in Object Template, attaching to root as fallback!",
					*InComponent->GetName(), *AttachParent->GetName());
				
				AttachToComponent = CachedObjectTemplate->GetRootComponent();
			}
		}
	}

	// Ensure the component name is unique within the Object Template. If there's complex spawn/destroy patterns that don't always use unique names this can
	// cause UniqueComponentName to become a different name than the object it's being copied from which will cause anything attached to this to fail attachment.
	// Note, we use NAME_None as the base name as opposed to anything the actual component's name because it could conflict with subsequence spawned components.
	FName UniqueComponentName = MakeUniqueObjectName(CachedObjectTemplate.Get(), InComponent->GetClass(), NAME_None);
	OutComponent = Cast<UActorComponent>(StaticDuplicateObject(InComponent, CachedObjectTemplate.Get(), UniqueComponentName, RF_AllFlags & ~RF_Transient));

	// Restore attachment
	if (SceneComponent && AttachToComponent && OutComponent->IsA<USceneComponent>())
	{
		USceneComponent* NewSceneComponent = Cast<USceneComponent>(OutComponent);
		NewSceneComponent->AttachToComponent(AttachToComponent, FAttachmentTransformRules::KeepRelativeTransform, SceneComponent->GetAttachSocketName());
	}

	// Update our Object Template with a reference to our component
	UE_LOGF(LogTakesCore, Log, "Duplicating Component '%ls' to '%ls' and adding to Spawnable Object Template.", *InComponent->GetPathName(), *OutComponent->GetPathName());
	CachedObjectTemplate->AddInstanceComponent(OutComponent);

	return true;
}

#if WITH_EDITOR
void UTakeRecorderActorSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTakeRecorderActorSource, Target))
	{
		SetSourceActor(Target);
	}
}

void UTakeRecorderActorSource::PostEditUndo()
{
	Super::PostEditUndo();
	UpdateCachedNumberOfRecordedProperties();
}
#endif // WITH_EDITOR

void UTakeRecorderActorSource::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bRecordParentHierarchy)
	{
		SetAttachRecordBehaviour(ETakeRecorderAttachRecordBehaviour::Hierarchy);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
}

void UTakeRecorderActorSource::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	// When we get deserialized from being duplicated we need to update our numbers.
	// This has to be done after the constructor as the Property Map hasn't been deserialized
	// by that point.
	UpdateCachedNumberOfRecordedProperties();

	// NOTE: No ticker registration here. DuplicateObject calls PostInitProperties before
	// PostDuplicate, so the recovery ticker is already registered by the time we reach this
	// function. Registering again would leak the first handle and create a double-tick.
}

void UTakeRecorderActorSource::PostInitProperties()
{
	Super::PostInitProperties();
#if WITH_EDITOR
	if (!IsTemplate())
	{
		// Start a per-tick recovery check as a robust way to handle actor source recovery.
		RecoveryTickerHandle = FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateUObject(this, &UTakeRecorderActorSource::TickRecovery)
		);
	}
#endif // WITH_EDITOR
}

void UTakeRecorderActorSource::BeginDestroy()
{
#if WITH_EDITOR
	FTSTicker::GetCoreTicker().RemoveTicker(RecoveryTickerHandle);

	CleanupAttachedDelegates();

#endif
	Super::BeginDestroy();
}

void UTakeRecorderActorSource::RebuildRecordedPropertyMap()
{
	// Reset our property map before checking the current actor, this allows null actors to 
	// empty out the property map in the UI.
	FName RootName = Target.IsValid() ? Target->GetFName() : NAME_None;
	RecordedProperties = NewObject<UActorRecorderPropertyMap>(this, MakeUniqueObjectName(GetTransientPackage(), UActorRecorderPropertyMap::StaticClass(), RootName), RF_Transactional);

	//@matth this was making us not be able to record, everything was empty
	//RecordedProperties = NewObject<UActorRecorderPropertyMap>(this, MakeUniqueObjectName(this, UActorRecorderPropertyMap::StaticClass(), RootName), RF_Transactional);
	TrackRecorders.Empty();

	// No target actor means no properties will get recorded
	using namespace UE::TakeRecorderSources;
	if (!Target.IsValid() || !FTakeRecorderSourcesModule::Get().CanRecord(FCanRecordArgs(Target.Get())))
	{
		return;
	}

	RecordedProperties->RecordedObject = Target.Get();
	RebuildRecordedPropertyMapRecursive(Target.Get(), RecordedProperties);

	UpdateCachedNumberOfRecordedProperties();
}

void UTakeRecorderActorSource::RebuildRecordedPropertyMapRecursive(const FFieldVariant& InObject, UActorRecorderPropertyMap* PropertyMap, const FString& OuterStructPath)
{
	ensure(InObject);
	ensure(PropertyMap);

	// // Iterate through our recorders and find any that can record this object that aren't tied to a specific property. Some things
	// we wish to record (such as Transforms) don't have a specific FProperty or UActorComponent associated with them.
	TArray<IMovieSceneTrackRecorderFactory*> ModularFactories = IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneTrackRecorderFactory>(MovieSceneSectionRecorderFactoryName);
	for (IMovieSceneTrackRecorderFactory* Factory : ModularFactories)
	{
	 	if (InObject.IsUObject() && Factory->CanRecordObject(InObject.ToUObject()))
	 	{
	 		// @sequencer-todo: Instead of defaulting to true this should copy from the global settings
	 		FName PropertyName = FName(*Factory->GetDisplayName().ToString());
	 		FActorRecordedProperty RecordedProperty(PropertyName, true, Factory->GetDisplayName());
	 		PropertyMap->Properties.Add(RecordedProperty);

			// Initialize an instance of this factory's settings object if we haven't already.
			TSubclassOf<UMovieSceneTrackRecorderSettings> FactorySettingsClass = Factory->GetSettingsClass();
			if (FactorySettingsClass)
			{
				InitializeFactorySettingsObject(FactorySettingsClass);
			}
	 	}
	}

	// Iterate through the properties on this object and look for ones marked with CPF_Interp ("Expose for Cinematics") or that have metadata
	// that explicitly specifies a sequence track metadata.

	UStruct* ObjectClass = InObject.IsUObject() ? InObject.ToUObject()->GetClass() : nullptr;
	if (FStructProperty* AsStructProperty = InObject.Get<FStructProperty>())
	{
		ObjectClass = AsStructProperty->Struct;
	}
	check(ObjectClass); // With FProperties ObjectClass can only be obtained from AsStructProperty. If we hit this we need to change this check to an 'if (ObjectClass)'
	for (TFieldIterator<FProperty> It(ObjectClass); It; ++It)
	{
		const bool bIsInterpField = It->HasAllPropertyFlags(CPF_Interp);

	#if WITH_METADATA
		const bool bHasTrackMetadata = It->HasMetaData(SequencerTrackClassMetadataName);
	#else // WITH_METADATA
		const bool bHasTrackMetadata = false;
	#endif // WITH_METADATA

		FString PropertyName = It->GetFName().ToString();
		FString PropertyPath = OuterStructPath + FString(*PropertyName);

		if (bIsInterpField || bHasTrackMetadata)
		{
			bool bFoundRecorder = false;
			FText DebugDisplayName;

			// For each property we look to see if there is a specific recorder that can handle this property. This is the case for
			// properties such as "bVisible" which needs the specific Visibility Recorder (instead of a generic bool property recorder).
			// We fall back to the generic property recorder if we can't find a specific recorder, and if any recorder is found then we
			// create an instance to show up in the UI so the user can still choose to toggle on/off properties (and know that the properties
			// shown there do actually have something trying to record them).
			for (IMovieSceneTrackRecorderFactory* Factory : ModularFactories)
			{
				if ((InObject.IsUObject() && Factory->CanRecordProperty(InObject.ToUObject(), *It))
					||
					// A factory may have special handling for any property of this type.
					(Factory->CanHandlePropertyInstance() && Factory->CanRecordProperty(Target.Get(), *It)))
				{
			 		DebugDisplayName = Factory->GetDisplayName();

					// Initialize an instance of this factory's settings object if we haven't already.
					TSubclassOf<UMovieSceneTrackRecorderSettings> FactorySettingsClass = Factory->GetSettingsClass();
					if (FactorySettingsClass)
					{
						InitializeFactorySettingsObject(FactorySettingsClass);
					}

			 		// Only one recorder gets a chance to record it
			 		bFoundRecorder = true;
			 		break;
				}
			}
			 
			if (!bFoundRecorder)
			{
				// If we didn't find an explicit recorder for the property, we'll fall back to a generic property recorder which simply stores their state changes in a track.
				FMovieScenePropertyTrackRecorderFactory TrackRecorderFactory;
				if (TrackRecorderFactory.CanRecordProperty(Target.Get(), *It))
				{
			 		DebugDisplayName = TrackRecorderFactory.GetDisplayName();
			 		bFoundRecorder = true;

					// Initialize an instance of this factory's settings object if we haven't already.
					TSubclassOf<UMovieSceneTrackRecorderSettings> FactorySettingsClass = TrackRecorderFactory.GetSettingsClass();
					if (FactorySettingsClass)
					{
						InitializeFactorySettingsObject(FactorySettingsClass);
					}
				}
			}

			if (!bFoundRecorder)
			{
				if (FStructProperty* StructProperty = CastField<FStructProperty>(*It)) 
				{
					FString NewOuterStructPath = OuterStructPath + PropertyName + TEXT(".");
					RebuildRecordedPropertyMapRecursive(StructProperty, PropertyMap, NewOuterStructPath);
				}
			}

			if (bFoundRecorder)
			{
				// @sequencer-todo: Instead of defaulting to true this should copy from the global settings
				FActorRecordedProperty RecordedProperty(*PropertyPath, true, DebugDisplayName);
				PropertyMap->Properties.Add(RecordedProperty);
			}
		}

		else if (FStructProperty* StructProperty = CastField<FStructProperty>(*It))
		{
			FString NewOuterStructPath = OuterStructPath + PropertyName + TEXT(".");
			RebuildRecordedPropertyMapRecursive(StructProperty, PropertyMap, NewOuterStructPath);
		}

	}

	// Now try to iterate through any children on this object and continue this process recursively.
	TSet<TWeakObjectPtr<UActorComponent>> PossibleComponents;
	TSet<AActor*> ExternalActorsReferenced;

	if (InObject.IsA<AActor>())
	{
		AActor* Actor = InObject.Get<AActor>();
		
		// Actors only have their Root Component plus any Actor Components (which have no hierarchy)
		// After that the structure is recursive down from the Root Component.
		if (Actor->GetRootComponent())
		{
			PossibleComponents.Add(Actor->GetRootComponent());
		}
		GetActorComponents(Actor, PossibleComponents);
	}
	else if (InObject.IsA<USceneComponent>())
	{
		USceneComponent* SceneComponent = InObject.Get<USceneComponent>();
		GetChildSceneComponents(SceneComponent, PossibleComponents, true);
	}

	NewReferencedActors.Append(ExternalActorsReferenced);

	// Now iterate through our children and build the property map recursively.
	for (TWeakObjectPtr<UActorComponent> Component : PossibleComponents)
	{
		if (!Component.IsValid())
		{
			continue;
		}

		UE_LOGF(LogTakesCore, Log, "Component: %ls EditorOnly: %d Transient: %d", *Component->GetFName().ToString(), Component->IsEditorOnly(), Component->HasAnyFlags(RF_Transient));
		// takerecorder-todo: When merged with Dev Framework, CL 4279185, switch this to checking against
		// IsVisualizationComponent() so that we can exclude things like default component billboards.
		// We also need a deny list of classes, such as those that derive from the input framework that are
		// added at runtime, we don't want to record those.
		if (Component->IsEditorOnly())
		{
			continue;
		}

		UActorRecorderPropertyMap* ComponentPropertyMap = NewObject<UActorRecorderPropertyMap>(this, MakeUniqueObjectName(this, UActorRecorderPropertyMap::StaticClass(), Component->GetFName()), RF_Transactional);
		ComponentPropertyMap->RecordedObject = Component.Get();
		PropertyMap->Children.Add(ComponentPropertyMap);

		RebuildRecordedPropertyMapRecursive(Component.Get(), ComponentPropertyMap);
	}
}

void UTakeRecorderActorSource::UpdateCachedNumberOfRecordedProperties()
{
	if (RecordedProperties)
	{
		RecordedProperties->UpdateCachedValues();
	}
}

void UTakeRecorderActorSource::CleanupAttachedDelegates()
{
#if WITH_EDITOR
	if (!IsEngineExitRequested() && GEngine)
	{
		GEngine->OnLevelActorDetached().RemoveAll(this);
		GEngine->OnLevelActorAttached().RemoveAll(this);
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UTakeRecorderActorSource::HandleActorAttached(AActor* InChild, const AActor* InParent)
{
	if (InChild == nullptr || InParent == nullptr)
	{
		return;
	}

	if (const UTakeRecorderSubsystem* TRSubsystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>();
		TRSubsystem && TRSubsystem->IsRecording())
	{
		return;
	}

	if (InChild != Target.Get())
	{
		return;
	}

	// In the case of an attach track, this callback can fire a lot. Therefore, check to see if the attachment has actually changed to
	// a new parent, otherwise return early.
	if (InParent == WeakAttachParent)
	{
		return;
	}

	AActor* NonConstParent = const_cast<AActor*>(InParent);
	if (UE::TakeRecorderActorSource::Private::IsActorSpawnedInActiveSequence(NonConstParent))
	{
		// Defer out of the AttachToComponent broadcast scope: AddAttachTrackInActiveSequence triggers
		// a synchronous sequence re-evaluation whose attach system would re-enter AttachToComponent
		// from inside the outer call's broadcast (not re-entrant). Running next tick lets the outer
		// call unwind first.
		TWeakObjectPtr<AActor> WeakChild(InChild);
		TWeakObjectPtr<AActor> WeakParent(NonConstParent);
		TWeakObjectPtr<UTakeRecorderActorSource> WeakSource(this);
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
			[WeakSource, WeakChild, WeakParent](float) -> bool
			{
				TStrongObjectPtr<UTakeRecorderActorSource> StrongSource = WeakSource.Pin();
				TStrongObjectPtr<AActor> StrongChild = WeakChild.Pin();
				TStrongObjectPtr<AActor> StrongParent = WeakParent.Pin();
				if (StrongSource.IsValid() && StrongChild.IsValid() && StrongParent.IsValid())
				{
					if (UE::TakeRecorderActorSource::Private::AddAttachTrackInActiveSequence(
						StrongChild.Get(), StrongParent.Get(), StrongSource.Get()))
					{
						StrongSource->WeakAttachParent = StrongParent.Get();
					}
				}
				return false; // one-shot
			}));
	}
}

void UTakeRecorderActorSource::HandleActorDetached(AActor* InChild, const AActor* InParent)
{
	if (InChild == nullptr || InParent == nullptr)
	{
		return;
	}

	if (InChild != Target.Get())
	{
		return;
	}

	// if this was our parent, clear out weak pointer
	// We will pick this up again the next time the attachment is made.
	if (InParent == WeakAttachParent)
	{
		WeakAttachParent.Reset();
	}
}

#endif // WITH_EDITOR

ETakeRecorderAttachRecordBehaviour UTakeRecorderActorSource::ResolveAttachRecordBehaviour() const
{
	return AttachRecordBehaviour == ETakeRecorderAttachRecordBehaviour::ProjectDefault
		? GetDefault<UTakeRecorderProjectSettings>()->Settings.AttachRecordBehaviour
		: AttachRecordBehaviour;
}

ETakeRecorderSpawnableOverwriteBehavior UTakeRecorderActorSource::ResolveSpawnableOverwriteBehaviour() const
{
	return SpawnableOverwriteBehavior == ETakeRecorderSpawnableOverwriteBehavior::ProjectDefault
		? GetDefault<UTakeRecorderProjectSettings>()->Settings.SpawnableOverwriteBehavior
		: SpawnableOverwriteBehavior;
}

ETakeRecorderActorRecordType UTakeRecorderActorSource::ResolveRecordType() const
{
	if (RecordType == ETakeRecorderActorRecordType::ProjectDefault)
	{
		if (UTakeRecorderSources* Sources = GetTypedOuter<UTakeRecorderSources>())
		{
			return Sources->GetSettings().bRecordToPossessable
				? ETakeRecorderActorRecordType::Possessable
				: ETakeRecorderActorRecordType::Spawnable;
		}
	}

	return RecordType;
}

const FSlateBrush* UTakeRecorderActorSource::GetDisplayIconImpl() const
{
	AActor* TargetActor = Target.Get();
	if (TargetActor)
	{
		return FSlateIconFinder::FindCustomIconBrushForClass(TargetActor->GetClass(), TEXT("ClassThumbnail"));
	}

	return FSlateIconFinder::FindIcon("ClassIcon.Deleted").GetIcon();
}

FText UTakeRecorderActorSource::GetDisplayTextImpl() const
{
	AActor* TargetActor = Target.Get();
	if (TargetActor)
	{
	#if WITH_EDITOR
		return FText::FromString(TargetActor->GetActorLabel());
	#else // WITH_EDITOR
		return FText::FromString(TargetActor->GetName());
	#endif // WITH_EDITOR
	}

	return LOCTEXT("ActorLabel", "Actor (None)");
}

FText UTakeRecorderActorSource::GetAddSourceDisplayTextImpl() const
{
    return LOCTEXT("TakeRecorderDisplayName", "Actor Any");
}

FText UTakeRecorderActorSource::GetCategoryTextImpl() const
{
	AActor* TargetActor = Target.Get();
	if (TargetActor && TargetActor->GetComponentByClass(UCameraComponent::StaticClass()))
	{
		return LOCTEXT("CamerasCategoryLabel", "Cameras");
	}

	return FText();
}


FText UTakeRecorderActorSource::GetDescriptionTextImpl() const
{
	if (Target.IsValid())
	{
		UActorRecorderPropertyMap::Cache CachedValues;
		if (RecordedProperties)
		{
			CachedValues = RecordedProperties->CachedPropertyComponentCount();
		}
		return FText::Format(LOCTEXT("ActorDescriptionFormat", "{0} Properties {1} Components"), CachedValues.Properties, CachedValues.Components);
	}
	else
	{
		return LOCTEXT("InvalidActorDescription", "No Target Specified");
	}
}

FGuid UTakeRecorderActorSource::ResolveActorFromSequence(AActor* InActor, ULevelSequence* CurrentSequence) const
{
	return TakesUtils::ResolveActorFromSequence(InActor, CurrentSequence);
}

void UTakeRecorderActorSource::PostProcessCreatedObjectTemplateImpl(AActor* ObjectTemplate)
{
	 // Override the Skeletal Mesh components animation modes so that they can play back the recorded
	 // animation asset instead of their original animation source (such as Animation Blueprint)
	 TInlineComponentArray<USkeletalMeshComponent*> SkeletalMeshComponents;
	 ObjectTemplate->GetComponents(SkeletalMeshComponents);
	 for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
	 {
	 	SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
	 	SkeletalMeshComponent->bEnableUpdateRateOptimizations = false;
	 	SkeletalMeshComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	 	SkeletalMeshComponent->SetForcedLOD(1);
	 }

	// Disable auto-possession on recorded Pawns so that when the Spawnable is spawned it doesn't auto-possess the player
	// and override their current live player pawn.
	if (ObjectTemplate->IsA(APawn::StaticClass()))
	{
		APawn* Pawn = CastChecked<APawn>(ObjectTemplate);
		Pawn->AutoPossessPlayer = EAutoReceiveInput::Disabled;
	}

	// Disable any Movement Components so that things such as RotatingMovementComponent or ProjectileMovementComponent don't suddenly
	// start moving and overriding our position at runtime.
	// takerecorder-todo: This should ideally check to see if you recorded the transform of the root object or not before assuming you
	// don't want its movement?
	TInlineComponentArray<UMovementComponent*> MovementComponents;
	ObjectTemplate->GetComponents(MovementComponents);
	for (UMovementComponent* MovementComponent : MovementComponents)
	{
		MovementComponent->bAutoActivate = false;
	}
}

void GetChildBindings(UMovieScene* InMovieScene, const FGuid& InGuid, TArray<FGuid>& OutChildGuids)
{
	for (int32 PossessableIndex = 0; PossessableIndex < InMovieScene->GetPossessableCount(); ++PossessableIndex)
	{
		FMovieScenePossessable& Child = InMovieScene->GetPossessable(PossessableIndex);
		if (Child.GetParent() == InGuid)
		{
			OutChildGuids.Add(Child.GetGuid());

			GetChildBindings(InMovieScene, Child.GetGuid(), OutChildGuids);
		}
	}
}

void UTakeRecorderActorSource::CleanExistingDataFromSequence(const FGuid& ForGuid, ULevelSequence& InSequence)
{
	if (ForGuid.IsValid())
	{
		// Check to see if there is a Possessable in this sequence with the specified Guid and remove their old data if needed.
		// Removing the Possessable will remove their bindings which will remove the associated tracks and their data as well.
		UMovieScene* MovieScene = InSequence.GetMovieScene();

	#if WITH_EDITOR
		// Remove the binding for any folder if found to prevent invalid bindings left in folders
		for (UMovieSceneFolder* RootFolder : MovieScene->GetRootFolders())
		{
			if (UMovieSceneFolder* ContainingFolder = RootFolder->FindFolderContaining(ForGuid))
			{
				ContainingFolder->RemoveChildObjectBinding(ForGuid);
				break;
			}
		}
	#endif // WITH_EDITOR

		TArray<FGuid> OutChildGuids;
		GetChildBindings(MovieScene, ForGuid, OutChildGuids);

		MovieScene->RemoveSpawnable(ForGuid);
		MovieScene->RemovePossessable(ForGuid);

		for (auto ChildGuid : OutChildGuids)
		{
			MovieScene->RemovePossessable(ChildGuid);
		}
	}

	// Call any derived class implementation
	CleanExistingDataFromSequenceImpl(ForGuid, InSequence);
}

void UTakeRecorderActorSource::GetAllComponents(TSet<TWeakObjectPtr<UActorComponent>>& OutArray, bool bUpdateReferencedActorList)
{
	if (Target.IsValid())
	{
		GetActorComponents(Target.Get(), OutArray);
		GetSceneComponents(Target->GetRootComponent(), OutArray, bUpdateReferencedActorList);
	}
}

void UTakeRecorderActorSource::GetSceneComponents(USceneComponent* OnSceneComponent, TSet<TWeakObjectPtr<UActorComponent>>& OutArray, bool bUpdateReferencedActorList)
{
	if (!OnSceneComponent)
	{
		return;
	}

	// Add the passed in component to the out array and then we'll recursively call GetSceneComponents on each child
	// so that each child gets added to the out array and their children recursively.
	if (OnSceneComponent->ComponentHasTag(DoNotRecordTag))
	{
		UE_LOGF(LogTakesCore, Verbose, "Skipping record component: %ls with do not record tag", *OnSceneComponent->GetName());
		return;
	}

	OutArray.Add(OnSceneComponent);

	TSet<TWeakObjectPtr<UActorComponent>> ChildComponents;
	GetChildSceneComponents(OnSceneComponent, ChildComponents, bUpdateReferencedActorList);

	for (TWeakObjectPtr<UActorComponent> Component : ChildComponents)
	{
		GetSceneComponents(Cast<USceneComponent>(Component), OutArray, bUpdateReferencedActorList);
	}
}

void UTakeRecorderActorSource::GetChildSceneComponents(USceneComponent* OnSceneComponent, TSet<TWeakObjectPtr<UActorComponent>>& OutArray, bool bUpdateReferencedActorList)
{
	if (OnSceneComponent)
	{
		const bool bIncludeAllDescendants = false;
		TArray<USceneComponent*> OutDirectChildren;
		OnSceneComponent->GetChildrenComponents(bIncludeAllDescendants, OutDirectChildren);
		
		using namespace UE::TakeRecorderSources;
		FTakeRecorderSourcesModule& Module = FTakeRecorderSourcesModule::Get();
		
		// Add Scene Components to the OutArray
		for (USceneComponent* SceneComponent : OutDirectChildren)
		{
			if (!SceneComponent)
			{
				continue;
			}

			// If this scene component is owned by another Actor we have to make a complicated decision. In general, we don't want to record
			// components that are owned by another Actor because if that Actor is also being recorded we end up with duplicate bindings in the resulting
			// sequence. To solve this one, we want to create a recording for the Actor that owns that component (if it's not already being recorded) and
			// re-create the effect using Attach tracks in Sequencer.
			// Unfortunately, this leads to its own set of problems. In a situation where a complex hierarchy has been created via the World Outliner and you
			// are recording the root object, it will create bindings for all children as they will all show up as belonging to different actors and we'll create
			// a recording for each one. This isn't desirable either so we could just not record components that belong to other actors unless you specifically add
			// them. In the usual twist of fate this isn't desirable either as character setups (especially QAPawn) use a separate actor for inventory and hot-swap
			// which gun your Pawn is holding by changing the attachment of the weapon skeletal mesh. In this case we do want to record the separate actor automatically
			// as it's pretty hard for the user to add the player already, much less actors the player spawns.
			// The current solution is as follows:
			// If the Scene Component is the Root Component of its owner then we do NOT add that actor to be recorded. This solves the case of nestled hierarchies. Do not recurse children.
			// If the Scene Component is not the Root Component of its owner, then we DO add the owner actor to be recorded (but skip the component as that actor will record it)
			// If the Scene Component's owner is spawned at Runtime, we record it.
			if (SceneComponent->GetOwner() != Target.Get())
			{
			#if WITH_EDITOR
				const bool bActorIsTemporary = GEditor && (SceneComponent->GetOwner()->GetWorld()->WorldType == EWorldType::PIE && !GEditor->ObjectsThatExistInEditorWorld.Get(SceneComponent->GetOwner()));
			#else // WITH_EDITOR
				const bool bActorIsTemporary = false;
			#endif // WITH_EDITOR
				
				if (bActorIsTemporary)
				{
					if (bUpdateReferencedActorList)
					{
						// Only log if they care about the referenced actors
						UE_LOGF(LogTakesCore, Log, "Detected Runtime-Spawned Actor %ls that is attached to current hierarchy. Adding Actor to list to be recorded so we can re-create this hierarchy through Attach Tracks!",
							*SceneComponent->GetName());
						NewReferencedActors.Add(SceneComponent->GetOwner());
					}
					continue;
				}
				// This component belongs to another actor. We check to see if it's the root component to decide if we should record it or not.
				else if (SceneComponent == SceneComponent->GetOwner()->GetRootComponent())
				{
					if (bUpdateReferencedActorList)
					{
						// Only log if they care about the referenced actors.
						UE_LOGF(LogTakesCore, Warning, "Detected Root Component %ls on Actor %ls attached to current hierarchy. Skipping the automatic addition of this actor to the Recording to avoid recording hierarchies created in the World Outliner!",
							*SceneComponent->GetName(), *SceneComponent->GetOwner()->GetName());
					}
					continue;
				}
				else
				{
					if (bUpdateReferencedActorList)
					{
						// Only log if they care about the referenced actors
						UE_LOGF(LogTakesCore, Log, "Detected Component %ls from Actor %ls that is attached to current hierarchy. Adding Actor to list to be recorded so we can re-create this hierarchy through Attach Tracks!",
							*SceneComponent->GetName(), *SceneComponent->GetOwner()->GetName());
						NewReferencedActors.Add(SceneComponent->GetOwner());
					}
					continue;
				}
			}

			if (SceneComponent->ComponentHasTag(DoNotRecordTag))
			{
				UE_LOGF(LogTakesCore, Warning, "Skipping record component: %ls with do not record tag", *SceneComponent->GetName());
				continue;
			}

			if (!Module.CanRecord(FCanRecordArgs(SceneComponent)))
			{
				UE_LOGF(LogTakesCore, Warning, "Skipping record component: %ls because an external module skipped it", *SceneComponent->GetName());
				continue;
			}

			// We own this component so we're going to go ahead and return it so that we record it.
			OutArray.Add(SceneComponent);
		}
	}
}

void UTakeRecorderActorSource::GetActorComponents(AActor* OnActor, TSet<TWeakObjectPtr<UActorComponent>>& OutArray) const
{
	if (OnActor)
	{
		TInlineComponentArray<UActorComponent*> ActorComponents(OnActor);
		OutArray.Reserve(ActorComponents.Num());

		using namespace UE::TakeRecorderSources;
		FTakeRecorderSourcesModule& Module = FTakeRecorderSourcesModule::Get();
		
		for (UActorComponent* ActorComponent : ActorComponents)
		{
			USceneComponent* SceneComponent = Cast<USceneComponent>(ActorComponent);

			// Child of the root component are gathered in GetSceneComponents(). 
			// Here we gather the rest of the components - either non scene components or 
			// scene components that are not directly attached to the root component. This 
			// includes spawned particle systems
			if (!SceneComponent || !SceneComponent->GetAttachParent())
			{
				if (ActorComponent->GetOwner() != Target.Get())
				{
					UE_LOGF(LogTakesCore, Warning, "Unsupported Functionality: Actor Component: %ls is owned by another Actor: %ls, skipping record!",
						*ActorComponent->GetName(), *ActorComponent->GetOwner()->GetName());
					continue;
				}

				if (ActorComponent->ComponentHasTag(DoNotRecordTag))
				{
					UE_LOGF(LogTakesCore, Warning, "Skipping record component: %ls with do not record tag", *ActorComponent->GetName());
					continue;
				}

				if (!Module.CanRecord(FCanRecordArgs(ActorComponent)))
				{
					UE_LOGF(LogTakesCore, Warning, "Skipping record component: %ls because an external module skipped it", *ActorComponent->GetName());
					continue;
				}

				OutArray.Add(ActorComponent);
			}
		}
	}
}

void UTakeRecorderActorSource::CreateNewActorSourceForReferencedActors()
{
	UTakeRecorderSources* SourcesList = GetTypedOuter<UTakeRecorderSources>();
	TArray<UTakeRecorderSource*> NewSources;

	TSet<AActor*> ActorsWithEnabledSources;
	TSet<AActor*> ActorsWithDisabledSources;
	for (UTakeRecorderSource* Source : SourcesList->GetSources())
	{
		if (UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source))
		{
			AActor* TargetActor = ActorSource->Target.Get();
			if (TargetActor)
			{
				if (ActorSource->bEnabled)
				{
					ActorsWithEnabledSources.Add(TargetActor);
				}
				else
				{
					ActorsWithDisabledSources.Add(TargetActor);
				}
			}
		}
	}
	
	for (AActor* Actor : NewReferencedActors)
	{
		// Don't create a recording for this actor if there is an existing source for this actor. Another source may have added it
		// or the user may have added it by hand and adjusted settings.
		if (ActorsWithEnabledSources.Contains(Actor))
		{
			continue;
		}

		// Also, don't create a recording if the actor has a disabled source
		if (ActorsWithDisabledSources.Contains(Actor))
		{
			UE_LOGF(LogTakesCore, Warning, "Disregarding automatically adding %ls as a recording source because it has been explicitly disabled.", *Actor->GetName());
			continue;
		}

		if (Actor == Target.Get())
		{
			// This probably shouldn't happen but safe guard to keep us from creating a new recording for ourself. We won't
			// fail the above check as the recording hasn't gotten created yet.
			continue;
		}

		// We don't use AddSource on the UTakeRecorderSources because this is called from functions that also adds returned items to the Source List. This prevents a 
		// double add from occuring.
		UTakeRecorderActorSource* ActorSource = NewObject<UTakeRecorderActorSource>(SourcesList, UTakeRecorderActorSource::StaticClass(), NAME_None, RF_Transactional);
		ActorSource->ParentSource = this;

		// We add it both to the local list (in case we need to start recording immediately) and to the class list
		// so that we can clean up the recording when we finish.
		NewSources.Add(ActorSource);
		AddedActorSources.Add(ActorSource);

		// Setting the source actor through the function ensures internal property maps are also updated.
		ActorSource->SetSourceActor(Actor);

		// For consistency in the hierarchy, actor sources should have the same state as the source automatically adding them
		ActorSource->RecordType = RecordType;
	}
	NewReferencedActors.Reset();
}

bool UTakeRecorderActorSource::IsOtherActorBeingRecorded(AActor* OtherActor) const
{
	return TakeRecorderSourcesUtils::IsActorBeingRecorded(this, OtherActor);
}

FGuid UTakeRecorderActorSource::GetRecordedActorGuid(class AActor* OtherActor) const
{
	return TakeRecorderSourcesUtils::GetRecordedActorGuid(this, OtherActor);
}

ULevelSequence* UTakeRecorderActorSource::GetLevelSequence(const AActor* OtherActor)
{
	return TakeRecorderSourcesUtils::GetLevelSequence(this, OtherActor, RootLevelSequence);
}

FTransform UTakeRecorderActorSource::GetRecordedActorAnimationInitialRootTransform(class AActor* OtherActor) const
{
	return TakeRecorderSourcesUtils::GetRecordedActorAnimationInitialRootTransform(this, OtherActor);
}

FMovieSceneSequenceID UTakeRecorderActorSource::GetLevelSequenceID(class AActor* OtherActor)
{
	return TakeRecorderSourcesUtils::GetLevelSequenceID(this, OtherActor, RootLevelSequence);
}

FTrackRecorderSettings UTakeRecorderActorSource::GetTrackRecorderSettings() const
{
	FTrackRecorderSettings TrackRecorderSettings;

	UTakeRecorderSources* Sources = GetTypedOuter<UTakeRecorderSources>();
	if (!Sources)
	{
		return TrackRecorderSettings;
	}
			
	FTakeRecorderParameters Parameters;
	Parameters.User = GetDefault<UTakeRecorderUserSettings>()->Settings;
	Parameters.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;

#if WITH_EDITOR
	const bool bShouldSavedRecordedAssets = Sources->GetSettings().bSaveRecordedAssets || GEditor == nullptr;
#else // WITH_EDITOR
	const bool bShouldSavedRecordedAssets = Sources->GetSettings().bSaveRecordedAssets;
#endif // WITH_EDITOR

	TrackRecorderSettings.bRecordToPossessable = GetRecordToPossessable();
	TrackRecorderSettings.bReduceKeys = bReduceKeys;
	TrackRecorderSettings.bRemoveRedundantTracks = Parameters.User.bRemoveRedundantTracks;
	TrackRecorderSettings.bSaveRecordedAssets = bShouldSavedRecordedAssets;
	TrackRecorderSettings.ReduceKeysTolerance = Parameters.User.ReduceKeysTolerance;
	TrackRecorderSettings.AttachRecordBehaviour = ResolveAttachRecordBehaviour();

	TrackRecorderSettings.DefaultTracks = Parameters.Project.DefaultTracks;
	TrackRecorderSettings.IncludeAnimationNames = IncludeAnimationNames;
	TrackRecorderSettings.ExcludeAnimationNames = ExcludeAnimationNames;

	TrackRecorderSettings.TransformOrigin = Sources->GetSettings().TransformOrigin;

	return TrackRecorderSettings;
}

const IMovieSceneTrackRecorderHost* UTakeRecorderActorSource::AsTrackRecorderHost() const
{
	return this;
}

bool UTakeRecorderActorSource::IsActorSourceForActor(const AActor* Actor) const
{
#if WITH_EDITOR
	return Actor && Target.LoadSynchronous() && Actor->GetActorLabel() == Target->GetActorLabel();
#else
	// branch untested
	return Actor == Target;
#endif
}

bool UTakeRecorderActorSource::OverwriteExistingTrackData() const
{
	const UTakeRecorderSubsystem* TakeRecorderSubsystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>();
	check(TakeRecorderSubsystem);

	// Sequencer-managed actors (spawnables or possessables with the SequencerActor tag) are
	// recorded via the possessable path, but should respect SpawnableOverwriteBehavior rather
	// than always overwriting. Only short-circuit to true for true (non-Sequencer) possessables.
	const bool bIsTruePossessable = GetRecordToPossessable() && !bTargetIsSequencerActor;
	if ((!TakeRecorderSubsystem->IsRecordingUsingExistingSequence() || bIsTruePossessable) && !bTargetIsSequencerActor)
	{
		// If we're creating a completely new recording or using possessables we always want to overwrite.
		return true;
	}

	switch (SpawnableOverwriteBehavior)
	{
	case ETakeRecorderSpawnableOverwriteBehavior::OverwriteLegacy:
		return true;

	case ETakeRecorderSpawnableOverwriteBehavior::CreateNew:
		return false;

	case ETakeRecorderSpawnableOverwriteBehavior::ProjectDefault:
		if (bTargetIsSequencerActor)
		{
			// For Sequencer-managed actors, the default behavior is to preserve existing track
			// data. The user must explicitly set OverwriteLegacy to re-record over it.
			return false;
		}
		if (const UTakeRecorderSources* SourcesList = GetTypedOuter<UTakeRecorderSources>())
		{
			return SourcesList->GetSettings().SpawnableOverwriteBehavior == ETakeRecorderSpawnableOverwriteBehavior::OverwriteLegacy;
		}
		break;

	default:
		break;
	}

	return Super::OverwriteExistingTrackData();
}

void UTakeRecorderActorSource::PreRemove()
{
	CleanupAttachedDelegates();
}

void UTakeRecorderActorSource::SetAttachRecordBehaviour(ETakeRecorderAttachRecordBehaviour InAttachRecordBehaviour)
{
	AttachRecordBehaviour = InAttachRecordBehaviour;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bRecordParentHierarchy = (AttachRecordBehaviour == ETakeRecorderAttachRecordBehaviour::Hierarchy);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UTakeRecorderActorSource::SetRecordParentHierarchy(bool bValue)
{
	SetAttachRecordBehaviour(bValue
		? ETakeRecorderAttachRecordBehaviour::Hierarchy
		: ETakeRecorderAttachRecordBehaviour::None);
}

void UTakeRecorderActorSource::SetSourceActor(TSoftObjectPtr<AActor> InTarget)
{
	Target = InTarget;

	TrackTint = FColor(67, 148, 135);
	if (AActor* Actor = InTarget.Get())
	{
	#if WITH_EDITOR
		CachedActorLabel = Actor->GetActorLabel();

		// if the target is also attached to a parent, handle that now
		if (AActor* ParentActor = Actor->GetAttachParentActor())
		{
			HandleActorAttached(Actor, ParentActor);
		}

	#else // WITH_EDITOR
		CachedActorLabel = Actor->GetName();
	#endif // WITH_EDITOR
		bTargetIsSequencerActor = FMovieSceneSpawnableAnnotation::Find(Actor).IsSet();
		if (Actor->GetComponentByClass(UCameraComponent::StaticClass()))
		{
			TrackTint = FColor(148, 67, 108);
		}
	}

	// Whenever the actor to record changes we need to rebuild the recorded property map as it
	// displays all possible properties/components to record for the current actor class.
	RebuildRecordedPropertyMap();
}

bool UTakeRecorderActorSource::CheckIfBindingCanBeReused(const UE::TakesCore::FActorSequenceInformation& SequenceInfo) const
{
	return SequenceInfo.IsPossessable() && SequenceInfo.GetControllingSequenceID() == MovieSceneSequenceID::Root;
}

bool UTakeRecorderActorSource::GetRecordToPossessable() const
{	
	if (TargetLevelSequence && !TargetLevelSequence->AllowsSpawnableObjects())
	{
		return true;
	}
	
	// In non-editor worlds, we should defer to the record type versus what the global spawnable specifices.  This is to address cases where we change
	// the recording type on pre-recording. See UE-306474.
	if (Target && Target->GetWorld() && Target->GetWorld()->WorldType == EWorldType::Editor && !AllowsSpawnableObjects())
	{
		return true;
	}

	return ResolveRecordType() == ETakeRecorderActorRecordType::Possessable;
}

void UTakeRecorderActorSource::SetIsTemporarySource(bool bValue)
{
	bIsTemporarySource = bValue;
}

void UTakeRecorderActorSource::InitializeFactorySettingsObject(TSubclassOf<UMovieSceneTrackRecorderSettings> InClass)
{
	ensure(InClass);

	bool bHasExisting = false;
	for (UObject* ExistingSetting : FactorySettings)
	{
		if (ExistingSetting->GetClass() == InClass)
		{
			bHasExisting = true;
		}
	}

	// We only want to add it to the list if we don't already have it so that only one instance shows up in the UI
	// regardless of how many instances of this factory are recording.
	if (!bHasExisting)
	{
		UMovieSceneTrackRecorderSettings* NewSettingsObject = NewObject<UMovieSceneTrackRecorderSettings>(this, InClass, NAME_None, RF_Transactional);
		FactorySettings.Add(NewSettingsObject);
	}
}

UMovieSceneTrackRecorderSettings* UTakeRecorderActorSource::GetSettingsObjectForFactory(TSubclassOf<UMovieSceneTrackRecorderSettings> InClass) const
{
	for (UObject* ExistingSetting : FactorySettings)
	{
		if (ExistingSetting->GetClass() == InClass)
		{
			return Cast<UMovieSceneTrackRecorderSettings>(ExistingSetting);
		}
	}

	// Most factories won't have a settings object and that's okay!
	return nullptr;
}

FString UTakeRecorderActorSource::GetSubsceneTrackName(ULevelSequence* InSequence) const
{
#if WITH_EDITOR
	if (const UTakeMetaData* TakeMetaData = InSequence->FindMetaData<UTakeMetaData>())
	{
		if (const UMovieSceneAnimationTrackRecorderEditorSettings* Settings = GetBestTrackRecorderEditorSettings())
		{
			UTakeRecorderNamingTokensContext* Context = NewObject<UTakeRecorderNamingTokensContext>();
			Context->Actor = Target.LoadSynchronous();
			return TakeMetaData->GenerateAssetPath(Settings->AnimationTrackName.ToString(), Context);
		}
	}
#endif // WITH_EDITOR
	
	if (Target.IsValid())
	{
	#if WITH_EDITOR
		return Target->GetActorLabel();
	#else // WITH_EDITOR
		return Target->GetName();
	#endif // WITH_EDITOR
	}

	return Super::GetSubsceneTrackName(InSequence);
}

FString UTakeRecorderActorSource::GetSubsceneAssetName(ULevelSequence* InSequence) const
{
#if WITH_EDITOR
	if (const UTakeMetaData* TakeMetaData = InSequence->FindMetaData<UTakeMetaData>())
	{
		if (const UMovieSceneAnimationTrackRecorderEditorSettings* Settings = GetBestTrackRecorderEditorSettings())
		{
			UTakeRecorderNamingTokensContext* Context = NewObject<UTakeRecorderNamingTokensContext>();
			Context->Actor = Target.LoadSynchronous();
			return TakeMetaData->GenerateAssetPath(Settings->AnimationAssetName, Context);
		}
	}
#endif // WITH_EDITOR
	
	if (Target.IsValid())
	{
	#if WITH_EDITOR
		return Target->GetActorLabel();
	#else // WITH_EDITOR
		return Target->GetName();
	#endif // WITH_EDITOR
	}

	return Super::GetSubsceneAssetName(InSequence);
}

void UTakeRecorderActorSource::AddContentsToFolder(UMovieSceneFolder* InFolder)
{
	InFolder->AddChildObjectBinding(CachedObjectBindingGuid);
}

bool UTakeRecorderActorSource::ShouldCreateFolderForContents() const
{
	// If our target is already driven by a sequence, we dont want to create new folders. Otherwise yes.
	return !UE::TakesCore::FActorSequenceInformation(GetRootLevelSequence(), Target.Get()).IsControlledBySequence();
}

UActorRecorderPropertyMap* UTakeRecorderActorSource::GetPropertyMapForComponentRecursive(UActorComponent* InComponent, UActorRecorderPropertyMap* CurrentPropertyMap)
{
	check(CurrentPropertyMap);
	if (CurrentPropertyMap->RecordedObject.Get() == InComponent)
	{
		return CurrentPropertyMap;
	}

	for (UActorRecorderPropertyMap* Child : CurrentPropertyMap->Children)
	{
		UActorRecorderPropertyMap* ChildMap = GetPropertyMapForComponentRecursive(InComponent, Child);
		if (ChildMap)
		{
			return ChildMap;
		}
	}

	return nullptr;
}

UActorRecorderPropertyMap* UTakeRecorderActorSource::GetParentPropertyMapForComponent(UActorComponent* InComponent)
{
	check(InComponent);

	if (USceneComponent* SceneComponent = Cast<USceneComponent>(InComponent))
	{
		USceneComponent* AttachParent = SceneComponent->GetAttachParent();
		if (AttachParent)
		{
			return GetPropertyMapForComponentRecursive(AttachParent, RecordedProperties);
		}
	}

	// ActorComponents and Root Scene Components will go through this path and we'll use the root actor property map.
	return RecordedProperties;
}

#undef LOCTEXT_NAMESPACE // "UTakeRecorderActorSource"

