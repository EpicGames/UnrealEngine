// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderCameraCutSource.h"

#include "TakeRecorderSources.h"
#include "TakeRecorderSource.h"
#include "TakeRecorderActorSource.h"
#include "TakeRecorderSourcesUtils.h"
#include "TakesUtils.h"

#include "LevelSequence.h"
#include "Engine/Engine.h"
#include "Camera/PlayerCameraManager.h"
#include "GameFramework/PlayerController.h"
#include "Engine/GameInstance.h"

#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Sections/MovieSceneCameraCutSection.h"

#if WITH_EDITOR
#include "LevelEditorViewport.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderCameraCutSource)

#define LOCTEXT_NAMESPACE "UTakeRecorderCameraCutSource"

UTakeRecorderCameraCutSource::UTakeRecorderCameraCutSource(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	TrackTint = FColor(160, 160, 160);
}

TArray<UTakeRecorderSource*> UTakeRecorderCameraCutSource::PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer)
{
	World = TakeRecorderSourcesUtils::GetSourceWorld(InSequence);
	RootLevelSequence = InRootSequence;

	return TArray<UTakeRecorderSource*>();
}

void UTakeRecorderCameraCutSource::TickRecording(const FQualifiedFrameTime& CurrentTime)
{	
	AActor* Target = nullptr;
	if (!World)
	{
		return;
	}

#if WITH_EDITOR
	if (World->WorldType == EWorldType::Editor)
	{
		if (GCurrentLevelEditingViewportClient)
		{
			UCameraComponent* CameraComponent = GCurrentLevelEditingViewportClient->GetCameraComponentForView();
			if (CameraComponent)
			{
				Target = CameraComponent->GetOwner();
			}
		}
	}
	else
#endif // WITH_EDITOR
	{
		APlayerController* PC = World->GetGameInstance() ? World->GetGameInstance()->GetFirstLocalPlayerController() : nullptr;
		APlayerCameraManager* PlayerCameraManager = PC ? ToRawPtr(PC->PlayerCameraManager) : nullptr;
		if (PlayerCameraManager)
		{
			Target = PlayerCameraManager->ViewTarget.Target;
		}
	}

	if (!Target)
	{
		return;
	}

	if (!TakeRecorderSourcesUtils::IsActorBeingRecorded(this, Target))
	{
		UTakeRecorderSources* OwningSources = CastChecked<UTakeRecorderSources>(GetOuter());

		UTakeRecorderActorSource* ActorSource = OwningSources->AddSource<UTakeRecorderActorSource>();
		// Setting the source actor through the function ensures internal property maps are also updated.
		ActorSource->SetSourceActor(Target);

		// This has to be called after setting the Target and propagating the change event so that it has a chance to know what to record
		// about the actor.
		OwningSources->StartRecordingSource(TArray<UTakeRecorderSource*>({ ActorSource }), CurrentTime);

		NewActorSources.Add(ActorSource);
	}

	FGuid RecordedCameraGuid = TakeRecorderSourcesUtils::GetRecordedActorGuid(this, Target);
	FMovieSceneSequenceID RecordedCameraSequenceID = TakeRecorderSourcesUtils::GetLevelSequenceID(this, Target, RootLevelSequence);

	// If this camera is already noted, skip it until it changes
	if (CameraCutData.Num() && CameraCutData.Last().Guid == RecordedCameraGuid && CameraCutData.Last().SequenceID == RecordedCameraSequenceID)
	{
		return;
	}

	CameraCutData.Add(FCameraCutData(RecordedCameraGuid, RecordedCameraSequenceID, CurrentTime));
}


TArray<UTakeRecorderSource*> UTakeRecorderCameraCutSource::PostRecording(class ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled)
{
	// Build the camera cut track
	if (!bCancelled && CameraCutData.Num())
	{
		UMovieSceneTrack* CameraCutTrack = InRootSequence->GetMovieScene()->GetCameraCutTrack();
		if (!CameraCutTrack)
		{
			CameraCutTrack = InRootSequence->GetMovieScene()->AddCameraCutTrack(UMovieSceneCameraCutTrack::StaticClass());
		}
		else
		{
			CameraCutTrack->RemoveAllAnimationData();
		}

 		FFrameRate TickResolution = InRootSequence->GetMovieScene()->GetTickResolution();

		for (int32 CameraCutIndex = 0; CameraCutIndex < CameraCutData.Num(); ++CameraCutIndex)
		{
			FFrameNumber CameraCutTime = CameraCutData[CameraCutIndex].Time.ConvertTo(TickResolution).FloorToFrame();
			TRange<FFrameNumber> Range;

			if (CameraCutIndex == 0)
			{
				FFrameNumber NextCameraCutTime = 
					CameraCutIndex+1 < CameraCutData.Num() ? 
					CameraCutData[CameraCutIndex+1].Time.ConvertTo(TickResolution).FloorToFrame() : 
					InRootSequence->GetMovieScene()->GetPlaybackRange().GetUpperBoundValue();
				Range = TRange<FFrameNumber>(InRootSequence->GetMovieScene()->GetPlaybackRange().GetLowerBoundValue(), NextCameraCutTime);
			}
			else if (CameraCutIndex == CameraCutData.Num()-1)
			{
				Range = TRange<FFrameNumber>(CameraCutTime, InRootSequence->GetMovieScene()->GetPlaybackRange().GetUpperBoundValue());
			}
			else
			{
				FFrameNumber NextCameraCutTime = CameraCutData[CameraCutIndex+1].Time.ConvertTo(TickResolution).FloorToFrame();
				Range = TRange<FFrameNumber>(CameraCutTime, NextCameraCutTime);
			}

			UMovieSceneCameraCutSection* CameraCutSection = Cast<UMovieSceneCameraCutSection>(CameraCutTrack->CreateNewSection());
			FMovieSceneObjectBindingID CameraCutBinding = UE::MovieScene::FRelativeObjectBindingID(CameraCutData[CameraCutIndex].Guid, CameraCutData[CameraCutIndex].SequenceID);
			CameraCutSection->SetCameraBindingID(CameraCutBinding);
			CameraCutSection->SetRange(Range);
			CameraCutTrack->AddSection(*CameraCutSection);
		}
	}

	TArray<UTakeRecorderSource*> SourcesToRemove;
	for (auto NewActorSource : NewActorSources)
	{
		if (NewActorSource.IsValid())
		{
			SourcesToRemove.Add(NewActorSource.Get());
		}
	}

	World = nullptr;
	RootLevelSequence = nullptr;

	return SourcesToRemove;
}

FText UTakeRecorderCameraCutSource::GetDisplayTextImpl() const
{
	return LOCTEXT("Label", "Camera Cuts");
}

FText UTakeRecorderCameraCutSource::GetAddSourceDisplayTextImpl() const
{
    return GetDisplayTextImpl();
}

#undef LOCTEXT_NAMESPACE // "UTakeRecorderCameraCutSource"
