// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderLevelSequenceSource.h"
#include "TakesCoreLog.h"
#include "TakesUtils.h"
#include "GameFramework/Actor.h"
#include "LevelSequenceActor.h"
#include "LevelSequencePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderLevelSequenceSource)

UTakeRecorderLevelSequenceSource::UTakeRecorderLevelSequenceSource(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	TrackTint = FColor(164, 74, 74);
}

TArray<UTakeRecorderSource*> UTakeRecorderLevelSequenceSource::PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer)
{
	UWorld* World = TakesUtils::GetFirstPIEWorld();
	if (!World)
	{
		return TArray<UTakeRecorderSource*>();
	}

	for (auto LevelSequence : LevelSequencesToTrigger)
	{
		if (!LevelSequence)
		{
			continue;
		}

		// Spawn a level sequence actor to playback the level sequence
		ALevelSequenceActor* ActorToTrigger = World->SpawnActor<ALevelSequenceActor>();
		if (!ActorToTrigger)
		{
			UE_LOGF(LogTakesCore, Display, "Unabled to spawn actor to trigger: (%ls)", *LevelSequence->GetPathName());
			continue;
		}

		ULevelSequence* DupLevelSequence = CastChecked<ULevelSequence>(StaticDuplicateObject(LevelSequence, LevelSequence->GetOuter(), NAME_None, RF_AllFlags & ~RF_Transactional));
		ActorToTrigger->SetSequence(DupLevelSequence);

		// Always initialize the player so that the playback settings/range can be initialized from editor.
		ActorToTrigger->InitializePlayer();

		ActorsToTrigger.Add(ActorToTrigger);
	}

	return TArray<UTakeRecorderSource*>();
}

void UTakeRecorderLevelSequenceSource::StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence)
{
	// Play the level sequence actors	
	for (auto ActorToTrigger : ActorsToTrigger)
	{
		ULevelSequencePlayer* SequencePlayer = ActorToTrigger->GetSequencePlayer();
		if (SequencePlayer)
		{
			SequencePlayer->SetDisableCameraCuts(true);
			SequencePlayer->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(SequencePlayer->GetStartTime().Time, EUpdatePositionMethod::Jump));
			SequencePlayer->Play();
		}
		else
		{
			UE_LOGF(LogTakesCore, Display, "Level sequence (%ls) is not initialized for playback", *ActorToTrigger->GetPathName());
		}
	}
}

void UTakeRecorderLevelSequenceSource::StopRecording(class ULevelSequence* InSequence)
{
	// Stop any level sequences that were triggered
	for (auto Actor : ActorsToTrigger)
	{
		if (!Actor.IsValid())
		{
			continue;
		}

		ALevelSequenceActor* ActorToTrigger = Actor.Get();
		ULevelSequencePlayer* SequencePlayer = ActorToTrigger->GetSequencePlayer();
		if (SequencePlayer)
		{
			SequencePlayer->SetDisableCameraCuts(false);
			SequencePlayer->Stop();
		}
		if (ActorToTrigger->GetWorld())
		{
			ActorToTrigger->GetWorld()->DestroyActor(ActorToTrigger);
		}
	}

	ActorsToTrigger.Empty();
}

FText UTakeRecorderLevelSequenceSource::GetDisplayTextImpl() const
{
	if (ActorsToTrigger.Num() == 1 && ActorsToTrigger[0].IsValid())
	{
	#if WITH_EDITOR
		return FText::FromString(ActorsToTrigger[0].Get()->GetActorLabel());
	#else // WITH_EDITOR
		return FText::FromString(ActorsToTrigger[0].Get()->GetName());
	#endif // WITH_EDITOR
	}

	uint32 NumValid = 0;
	for (auto LevelSequence : LevelSequencesToTrigger)
	{
		if (!LevelSequence)
		{
			continue;
		}
		++NumValid;
	}

	if (NumValid)
	{
		return FText::Format(FText::FromString("Level Sequences ({0})"), NumValid);
	}

	return NSLOCTEXT("UTakeRecorderLevelSequenceSource", "LevelSequenceLabel", "Level Sequence (None)");
}

FText UTakeRecorderLevelSequenceSource::GetAddSourceDisplayTextImpl() const
{
    return NSLOCTEXT("UTakeRecorderLevelSequenceSource", "TakeRecorderDisplayName", "Level Sequence");
}

FText UTakeRecorderLevelSequenceSource::GetDescriptionTextImpl() const
{
	uint32 NumQueued = 0;
	uint32 NumPlaying = 0;

	for (auto LevelSequence : LevelSequencesToTrigger)
	{
		if (!LevelSequence)
		{
			continue;
		}
		++NumQueued;
	}
	
	for (auto Actor : ActorsToTrigger)
	{
		if (!Actor.IsValid())
		{
			continue;
		}

		ALevelSequenceActor* ActorToTrigger = Actor.Get();
		ULevelSequencePlayer* SequencePlayer = ActorToTrigger->GetSequencePlayer();
		if (SequencePlayer && SequencePlayer->IsPlaying())
		{
			++NumPlaying;
		}
	}

	return FText::Format(FText::FromString("{0} Playing, {1} Queued"), NumPlaying, NumQueued);
}



