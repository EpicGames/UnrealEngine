// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderLevelVisibilitySource.h"
#include "TakesUtils.h"

#include "TakeRecorderSources.h"
#include "TakeRecorderSourcesUtils.h"
#include "TakeMetaData.h"

#include "LevelSequence.h"
#include "Engine/Engine.h"

#include "MovieSceneFolder.h"
#include "Sections/MovieSceneLevelVisibilitySection.h"
#include "Tracks/MovieSceneLevelVisibilityTrack.h"

#include "Engine/LevelStreaming.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderLevelVisibilitySource)

UTakeRecorderLevelVisibilitySourceSettings::UTakeRecorderLevelVisibilitySourceSettings(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, LevelVisibilityTrackName(NSLOCTEXT("UTakeRecorderLevelVisibilitySource", "DefaultLevelVisibilityTrackName", "Recorded Level Visibility"))
{
	TrackTint = FColor(176, 117, 19);
}

#if WITH_EDITOR
void UTakeRecorderLevelVisibilitySourceSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		SaveConfig();
	}
}
#endif // WITH_EDITOR

FString UTakeRecorderLevelVisibilitySourceSettings::GetSubsceneTrackName(ULevelSequence* InSequence) const
{
#if WITH_EDITOR
	if (UTakeMetaData* TakeMetaData = InSequence->FindMetaData<UTakeMetaData>())
	{
		return FString::Printf(TEXT("Level Visibility_%s"), *TakeMetaData->GenerateAssetPath("{slate}"));
	}
#endif // WITH_EDITOR
	return TEXT("Level Visibility");
}

FString UTakeRecorderLevelVisibilitySourceSettings::GetSubsceneAssetName(ULevelSequence* InSequence) const
{
#if WITH_EDITOR
	if (UTakeMetaData* TakeMetaData = InSequence->FindMetaData<UTakeMetaData>())
	{
		return FString::Printf(TEXT("Level Visibility_%s"), *TakeMetaData->GenerateAssetPath("{slate}_{take}"));
	}
#endif // WITH_EDITOR
	return TEXT("Level Visibility");
}

UTakeRecorderLevelVisibilitySource::UTakeRecorderLevelVisibilitySource(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{}

TArray<UTakeRecorderSource*> UTakeRecorderLevelVisibilitySource::PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer)
{
	UWorld* World = TakeRecorderSourcesUtils::GetSourceWorld(InSequence);

	if (!World)
	{
		return TArray<UTakeRecorderSource*>();
	}

	TArray<FName> VisibleLevelNames;
	TArray<FName> HiddenLevelNames;

	for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
	{
		if (LevelStreaming)
		{
			bool bVisible = LevelStreaming->IsLevelVisible();

			FName ShortLevelName = FPackageName::GetShortFName(LevelStreaming->GetWorldAssetPackageFName());

			if (bVisible)
			{
				VisibleLevelNames.Add(ShortLevelName);
			}
			else
			{
				HiddenLevelNames.Add(ShortLevelName);
			}
		}
	}

	UMovieScene* MovieScene = InSequence->GetMovieScene();
	for (auto Track : MovieScene->GetTracks())
	{
		if (Track->IsA(UMovieSceneLevelVisibilityTrack::StaticClass())
		#if WITH_EDITOR
			&& Track->GetDisplayName().EqualTo(LevelVisibilityTrackName)
		#endif // WITH_EDITOR
			)
		{
			CachedLevelVisibilityTrack = Cast<UMovieSceneLevelVisibilityTrack>(Track);
		}
	}

	if (!CachedLevelVisibilityTrack.IsValid())
	{
		CachedLevelVisibilityTrack = MovieScene->AddTrack<UMovieSceneLevelVisibilityTrack>();
	#if WITH_EDITOR
		CachedLevelVisibilityTrack->SetDisplayName(LevelVisibilityTrackName);
	#endif // WITH_EDITOR
	}

	CachedLevelVisibilityTrack->RemoveAllAnimationData();

	UMovieSceneLevelVisibilitySection* VisibleSection = NewObject<UMovieSceneLevelVisibilitySection>(CachedLevelVisibilityTrack.Get(), UMovieSceneLevelVisibilitySection::StaticClass());
	VisibleSection->SetVisibility(ELevelVisibility::Visible);
	VisibleSection->SetRowIndex(0);
	VisibleSection->SetLevelNames(VisibleLevelNames);

	UMovieSceneLevelVisibilitySection* HiddenSection = NewObject<UMovieSceneLevelVisibilitySection>(CachedLevelVisibilityTrack.Get(), UMovieSceneLevelVisibilitySection::StaticClass());
	HiddenSection->SetVisibility(ELevelVisibility::Hidden);
	HiddenSection->SetRowIndex(1);
	HiddenSection->SetLevelNames(HiddenLevelNames);

	CachedLevelVisibilityTrack->AddSection(*VisibleSection);
	CachedLevelVisibilityTrack->AddSection(*HiddenSection);

	return TArray<UTakeRecorderSource*>();
}

void UTakeRecorderLevelVisibilitySource::TickRecording(const FQualifiedFrameTime& CurrentTime)
{
	if (CachedLevelVisibilityTrack.IsValid())
	{
		FFrameRate   TickResolution = CachedLevelVisibilityTrack->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame = CurrentTime.ConvertTo(TickResolution).FloorToFrame();

		for (UMovieSceneSection* Section : CachedLevelVisibilityTrack->GetAllSections())
		{
			Section->ExpandToFrame(CurrentFrame);
		}
	}
}

void UTakeRecorderLevelVisibilitySource::AddContentsToFolder(UMovieSceneFolder* InFolder)
{
	if (CachedLevelVisibilityTrack.IsValid())
	{
		InFolder->AddChildTrack(CachedLevelVisibilityTrack.Get());
	}
}

FText UTakeRecorderLevelVisibilitySource::GetDisplayTextImpl() const
{
	return NSLOCTEXT("UTakeRecorderLevelVisibilitySource", "Label", "Level Visibility");
}

FText UTakeRecorderLevelVisibilitySource::GetAddSourceDisplayTextImpl() const
{
	return NSLOCTEXT("UTakeRecorderLevelVisibilitySource", "TakeRecorderDisplayName", "Level Visibility");
}

bool UTakeRecorderLevelVisibilitySource::CanAddSource(UTakeRecorderSources* InSources) const
{
	for (UTakeRecorderSource* Source : InSources->GetSources())
	{
		if (Source->IsA<UTakeRecorderLevelVisibilitySource>())
		{
			return false;
		}
	}
	return true;
}

