// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderMicrophoneAudioManager.h"

#include "TakesCoreLog.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "IAudioCaptureEditor.h"
#include "IAudioCaptureEditorModule.h"
#include "AudioCaptureEditorTypes.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderMicrophoneAudioManager)


UTakeRecorderMicrophoneAudioManager::UTakeRecorderMicrophoneAudioManager(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
}

#if WITH_EDITOR
TUniquePtr<IAudioCaptureEditor> UTakeRecorderMicrophoneAudioManager::CreateAudioRecorderObject()
{
	IAudioCaptureEditorModule& Factory = FModuleManager::Get().LoadModuleChecked<IAudioCaptureEditorModule>("AudioCaptureEditor");
	return Factory.CreateAudioRecorder();
}

void UTakeRecorderMicrophoneAudioManager::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		SaveConfig();
	}

	// Notify the Mic sources of the new device channel count
	GetOnNotifySourcesOfDeviceChange().Broadcast(GetDeviceChannelCount());
	// Notify the UI of the change so it can update if needed
	GetOnAudioInputDeviceChanged().Broadcast();
	OnAudioInputDevicePropertyChanged.Broadcast(AudioInputDevice);
}
#endif // WITH_EDITOR

FAudioInputDeviceProperty UTakeRecorderMicrophoneAudioManager::GetAudioInputDevice() const
{
	return AudioInputDevice;
}

void UTakeRecorderMicrophoneAudioManager::SetAudioInputDevice(const FAudioInputDeviceProperty& InAudioInputDevice)
{
	AudioInputDevice = InAudioInputDevice;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		SaveConfig();
	}

	// Mirror PostEditChangeProperty so BP-driven writes keep mic sources and UI customizations in sync.
	GetOnNotifySourcesOfDeviceChange().Broadcast(GetDeviceChannelCount());
	GetOnAudioInputDeviceChanged().Broadcast();
	OnAudioInputDevicePropertyChanged.Broadcast(AudioInputDevice);
}

void UTakeRecorderMicrophoneAudioManager::EnumerateAudioDevices(bool InForceRefresh)
{
#if WITH_EDITOR
	if (!AudioRecorder.IsValid() || InForceRefresh)
	{
		AudioRecorder = CreateAudioRecorderObject();
	}
#endif // WITH_EDITOR

	if (AudioInputDevice.DeviceInfoArray.Num() == 0 || InForceRefresh)
	{
		BuildDeviceInfoArray();
	}
}

void UTakeRecorderMicrophoneAudioManager::StartRecording(int32 InChannelCount)
{
	if (InChannelCount <= 0)
	{
		UE_LOGF(LogTakesCore, Error, "Microphone Audio Source will not start. No active Mic Sources have been assigned audio device channels.");
		return;
	}

	if (!GIsEditor && !IsInAudioThread())
	{
		UE_LOGF(LogTakesCore, Error, "Microphone Audio Source will not start. Running in game with audio thread. Audio thread is not supported with take recording set UseAudioThread=false in DefaultEngine.ini to record audio in -game.");
		return;
	}
	
#if WITH_EDITOR
	if (AudioRecorder.IsValid())
	{
		if (AudioRecorder->IsReadyToRecord())
		{
			UE_LOGF(LogTakesCore, Verbose, "Microphone Audio Source AudioRecorder Device: %ls", *(AudioInputDevice.DeviceId));

			FTakeRecorderAudioSettings RecorderSettings;
			RecorderSettings.AudioCaptureDeviceId = AudioInputDevice.DeviceId;
			RecorderSettings.NumRecordChannels = InChannelCount;
			RecorderSettings.AudioInputBufferSize = AudioInputDevice.AudioInputBufferSize;

			AudioRecorder->Start(RecorderSettings);
		}
	}
	else
	{
		UE_LOGF(LogTakesCore, Error, "Microphone Audio Source could not start. Please check that the AudioCapture plugin is enabled");
	}
#else // WITH_EDITOR
	UE_LOG(LogTakesCore, Warning, TEXT("Microphone Audio Source is not currently supported in Cooked/Runtime builds. No microphone audio will be recorded."))
#endif // WITH_EDITOR
}

void UTakeRecorderMicrophoneAudioManager::StopRecording()
{
#if WITH_EDITOR
	if (AudioRecorder.IsValid() && AudioRecorder->IsRecording())
	{
		AudioRecorder->Stop();
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
TObjectPtr<USoundWave> UTakeRecorderMicrophoneAudioManager::GetRecordedSoundWave(const FTakeRecorderAudioSourceSettings& InSourceSettings)
{
	if (AudioRecorder.IsValid() && AudioRecorder->IsStopped())
	{
		return AudioRecorder->GetRecordedSoundWave(InSourceSettings);
	}

	return nullptr;
}
#endif // WITH_EDITOR

void UTakeRecorderMicrophoneAudioManager::FinalizeRecording()
{
#if WITH_EDITOR
	if (AudioRecorder.IsValid() && !AudioRecorder->IsReadyToRecord())
	{
		AudioRecorder.Reset();
	}

	if (!AudioRecorder.IsValid())
	{
		AudioRecorder = CreateAudioRecorderObject();
		check(AudioRecorder.IsValid());
	}
#endif // WITH_EDITOR
}

int32 UTakeRecorderMicrophoneAudioManager::GetDeviceChannelCount()
{
	if (!AudioInputDevice.DeviceId.IsEmpty())
	{
		for (const FAudioInputDeviceInfoProperty& DeviceInfo : AudioInputDevice.DeviceInfoArray)
		{
			if (DeviceInfo.DeviceId == AudioInputDevice.DeviceId)
			{
				return DeviceInfo.InputChannels;
			}
		}
	}

	return 0;
}

bool UTakeRecorderMicrophoneAudioManager::IsAudioDeviceAvailable(const FString& InDeviceId)
{
	if (!InDeviceId.IsEmpty())
	{
		for (const FAudioInputDeviceInfoProperty& DeviceInfo : AudioInputDevice.DeviceInfoArray)
		{
			if (DeviceInfo.DeviceId == InDeviceId)
			{
				return true;
			}
		}
	}

	return false;
}

void UTakeRecorderMicrophoneAudioManager::BuildDeviceInfoArray()
{
	AudioInputDevice.DeviceInfoArray.Empty();

	FString DefaultDeviceId;
	FString DefaultDeviceName;

#if WITH_EDITOR
	FTakeRecorderAudioDeviceInfo DefaultDeviceInfo;
	const bool bFoundDefaultDevice = AudioRecorder->GetCaptureDeviceInfo(DefaultDeviceInfo);
	DefaultDeviceId = DefaultDeviceInfo.DeviceId;
	DefaultDeviceName = DefaultDeviceInfo.DeviceName;
#else // WITH_EDITOR
	const bool bFoundDefaultDevice = false;
#endif // WITH_EDITOR

#if WITH_EDITOR
	TArray<FTakeRecorderAudioDeviceInfo> CaptureDevicesAvailable;
	AudioRecorder->GetCaptureDevicesAvailable(CaptureDevicesAvailable);
	for (const FTakeRecorderAudioDeviceInfo& DeviceInfo : CaptureDevicesAvailable)
	{
		const bool bIsDefaultDevice = bFoundDefaultDevice && (DeviceInfo.DeviceId == DefaultDeviceId);
		AudioInputDevice.DeviceInfoArray.Add(FAudioInputDeviceInfoProperty(DeviceInfo.DeviceName, DeviceInfo.DeviceId, DeviceInfo.InputChannels, DeviceInfo.PreferredSampleRate, bIsDefaultDevice));
	}
#endif // WITH_EDITOR

	// Device names often have numbers in them so perform
	// an alpha-numeric sort.
	Algo::Sort(AudioInputDevice.DeviceInfoArray,
		[](const FAudioInputDeviceInfoProperty& Left, const FAudioInputDeviceInfoProperty& Right) -> bool
		{
			auto StrToNum = [](const FString& InString, int32& InIndex) -> int32
			{
				const int32 StrLen = InString.Len();
				int32 Value = InString[InIndex] - '0';

				while ((InIndex + 1) < StrLen && FChar::IsDigit(InString[InIndex + 1]))
				{
					Value = (Value * 10) + (InString[++InIndex] - '0');
				}

				return Value;
			};

			const FString& LeftString = Left.DeviceName;
			const FString& RightString = Right.DeviceName;
			const int32 LeftLen = LeftString.Len();
			const int32 RightLen = RightString.Len();
			int32 LeftIndex = 0;
			int32 RightIndex = 0;

			while (LeftIndex < LeftLen && RightIndex < RightLen)
			{
				FString::ElementType LeftChar = LeftString[LeftIndex];
				FString::ElementType RightChar = RightString[RightIndex];
				if (FChar::IsDigit(LeftChar) && FChar::IsDigit(RightChar))
				{
					int32 LeftValue = StrToNum(LeftString, LeftIndex);
					int32 RightValue = StrToNum(RightString, RightIndex);

					if (LeftValue != RightValue)
					{
						return LeftValue < RightValue;
					}
				}
				else if (LeftChar != RightChar)
				{
					return LeftChar < RightChar;
				}

				++LeftIndex;
				++RightIndex;
			}

			return LeftLen < RightLen;
		});

	if (bFoundDefaultDevice)
	{
		if (AudioInputDevice.DeviceId.IsEmpty())
		{
			// Default to using the default input device if not already set
			AudioInputDevice.DeviceId = DefaultDeviceId;
		}
		else if (!IsAudioDeviceAvailable(AudioInputDevice.DeviceId))
		{
			// Revert to using the default input device if previously saved device is not available
			AudioInputDevice.DeviceId = DefaultDeviceId;
			UE_LOGF(LogTakesCore, Error, "Previously saved audio input device unavailable. Falling back to default device \"%ls\"", *DefaultDeviceName);
		}
	}
}
