// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderMicrophoneAudioSource.h"

#include "NamingTokens/TakeRecorderNamingTokensContext.h"
#include "TakeMetaData.h"
#include "TakeRecorderMicrophoneAudioManager.h"
#include "TakeRecorderSettings.h"
#include "TakeRecorderSources.h"
#include "TakesUtils.h"

#include "LevelSequence.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneFolder.h"
#include "Recorder/TakeRecorderParameters.h"
#include "Sections/MovieSceneAudioSection.h"
#include "Sound/SoundWave.h"
#include "Tracks/MovieSceneAudioTrack.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/PackageName.h"
#include "UObject/UObjectBaseUtility.h"

#if WITH_EDITOR
#include "Editor.h"
#include "AudioCaptureEditorTypes.h"
#include "ObjectTools.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderMicrophoneAudioSource)

UTakeRecorderMicrophoneAudioSourceSettings::UTakeRecorderMicrophoneAudioSourceSettings(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, AudioSourceName(NSLOCTEXT("UTakeRecorderMicrophoneAudioSource", "Label", "Microphone Audio"))
	, AudioTrackName(NSLOCTEXT("UTakeRecorderMicrophoneAudioSource", "DefaultAudioTrackName", "Recorded Audio"))
	, AudioAssetName(TEXT("Audio_{takeName}_Channel_{channel}"))
	, AudioSubDirectory(TEXT("Audio"))
{
	TrackTint = FColor(75, 67, 148);
	AudioSourceName = FText::Format(FText::FromString(TEXT("{0} {1}")), AudioSourceName, FText::FromString(TEXT("{channel}")));
}

#if WITH_EDITOR
void UTakeRecorderMicrophoneAudioSourceSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		SaveConfig();
	}
}
#endif // WITH_EDITOR

FString UTakeRecorderMicrophoneAudioSourceSettings::GetSubsceneTrackName(ULevelSequence* InSequence) const
{
#if WITH_EDITOR
	if (UTakeMetaData* TakeMetaData = InSequence->FindMetaData<UTakeMetaData>())
	{
		// Before Naming Tokens we were always evaluating AudioTrackName_{slate}.
		// @todo NamingTokens - Keep it consistent, but maybe subscene track name should become its own user defined field which includes {slate} by default?
		const FString SubSceneTrackName = FString::Printf(TEXT("%s_%s"), *AudioTrackName.ToString(), *TakeMetaData->GenerateAssetPath("{slate}"));
		return *TakeMetaData->GenerateAssetPath(SubSceneTrackName);
	}
#endif // WITH_EDITOR
	return TEXT("MicrophoneAudio");
}

FString UTakeRecorderMicrophoneAudioSourceSettings::GetSubsceneAssetName(ULevelSequence* InSequence) const
{
#if WITH_EDITOR
	if (UTakeMetaData* TakeMetaData = InSequence->FindMetaData<UTakeMetaData>())
	{
		return TakeMetaData->GenerateAssetPath(AudioAssetName);
	}
#endif // WITH_EDITOR
	return TEXT("MicrophoneAudio");
}

UTakeRecorderMicrophoneAudioSource::UTakeRecorderMicrophoneAudioSource(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, AudioGain(0.0f)
	, bReplaceRecordedAudio(true)
{
}

void UTakeRecorderMicrophoneAudioSource::DoAudioSetup()
{
	UTakeRecorderMicrophoneAudioManager* AudioInputManager = GetAudioInputManager();
	if (!bHaveDoneAudioSetup && AudioInputManager != nullptr)
	{
		AudioInputManager->EnumerateAudioDevices(true);
	
		TUniquePtr<TArray<bool>> ChannelsInUse = GetChannelsInUse(AudioInputManager->GetDeviceChannelCount());
		int32 AvailableChannelNumber = INDEX_NONE;

		for (int32 ChannelIndex = 0; ChannelIndex < ChannelsInUse->Num(); ++ChannelIndex)
		{
			if (!(*ChannelsInUse)[ChannelIndex])
			{
				AvailableChannelNumber = ChannelIndex + 1;
				break;
			}
		}

		if (AvailableChannelNumber != INDEX_NONE)
		{
			SetCurrentInputChannel(AvailableChannelNumber);
		}

		AudioInputManager->GetOnNotifySourcesOfDeviceChange().AddUObject(this, &UTakeRecorderMicrophoneAudioSource::OnNotifySourcesOfDeviceChange);

		bHaveDoneAudioSetup = true;
	}
}

void UTakeRecorderMicrophoneAudioSource::Initialize()
{
	DoAudioSetup();
}

#if WITH_EDITOR
void UTakeRecorderMicrophoneAudioSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	DoAudioSetup();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

UTakeRecorderMicrophoneAudioManager* UTakeRecorderMicrophoneAudioSource::GetAudioInputManager()
{
	return GetMutableDefault<UTakeRecorderMicrophoneAudioManager>();
}

void UTakeRecorderMicrophoneAudioSource::OnNotifySourcesOfDeviceChange(int32 InChannelCount)
{
	SetAudioDeviceChannelCount(InChannelCount);
}

TUniquePtr<TArray<bool>> UTakeRecorderMicrophoneAudioSource::GetChannelsInUse(const int32 InDeviceChannelCount)
{
	TUniquePtr<TArray<bool>> ChannelsInUse = MakeUnique<TArray<bool>>();
	ChannelsInUse->Init(false, InDeviceChannelCount);

	if (UTakeRecorderSources* SourcesList = GetTypedOuter<UTakeRecorderSources>())
	{
		for (UTakeRecorderSource* Source : SourcesList->GetSources())
		{
			if (UTakeRecorderMicrophoneAudioSource* MicSource = Cast<UTakeRecorderMicrophoneAudioSource>(Source))
			{
				int32 MicSourceChannelNumber = MicSource->AudioChannel.AudioInputDeviceChannel;
				int32 ChannelIndex = MicSourceChannelNumber - 1;
				if (ChannelIndex >= 0 && ChannelIndex < InDeviceChannelCount)
				{
					(*ChannelsInUse)[ChannelIndex] = true;
				}
			}
		}
	}

	return ChannelsInUse;
}

void UTakeRecorderMicrophoneAudioSource::SetAudioDeviceChannelCount(int32 InChannelCount)
{
	if (ensure(InChannelCount >= 0))
	{
		if (AudioChannel.AudioInputDeviceChannel > InChannelCount)
		{
			AudioChannel.AudioInputDeviceChannel = 0;
		}
	}
}

FString UTakeRecorderMicrophoneAudioSource::GetAudioTrackName(ULevelSequence* InSequence) const
{
	FString EvaluatedAudioTrackName;
	if (TryEvaluateTokenString(AudioTrackName.ToString(), InSequence, EvaluatedAudioTrackName))
	{
		return EvaluatedAudioTrackName;
	}

	return TEXT("MicrophoneAudio");
}

FString UTakeRecorderMicrophoneAudioSource::GetAudioAssetName(ULevelSequence* InSequence) const
{
	// This is called for both uassets and subsequences, both of which require unique names.
	// Append the channel number if it's not already part of the name to ensure uniqueness.

	FString EvaluatedAudioAssetName;
	if (TryEvaluateTokenString(AudioAssetName, InSequence, EvaluatedAudioAssetName))
	{
		return TakesUtils::SanitizePackageName(EvaluatedAudioAssetName);
	}

	return TEXT("MicrophoneAudio");
}

FString UTakeRecorderMicrophoneAudioSource::GetUniqueAudioAssetName(ULevelSequence* InSequence) const
{
	FString TempStr = GetAudioAssetName(InSequence);

	if (UTakeRecorderSources* SourcesList = GetTypedOuter<UTakeRecorderSources>())
	{
		TempStr = CreateUniqueAudioAssetName(InSequence, SourcesList, TempStr, AudioChannel.AudioInputDeviceChannel);
	}

	return TempStr;
}

FString UTakeRecorderMicrophoneAudioSource::GetSubsceneTrackName(ULevelSequence* InSequence) const
{
	return GetAudioTrackName(InSequence);
}

FString UTakeRecorderMicrophoneAudioSource::GetSubsceneAssetName(ULevelSequence* InSequence) const
{
	return GetUniqueAudioAssetName(InSequence);
}

static FString MakeNewAssetName(const FString& BaseAssetPath, const FString& BaseAssetName)
{
	const FString Dot(TEXT("."));
	FString AssetPath = BaseAssetPath;
	FString AssetName = BaseAssetName;

	AssetPath /= AssetName;
	AssetPath += Dot + AssetName;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));

	// if object with same name exists, try a different name until we don't find one
	int32 ExtensionIndex = 0;
	while (AssetData.IsValid())
	{
		AssetName = FString::Printf(TEXT("%s_%d"), *BaseAssetName, ExtensionIndex);
		AssetPath = (BaseAssetPath / AssetName) + Dot + AssetName;
		AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));

		ExtensionIndex++;
	}

	return AssetName;
}

TArray<UTakeRecorderSource*> UTakeRecorderMicrophoneAudioSource::PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer)
{
	if (AudioChannel.AudioInputDeviceChannel > 0)
	{
		FString TrackName = GetAudioTrackName(InSequence);
		UMovieScene* MovieScene = InSequence->GetMovieScene();
		
		// To review this. Ultimately currently we do not support microphone audio recording in runtime, but theres no reason this shouldnt work better
		// without using the display name.
	#if WITH_EDITOR
		for (auto Track : ReverseIterate(MovieScene->GetTracks()))
		{
			if (Track->IsA(UMovieSceneAudioTrack::StaticClass()) && Track->GetDisplayName().EqualTo(FText::FromString(TrackName)))
			{
				UMovieSceneAudioTrack* AudioTrack = Cast<UMovieSceneAudioTrack>(Track);
				if (!IsTrackAssociatedWithAnySource(AudioTrack))
				{
					CachedAudioTrack = AudioTrack;
					break;
				}
			}
		}
	#endif // WITH_EDITOR

		if (!CachedAudioTrack.IsValid())
		{
			CachedAudioTrack = MovieScene->AddTrack<UMovieSceneAudioTrack>();
		#if WITH_EDITOR
			CachedAudioTrack->SetDisplayName(FText::FromString(TrackName));
		#endif // WITH_EDITOR
		}

		if (bReplaceRecordedAudio)
		{
			CachedAudioTrack->RemoveAllAnimationData();
		}

		FString PathToRecordTo = FPackageName::GetLongPackagePath(InSequence->GetOutermost()->GetPathName());
		FString BaseName;
		UTakeRecorderSources* Sources = GetTypedOuter<UTakeRecorderSources>();

		// When recording into subsequences, the record path is appended to the slate string
		// which results in inconsistent asset names. Here we work around that by using the
		// subsequence name which will be the same as the asset name in the non-subsequence case.
		if (Sources && Sources->GetSettings().bRecordSourcesIntoSubSequences)
		{
			BaseName = InSequence->GetName();
		}
		else
		{
			BaseName = GetUniqueAudioAssetName(InSequence);
		}

		AudioDirectory.Path = PathToRecordTo;
		if (AudioSubDirectory.Len())
		{
			FString TempStr = AudioSubDirectory;
			if (TryEvaluateTokenString(TempStr, InSequence, TempStr))
			{
				TempStr = TakesUtils::SanitizePackageName(TempStr);
			}
			AudioDirectory.Path /= TempStr;
		}

		AssetFileName = MakeNewAssetName(AudioDirectory.Path, BaseName);

		// Add the section here so it is displayed during record (non-subsequence case)
		UMovieSceneAudioSection* NewAudioSection = NewObject<UMovieSceneAudioSection>(CachedAudioTrack.Get(), UMovieSceneAudioSection::StaticClass());
		NewAudioSection->SetRowIndex(0);
		NewAudioSection->SetLooping(false);
		CachedAudioTrack->AddSection(*NewAudioSection);
	}

	return TArray<UTakeRecorderSource*>();
}

void UTakeRecorderMicrophoneAudioSource::AddContentsToFolder(UMovieSceneFolder* InFolder)
{
	if (CachedAudioTrack.IsValid())
	{
		InFolder->AddChildTrack(CachedAudioTrack.Get());
	}
}


void UTakeRecorderMicrophoneAudioSource::StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence)
{
	Super::StartRecording(InSectionStartTimecode, InSectionFirstFrame, InSequence);

	StartTimecode = InSectionStartTimecode;

	if (CachedAudioTrack.IsValid())
	{
		const TArray<UMovieSceneSection*>& AudioSections = CachedAudioTrack->GetAudioSections();
		if (AudioSections.Num() > 0)
		{
			AudioSections[0]->SetRange(TRange<FFrameNumber>(InSectionFirstFrame, InSectionFirstFrame));
		}

		UTakeRecorderMicrophoneAudioManager* AudioInputManager = GetAudioInputManager();
		if (AudioInputManager != nullptr)
		{
			int32 LastChannelInUse = INDEX_NONE;
			TUniquePtr<TArray<bool>> ChannelsInUse = GetChannelsInUse(AudioInputManager->GetDeviceChannelCount());

			for (int32 ChannelIndex = 0; ChannelIndex < ChannelsInUse->Num(); ++ChannelIndex)
			{
				if ((*ChannelsInUse)[ChannelIndex])
				{
					LastChannelInUse = FMath::Max(LastChannelInUse, ChannelIndex + 1);
				}
			}

			AudioInputManager->StartRecording(LastChannelInUse);
		}
	}
}

void UTakeRecorderMicrophoneAudioSource::TickRecording(const FQualifiedFrameTime& CurrentTime)
{
	if (CachedAudioTrack.IsValid())
	{
		FFrameRate   TickResolution = CachedAudioTrack->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame = CurrentTime.ConvertTo(TickResolution).FloorToFrame();

		for (UMovieSceneSection* Section : CachedAudioTrack->GetAllSections())
		{
			if (Section)
			{
				// Not using UMovieSceneSection::ExpandToFrame since audio tracks leave bAutoExpandToKeyframes off preventing the expansion, 
				// and there are no relative keyframes to fix up while the section grows forward during recording.
				Section->SetRange(TRange<FFrameNumber>::Hull(Section->GetRange(), TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame)));
			}
		}
	}
}

void UTakeRecorderMicrophoneAudioSource::StopRecording(class ULevelSequence* InSequence)
{
	Super::StopRecording(InSequence);

	UTakeRecorderMicrophoneAudioManager* AudioInputManager = GetAudioInputManager();
	if (AudioInputManager != nullptr)
	{
		AudioInputManager->StopRecording();
	}
}

TArray<UTakeRecorderSource*> UTakeRecorderMicrophoneAudioSource::PostRecording(ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled)
{
	FTakeRecorderProjectParameters ProjectParameters = GetDefault<UTakeRecorderProjectSettings>()->Settings;
	if (ProjectParameters.bRecordTimecode)
	{
		ProcessRecordedTimes(InSequence);
	}

	GetRecordedSoundWave(InSequence);

	if (!RecordedSoundWave.IsValid())
	{
		return TArray<UTakeRecorderSource*>();
	}

	TArray<UObject*> AssetsToCleanUp;
	if (bCancelled)
	{
		if (USoundWave* SoundWave = RecordedSoundWave.Get()) 
		{
			AssetsToCleanUp.Add(SoundWave);
		}
	}
	else
	{
		if (USoundWave* SoundWave = RecordedSoundWave.Get())
		{
			SoundWave->MarkPackageDirty();

			FAssetRegistryModule::AssetCreated(SoundWave);
		}

		UMovieScene* MovieScene = InSequence->GetMovieScene();
		UTakeRecorderSources* Sources = GetTypedOuter<UTakeRecorderSources>();

		if (ensure(CachedAudioTrack.IsValid()) && ensure(Sources != nullptr))
		{
			if (USoundWave* SoundWave = RecordedSoundWave.Get())
			{
				const TArray<UMovieSceneSection*>& AudioSections = CachedAudioTrack->GetAudioSections();
				if (AudioSections.Num() > 0)
				{
					UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(AudioSections[0]);					
					AudioSection->SetSound(SoundWave);
					AudioSection->TimecodeSource = StartTimecode;
				}

			#if WITH_EDITOR
				const bool bEditorNotAvailable = GEditor == nullptr;
			#else // WITH_EDITOR
				const bool bEditorNotAvailable = true;
			#endif // WITH_EDITOR

				if ((Sources->GetSettings().bSaveRecordedAssets) || bEditorNotAvailable)
				{
					const bool bIsPossessable = TakesUtils::IsSourcePossessable(this);
					TakesUtils::SaveAsset(SoundWave, bIsPossessable);
				}
			}
		}
	}

	// Reset our audio track pointer
	CachedAudioTrack.Reset();
	
	// Reset our sound wave pointer
	RecordedSoundWave.Reset();
	
#if WITH_EDITOR
	if (GEditor && AssetsToCleanUp.Num() > 0)
	{
		ObjectTools::ForceDeleteObjects(AssetsToCleanUp, false);
	}
#endif // WITH_EDITOR

	return TArray<UTakeRecorderSource*>();
}

void UTakeRecorderMicrophoneAudioSource::FinalizeRecording()
{
	UTakeRecorderMicrophoneAudioManager* AudioInputManager = GetAudioInputManager();
	if (AudioInputManager != nullptr)
	{
		AudioInputManager->FinalizeRecording();
	}
}

void UTakeRecorderMicrophoneAudioSource::ProcessRecordedTimes(ULevelSequence* InSequence)
{
	if (ensure(CachedAudioTrack.IsValid()))
	{
		const TArray<TPair<FQualifiedFrameTime, FQualifiedFrameTime>>& RecordedTimes = UTakeRecorderSources::RecordedTimes;
		const TArray<UMovieSceneSection*>& AudioSections = CachedAudioTrack->GetAudioSections();

		TRange<FFrameNumber> FrameRange = AudioSections[0]->GetRange();

		for (const TPair<FQualifiedFrameTime, FQualifiedFrameTime>& RecordedTimePair : RecordedTimes)
		{
			FFrameNumber FrameNumber = RecordedTimePair.Key.Time.FrameNumber;
			if (FrameRange.Contains(FrameNumber))
			{
				StartTimecode = RecordedTimePair.Value.ToTimecode();
				break;
			}
		}
	}
}

void UTakeRecorderMicrophoneAudioSource::GetRecordedSoundWave(ULevelSequence* InSequence)
{
	UTakeRecorderMicrophoneAudioManager* AudioInputManager = GetAudioInputManager();
	if (AudioChannel.AudioInputDeviceChannel > 0 && AudioInputManager != nullptr)
	{
		UMovieScene* MovieScene = InSequence->GetMovieScene();
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	#if WITH_EDITOR
		FTakeRecorderAudioSourceSettings AudioSettings;
		AudioSettings.Directory = AudioDirectory;
		AudioSettings.AssetName = AssetFileName;
		AudioSettings.GainDb = AudioGain;
		AudioSettings.InputChannelNumber = AudioChannel.AudioInputDeviceChannel;
		AudioSettings.StartTimecode = StartTimecode;
		AudioSettings.VideoFrameRate = DisplayRate;

		TObjectPtr<USoundWave> SoundWave = AudioInputManager->GetRecordedSoundWave(AudioSettings);
		if (SoundWave != nullptr)
		{
			FSoundWaveTimecodeInfo TimecodeInfo;
			FSoundTimecodeOffset TimecodeOffset;

			TimecodeOffset.NumOfSecondsSinceMidnight = StartTimecode.ToTimespan(DisplayRate).GetTotalSeconds();
			SoundWave->SetTimecodeOffset(TimecodeOffset);
			
			RecordedSoundWave = SoundWave;
		}
	#endif // WITH_EDITOR
	}
}

FText UTakeRecorderMicrophoneAudioSource::GetDisplayTextImpl() const
{
	// We purposely don't use naming tokens here to avoid evaluating all tokens per tick. It's unlikely we need
	// to support all tokens for AudioSourceName.
	FFormatNamedArguments FormatArgs;
	FormatArgs.Add(TEXT("channel"), FText::FromString(FString::FromInt(AudioChannel.AudioInputDeviceChannel)));
	
	return FText::Format(AudioSourceName, FormatArgs);
}

FText UTakeRecorderMicrophoneAudioSource::GetAddSourceDisplayTextImpl() const
{
    return NSLOCTEXT("UTakeRecorderMicrophoneAudioSource", "TakeRecorderDisplayName", "Microphone Audio");
}

bool UTakeRecorderMicrophoneAudioSource::CanAddSource(UTakeRecorderSources* InSources) const
{
	int32 MicrophoneSourceCount = 0;
	for (UTakeRecorderSource* Source : InSources->GetSources())
	{
		if (Source->IsA<UTakeRecorderMicrophoneAudioSource>())
		{
			++MicrophoneSourceCount;
		}
	}
	return MicrophoneSourceCount < FAudioInputDeviceProperty::MaxInputChannelCount;
}

void UTakeRecorderMicrophoneAudioSource::SetCurrentInputChannel(int32 InChannelNumber)
{
	if (ensure(InChannelNumber >= 0 && InChannelNumber <= FAudioInputDeviceProperty::MaxInputChannelCount))
	{
		AudioChannel.AudioInputDeviceChannel = InChannelNumber;
	}
}

bool UTakeRecorderMicrophoneAudioSource::IsTrackAssociatedWithAnySource(UMovieSceneAudioTrack* InAudioTrack)
{
	UTakeRecorderSources* SourcesList = GetTypedOuter<UTakeRecorderSources>();
	for (UTakeRecorderSource* Source : SourcesList->GetSources())
	{
		if (UTakeRecorderMicrophoneAudioSource* MicSource = Cast<UTakeRecorderMicrophoneAudioSource>(Source))
		{
			if (MicSource->CachedAudioTrack == InAudioTrack)
			{
				return true;
			}
		}
	}

	return false;
}

FString UTakeRecorderMicrophoneAudioSource::CreateUniqueAudioAssetName(ULevelSequence* InSequence, UTakeRecorderSources* InSources, const FString& InAssetName, const int32 InChannelNumber)
{
	FString NewAssetName = InAssetName;

	if (ensure(InSequence) && ensure(InSources) && ensure(InChannelNumber > 0))
	{
		// Enforce unique name across all microphone sources
		// Note, this enforcement is weak as it assumes channel numbers are unique. Strict uniqueness at
		// the file name level is enforced later at asset creation time in MakeNewAssetName().
		for (UTakeRecorderSource* Source : InSources->GetSources())
		{
			if (UTakeRecorderMicrophoneAudioSource* MicSource = Cast<UTakeRecorderMicrophoneAudioSource>(Source))
			{
				int32 ChannelNumber = MicSource->AudioChannel.AudioInputDeviceChannel;

				if (ChannelNumber != InChannelNumber && NewAssetName == MicSource->GetAudioAssetName(InSequence))
				{
					// Append channel number to enforce uniqueness
					NewAssetName = FString::Printf(TEXT("%s_%d"), *NewAssetName, InChannelNumber);
					break;
				}
			}
		}
	}
	
	return NewAssetName;
}

bool UTakeRecorderMicrophoneAudioSource::TryEvaluateTokenString(const FString& InString, const ULevelSequence* InSequence, FString& OutString) const
{
#if WITH_EDITOR
	if (const UTakeMetaData* TakeMetaData = InSequence->FindMetaData<UTakeMetaData>())
	{
		UTakeRecorderNamingTokensContext* Context = NewObject<UTakeRecorderNamingTokensContext>();
		Context->AudioInputDeviceChannel = AudioChannel.AudioInputDeviceChannel;
		OutString = TakeMetaData->GenerateAssetPath(InString, Context);
		return true;
	}
#endif // WITH_EDITOR
	return false;
}
