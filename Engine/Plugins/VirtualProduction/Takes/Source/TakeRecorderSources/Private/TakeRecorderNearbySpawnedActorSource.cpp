// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderNearbySpawnedActorSource.h"

#include "GameFramework/Pawn.h"
#include "TakeRecorderSource.h"
#include "TakeRecorderActorSource.h"
#include "TakeRecorderSourcesUtils.h"
#include "TakesCoreLog.h"

#include "LevelSequence.h"
#include "Engine/Engine.h"
#include "GameFramework/PlayerController.h"

#include "Math/UnitConversion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderNearbySpawnedActorSource)

#define LOCTEXT_NAMESPACE "UTakeRecorderNearbySpawnedActorSource"

UTakeRecorderNearbySpawnedActorSource::UTakeRecorderNearbySpawnedActorSource(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, Proximity(0.f)
	, bFilterSpawnedActors(false)
{
	TrackTint = FColor(167, 167, 59);
}

TArray<UTakeRecorderSource*> UTakeRecorderNearbySpawnedActorSource::PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer)
{
	// Get the sources, add callbacks for new spawned
	UWorld* World = TakeRecorderSourcesUtils::GetSourceWorld(InSequence);

	if (World)
	{
		FDelegateHandle* FoundHandle = ActorSpawningDelegateHandles.Find(World);
		if (FoundHandle == nullptr)
		{
			FDelegateHandle NewHandle = World->AddOnActorSpawnedHandler(FOnActorSpawned::FDelegate::CreateUObject(this, &UTakeRecorderNearbySpawnedActorSource::HandleActorSpawned, InSequence));
			ActorSpawningDelegateHandles.Add(World, NewHandle);
		}
	}

	return TArray<UTakeRecorderSource*>();
}

TArray<UTakeRecorderSource*> UTakeRecorderNearbySpawnedActorSource::PostRecording(class ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled)
{
	// Remove spawn delegates
	for (auto It = ActorSpawningDelegateHandles.CreateConstIterator(); It; ++It)
	{
		TWeakObjectPtr<UWorld> World = It->Key;
		if (World.IsValid())
		{
			World->RemoveOnActorSpawnedHandler(It->Value);
		}
	}

	ActorSpawningDelegateHandles.Empty();

	TArray<UTakeRecorderSource*> SourcesToRemove;
	for (auto SpawnedActorSource : SpawnedActorSources)
	{
		if (SpawnedActorSource.IsValid())
		{
			SourcesToRemove.Add(SpawnedActorSource.Get());
		}
	}
	return SourcesToRemove;
}

FText UTakeRecorderNearbySpawnedActorSource::GetDisplayTextImpl() const
{
	if (!bFilterSpawnedActors)
	{
		return LOCTEXT("LabelAllActors", "All Nearby Spawned Actors");
	}

	bool bHasValidFilter = false;
	for (TSubclassOf<AActor> Subclass : FilterTypes)
	{
		if (Subclass != nullptr)
		{
			bHasValidFilter = true;
			break;
		}
	}

	if (!bHasValidFilter)
	{
		return LOCTEXT("LabelNoActors", "No Nearby Spawned Actors");
	}
	else
	{
		return LOCTEXT("LabelFilteredActors", "Some Nearby Spawned Actors");
	}
}

FText UTakeRecorderNearbySpawnedActorSource::GetAddSourceDisplayTextImpl() const
{
    return LOCTEXT("TakeRecorderDisplayName", "Nearby Spawned Actors");
}

FText UTakeRecorderNearbySpawnedActorSource::GetDescriptionTextImpl() const
{
	FNumericUnit<float> Unit(Proximity, EUnit::Centimeters);
	FString ProximityString = LexToString(Unit.QuantizeUnitsToBestFit());
	if (SpawnedActorSources.Num())
	{
		if (Proximity == 0.f)
		{
			return FText::Format(LOCTEXT("LabelSourceCountInfiniteProximityFormat", "{0} Actors"), SpawnedActorSources.Num());
		}
		else
		{
			return FText::Format(LOCTEXT("LabelSourceCountBoundedProximityFormat", "{0} Actors (within {1})"), SpawnedActorSources.Num(), FText::FromString(ProximityString));
		}
	}
	else
	{
		if (Proximity == 0.f)
		{
			return LOCTEXT("LabelInfiniteProximity", "(within any distance)");
		}
		else
		{
			return FText::Format(LOCTEXT("LabelBoundedProximityFormat", "(within {0})"), FText::FromString(ProximityString));
		}
	}
}

bool UTakeRecorderNearbySpawnedActorSource::IsActorValid(AActor* Actor)
{
	if (Proximity > 0.f)
	{
		APlayerController* Controller = GEngine->GetFirstLocalPlayerController(Actor->GetWorld());
		if (!Controller || !Controller->GetPawn())
		{
			return false;
		}

		APawn* CurrentPlayer = Controller->GetPawn();

		const FTransform ActorTransform = Actor->GetTransform();
		const FVector ActorTranslation = ActorTransform.GetTranslation();

		const FTransform OtherActorTransform = CurrentPlayer->GetTransform();
		const FVector OtherActorTranslation = OtherActorTransform.GetTranslation();

		if ((OtherActorTranslation - ActorTranslation).Size() > Proximity)
		{
			return false;
		}
	}

	if (!bFilterSpawnedActors)
	{
		return true;
	}

	for (auto FilterType : FilterTypes)
	{
		if (*FilterType != nullptr && Actor->IsA(*FilterType))
		{
			return true;
		}
	}

	return false;
}

void UTakeRecorderNearbySpawnedActorSource::HandleActorSpawned(AActor* Actor, class ULevelSequence* InSequence)
{
	if (!InSequence)
	{
		return;
	}

	if (!IsActorValid(Actor))
	{
		return;
	}

	UE_LOGF(LogTakesCore, Log, "Actor: %ls IsValid: %d Unreachable: %d PendingKillPending: %d", *Actor->GetName(), ::IsValid(Actor), Actor->IsUnreachable(), Actor->IsPendingKillPending());

	if (UTakeRecorderActorSource* ActorSource = TakeRecorderSourcesUtils::StartRecordingActorDuringSession(Actor, InSequence))
	{
		SpawnedActorSources.Add(ActorSource);
	}
}

#undef LOCTEXT_NAMESPACE // "UTakeRecorderNearbySpawnedActorSource"
