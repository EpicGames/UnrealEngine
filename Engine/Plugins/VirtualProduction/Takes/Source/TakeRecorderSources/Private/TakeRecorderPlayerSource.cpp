// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderPlayerSource.h"
#include "GameFramework/Pawn.h"
#include "TakesUtils.h"

#include "TakeRecorderSources.h"
#include "TakeRecorderActorSource.h"

#include "LevelSequence.h"
#include "Engine/Engine.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderPlayerSource)


UTakeRecorderPlayerSource::UTakeRecorderPlayerSource(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	TrackTint = FColor(70, 148, 67);
}


TArray<UTakeRecorderSource*> UTakeRecorderPlayerSource::PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer)
{
	TArray<UTakeRecorderSource*> NewSources;

		
	UWorld* PIEWorld = TakesUtils::GetFirstPIEWorld();
	if (!PIEWorld)
	{
		return NewSources;
	}
		
	APlayerController* Controller = GEngine->GetFirstLocalPlayerController(PIEWorld);
	if (!Controller || !Controller->GetPawn())
	{
		return NewSources;
	}

	APawn* CurrentPlayer = Controller->GetPawn();
	UTakeRecorderSources* Sources = GetTypedOuter<UTakeRecorderSources>();

	// Don't add the Player pawn to the recording if we're already recording the Player
	for (auto Source : Sources->GetSources())
	{
		if (Source->IsA<UTakeRecorderActorSource>())
		{
			UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source);
			if (ActorSource->Target.IsValid())
			{
				if (ActorSource->Target.Get() == CurrentPlayer)
				{
					return NewSources;
				}
			}
		}
	}

	UTakeRecorderActorSource* ActorSource = NewObject<UTakeRecorderActorSource>(Sources, UTakeRecorderActorSource::StaticClass(), NAME_None, RF_Transactional);
	ActorSource->SetSourceActor(CurrentPlayer);

	NewSources.Add(ActorSource);

	PlayerActorSource = ActorSource;

	return NewSources;
}

TArray<UTakeRecorderSource*> UTakeRecorderPlayerSource::PostRecording(class ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled)
{
	TArray<UTakeRecorderSource*> SourcesToRemove;

	if (PlayerActorSource.IsValid())
	{
		SourcesToRemove.Add(PlayerActorSource.Get());
	}

	return SourcesToRemove;
}

FText UTakeRecorderPlayerSource::GetDisplayTextImpl() const
{
	return NSLOCTEXT("UTakeRecorderPlayerSource", "Label", "Player");
}

FText UTakeRecorderPlayerSource::GetAddSourceDisplayTextImpl() const
{
	return GetDisplayTextImpl();
}

bool UTakeRecorderPlayerSource::CanAddSource(UTakeRecorderSources* InSources) const
{
	for (UTakeRecorderSource* Source : InSources->GetSources())
	{
		if (Source->IsA<UTakeRecorderPlayerSource>())
		{
			return false;
		}
	}
	return true;
}
