// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderSourceHelpers.h"
#include "TakeRecorderSource.h"
#include "TakeRecorderActorSource.h"
#include "LevelSequenceActor.h"
#include "GameFramework/Actor.h"
#include "TakeRecorderLevelSequenceSource.h"
#include "TakeRecorderTimeProcessing.h"
#include "LevelSequence.h"

#if WITH_EDITOR
#include "Editor.h"
#include "ScopedTransaction.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "TakeRecorderSourceHelpers"

namespace TakeRecorderSourceHelpers
{

void AddActorSources(
	UTakeRecorderSources* TakeRecorderSources,
	TArrayView<AActor* const> ActorsToRecord,
	bool                      bReduceKeys,
	bool                      bShowProgress)
{
	if (ActorsToRecord.Num() > 0)
	{
	#if WITH_EDITOR
		FScopedTransaction Transaction(FText::Format(
			LOCTEXT("AddSources", "Add Recording {0}|plural(one=Source, other=Sources)"), ActorsToRecord.Num()));
	#endif // WITH_EDITOR
		TakeRecorderSources->Modify();

		for (AActor* Actor : ActorsToRecord)
		{
			if (!Actor)
			{
				continue;
			}
			if (Actor->IsA<ALevelSequenceActor>())
			{
				ALevelSequenceActor* LevelSequenceActor = Cast<ALevelSequenceActor>(Actor);

				UTakeRecorderLevelSequenceSource* LevelSequenceSource = nullptr;

				for (UTakeRecorderSource* Source : TakeRecorderSources->GetSources())
				{
					if (Source->IsA<UTakeRecorderLevelSequenceSource>())
					{
						LevelSequenceSource = Cast<UTakeRecorderLevelSequenceSource>(Source);
						break;
					}
				}

				if (!LevelSequenceSource)
				{
					LevelSequenceSource = TakeRecorderSources->AddSource<UTakeRecorderLevelSequenceSource>();
				}

				ULevelSequence* Sequence = LevelSequenceActor->GetSequence();
				if (Sequence)
				{
					if (!LevelSequenceSource->LevelSequencesToTrigger.Contains(Sequence))
					{
						LevelSequenceSource->LevelSequencesToTrigger.Add(Sequence);
					}
				}
			}
			else
			{
				UTakeRecorderActorSource* NewSource = TakeRecorderSources->AddSource<UTakeRecorderActorSource>();
				AActor* TargetActor = Actor;

			#if WITH_EDITOR
				if (AActor* EditorActor = EditorUtilities::GetEditorWorldCounterpartActor(Actor))
				{
					TargetActor = EditorActor;
				}
			#endif // WITH_EDITOR
				
				NewSource->SetSourceActor(TargetActor);

				NewSource->bShowProgressDialog = bShowProgress;
				NewSource->bReduceKeys = bReduceKeys;
			}
		}
	}
}

void RemoveActorSources(UTakeRecorderSources* TakeRecorderSources, TArrayView<AActor* const> ActorsToRemove)
{
	if (TakeRecorderSources->GetSources().IsEmpty())
	{
		return;
	}

#if WITH_EDITOR
	FScopedTransaction Transaction(FText::Format(LOCTEXT(
			"RemoveActorSources", 
			"Remove Recording {0}|plural(one=Source, other=Sources)"), ActorsToRemove.Num()));
#endif // WITH_EDITOR
	TakeRecorderSources->Modify();

	for (UTakeRecorderSource* Source : TakeRecorderSources->GetSourcesCopy())
	{
		if (const UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source))
		{
			if (ActorsToRemove.Contains(ActorSource->Target))
			{
				TakeRecorderSources->RemoveSource(Source);
			}
		}
	}
}

void RemoveAllActorSources(UTakeRecorderSources* TakeRecorderSources)
{
	if (TakeRecorderSources->GetSources().IsEmpty())
	{
		return;
	}

#if WITH_EDITOR
	FScopedTransaction Transaction(FText::Format(LOCTEXT(
			"RemoveAllActorSources", 
			"Remove Recording {0}|plural(one=Source, other=Sources)"), TakeRecorderSources->GetSources().Num()));
#endif // WITH_EDITOR
	TakeRecorderSources->Modify();

	while (!TakeRecorderSources->GetSources().IsEmpty())
	{
		TakeRecorderSources->RemoveSource(TakeRecorderSources->GetSources()[0]);
	}
}
	
AActor* GetSourceActor(UTakeRecorderSource* Source)
{
	if (const UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source))
	{
		return ActorSource->Target.LoadSynchronous();
	}

	return nullptr;
}

void ProcessRecordedTimes(
	ULevelSequence* InSequence, UMovieSceneTakeTrack* TakeTrack,
	const TOptional<TRange<FFrameNumber>>& FrameRange, const TArray<TPair<FQualifiedFrameTime, FQualifiedFrameTime>>& RecordedTimes
	)
{
	return UE::TakesCore::ProcessRecordedTimes(InSequence, TakeTrack, FrameRange, RecordedTimes);
}

}

#undef LOCTEXT_NAMESPACE // "TakeRecorderSourceHelpers"
