// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderSourcesCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "TakeRecorderSourcesCommands"

FTakeRecorderSourcesCommands::FTakeRecorderSourcesCommands()
	: TCommands<FTakeRecorderSourcesCommands>("TakeRecorderSources", LOCTEXT("Common", "Common"), NAME_None, FAppStyle::GetAppStyleSetName())
{
}

void FTakeRecorderSourcesCommands::RegisterCommands()
{
	UI_COMMAND(RecordSelectedActors, "Record Selected Actors", "Record selected actors", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::R) );
}

#undef LOCTEXT_NAMESPACE
