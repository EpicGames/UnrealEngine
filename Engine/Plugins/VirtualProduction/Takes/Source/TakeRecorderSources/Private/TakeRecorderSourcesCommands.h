// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

class FTakeRecorderSourcesCommands : public TCommands<FTakeRecorderSourcesCommands>
{
public:
	FTakeRecorderSourcesCommands();

	TSharedPtr<FUICommandInfo> RecordSelectedActors;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
