// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderSourcesModule.h"

#include "Algo/AllOf.h"
#include "Modules/ModuleManager.h"
#include "ITakeRecorderModule.h"
#include "ITakeRecorderDropHandler.h"
#include "Recorder/TakeRecorder.h"
#include "Recorder/TakeRecorderBlueprintLibrary.h"
#include "Recorder/TakeRecorderPanel.h"
#include "Recorder/TakeRecorderParameters.h"
#include "TakesCoreBlueprintLibrary.h"
#include "TakeMetaData.h"
#include "TakeRecorderSourceHelpers.h"
#include "TakeRecorderActorSource.h"
#include "TakeRecorderMicrophoneAudioManager.h"
#include "TakeRecorderSources.h"
#include "TakeRecorderSourcesCommands.h"
#include "TakeRecorderSettings.h"
#include "TakeRecorderSourcesUtils.h"
#include "Features/IModularFeatures.h"

#include "LevelSequenceActor.h"
#include "LevelSequence.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Engine/Level.h"
#include "Engine/GameInstance.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/SlateIconFinder.h"

#include "Engine/LevelScriptActor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "TakeRecorderMicrophoneAudioSource.h"
#include "TakeRecorderWorldSource.h"
#include "TrackRecorders/MovieSceneAnimationTrackRecorderSettings.h"

#include "ITakeRecorderSourcesModule.h"

#include "Bindings/MovieSceneCustomBinding.h"
#include "Bindings/MovieSceneSpawnableActorBinding.h"

#if WITH_EDITOR
#include "LevelSequenceEditorBlueprintLibrary.h"

#include "Dialogs/Dialogs.h"
#include "Engine/Selection.h"

#include "Editor.h"
#include "LevelEditor.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerPublicTypes.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ActorTreeItem.h"

#include "ISequencer.h"
#include "ISequencerModule.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "TakeRecorderSources"

namespace UE::TakeRecorderSources
{
#if WITH_EDITOR
	static bool AllowMenuExtensions = true;
	FAutoConsoleVariableRef CVarAllowMenuExtensions(TEXT("TakeRecorder.AllowMenuExtensions"), AllowMenuExtensions, TEXT(""), ECVF_Cheat);
#endif // WITH_EDITOR
}

namespace
{
#if WITH_EDITOR
	// ActorLabel is an editor time only concept.
	static AActor* FindActorByLabel(const FString& ActorNameStr, UWorld* InWorld, bool bFuzzy = false)
	{
		// search for the actor by name
		for (ULevel* Level : InWorld->GetLevels())
		{
			if (Level)
			{
				for (AActor* Actor : Level->Actors)
				{
					if (Actor)
					{
						if (Actor->GetActorLabel() == ActorNameStr)
						{
							return Actor;
						}
					}
				}
			}
		}

		// if we want to do a fuzzy search then we return the first actor whose name that starts 
		// the specified string
		if (bFuzzy)
		{
			for (ULevel* Level : InWorld->GetLevels())
			{
				if (Level)
				{
					for (AActor* Actor : Level->Actors)
					{
						if (Actor)
						{
							if (Actor->GetActorLabel().StartsWith(ActorNameStr))
							{
								return Actor;
							}
						}
					}
				}
			}
		}

		return nullptr;
	}
#endif // WITH_EDITOR

	static void FindActorsOfClass(UClass* Class, UWorld* InWorld, TArray<AActor*>& OutActors)
	{
		for (ULevel* Level : InWorld->GetLevels())
		{
			if (Level)
			{
				for (AActor* Actor : Level->Actors)
				{
					if (Actor && Actor->IsA(Class) && !Actor->IsA(ALevelScriptActor::StaticClass()) && !Actor->IsA(ALevelSequenceActor::StaticClass()) && !Actor->GetClass()->HasAnyClassFlags(CLASS_NotPlaceable))
					{
						OutActors.AddUnique(Actor);
					}
				}
			}
		}
	}
}

namespace UE::TakeRecorderSources
{
void FTakeRecorderSourcesModule::StartupModule()
{
	FTakeRecorderSourcesCommands::Register();
	
	BindCommands();

	RegisterMenuExtensions();

	IModularFeatures::Get().RegisterModularFeature(ITakeRecorderDropHandler::ModularFeatureName, &ActorDropHandler);

	ITakeRecorderModule& TakeRecorderModule = FModuleManager::Get().LoadModuleChecked<ITakeRecorderModule>("TakeRecorder");

#if WITH_EDITOR
	SourcesMenuExtension = TakeRecorderModule.RegisterSourcesMenuExtension(FOnExtendSourcesMenu::CreateStatic(ExtendSourcesMenu));
#endif // WITH_EDITOR

	ITakeRecorderModule::FSourceExtensionData ExtensionData;
	ExtensionData.OnAddActorSource = ITakeRecorderModule::FSourceExtensionData::FAddActorSource::CreateStatic(TakeRecorderSourceHelpers::AddActorSources);
	ExtensionData.OnRemoveActorSource = ITakeRecorderModule::FSourceExtensionData::FRemoveActorSource::CreateStatic(TakeRecorderSourceHelpers::RemoveActorSources);
	ExtensionData.OnGetSourceActor = ITakeRecorderModule::FSourceExtensionData::FGetSourceActor::CreateStatic(TakeRecorderSourceHelpers::GetSourceActor);
	TakeRecorderModule.RegisterSourcesExtension(ExtensionData);
	
	TakeRecorderModule.RegisterSettingsObject(GetMutableDefault<UTakeRecorderMicrophoneAudioSourceSettings>());
	TakeRecorderModule.RegisterSettingsObject(GetMutableDefault<UTakeRecorderMicrophoneAudioManager>());
	TakeRecorderModule.RegisterSettingsObject(GetMutableDefault<UMovieSceneAnimationTrackRecorderEditorSettings>());
	TakeRecorderModule.RegisterSettingsObject(GetMutableDefault<UTakeRecorderWorldSourceSettings>());

#if WITH_EDITOR
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	OnSequencerCreatedHandle = SequencerModule.RegisterOnSequencerCreated(FOnSequencerCreated::FDelegate::CreateRaw(this, &FTakeRecorderSourcesModule::OnSequencerCreated));
#endif // WITH_EDITOR
	
	UTakeRecorderSources::OnAddSourceForActorBinding().BindRaw(this, &FTakeRecorderSourcesModule::OnNewActorReferenceBindingAdded);
}

void FTakeRecorderSourcesModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(ITakeRecorderDropHandler::ModularFeatureName, &ActorDropHandler);

#if WITH_EDITOR
	ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
	if (SequencerModule)
	{
		SequencerModule->UnregisterOnSequencerCreated(OnSequencerCreatedHandle);
	}
#endif // WITH_EDITOR
	
	ITakeRecorderModule* TakeRecorderModule = FModuleManager::Get().GetModulePtr<ITakeRecorderModule>("TakeRecorder");
	if (TakeRecorderModule)
	{
		TakeRecorderModule->UnregisterSourcesMenuExtension(SourcesMenuExtension);
		TakeRecorderModule->UnregisterSourcesExtension();
	}

	FTakeRecorderSourcesCommands::Unregister();

	UnregisterMenuExtensions();
	
	UTakeRecorderSources::OnAddSourceForActorBinding().Unbind();
}

void FTakeRecorderSourcesModule::RegisterCanRecordDelegate(FName HandleId, FCanRecordDelegate InDelegate)
{
	CanRecordDelegates.Add(HandleId, MoveTemp(InDelegate));
}
void FTakeRecorderSourcesModule::UnregisterCanRecordDelegate(FName HandleId)
{
	CanRecordDelegates.Remove(HandleId);
}

bool FTakeRecorderSourcesModule::CanRecord(const FCanRecordArgs& InArgs)
{
	return Algo::AllOf(CanRecordDelegates, [&InArgs](const TPair<FName, FCanRecordDelegate>& Pair)
	{
		return Pair.Value.Execute(InArgs);
	});
}

void FTakeRecorderSourcesModule::RegisterMenuExtensions()
{
#if WITH_EDITOR
	if (GEditor)
	{
		// Register level editor menu extender
		LevelEditorMenuExtenderDelegate = FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateRaw(this, &FTakeRecorderSourcesModule::ExtendLevelViewportContextMenu);
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		auto& MenuExtenders = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();
		MenuExtenders.Add(LevelEditorMenuExtenderDelegate);
		LevelEditorExtenderDelegateHandle = MenuExtenders.Last().GetHandle();
	}
#endif
}

void FTakeRecorderSourcesModule::UnregisterMenuExtensions()
{
#if WITH_EDITOR
	// Unregister level editor menu extender
	if (FModuleManager::Get().IsModuleLoaded(TEXT("LevelEditor")))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		LevelEditorModule.GetAllLevelViewportContextMenuExtenders().RemoveAll([&](const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& Delegate) {
			return Delegate.GetHandle() == LevelEditorExtenderDelegateHandle;
		});
	}
#endif // WITH_EDITOR
}

void FTakeRecorderSourcesModule::BindCommands()
{
	CommandList = MakeShareable(new FUICommandList);
	
	CommandList->MapAction(
		FTakeRecorderSourcesCommands::Get().RecordSelectedActors,
		FExecuteAction::CreateLambda( [this] { RecordSelectedActors(); } ) );
}

TSharedRef<FExtender> FTakeRecorderSourcesModule::ExtendLevelViewportContextMenu(const TSharedRef<FUICommandList> InCommandList, const TArray<AActor*> SelectedActors)
{
	TSharedRef<FExtender> Extender(new FExtender());

#if WITH_EDITOR
	if (!TakeRecorderSources::AllowMenuExtensions)
	{
		return Extender;
	}
#endif
	if (SelectedActors.Num() > 0)
	{
		Extender->AddMenuExtension("ActorUETools", EExtensionHook::After, CommandList, FMenuExtensionDelegate::CreateLambda(
			[this, SelectedActors](FMenuBuilder& MenuBuilder) 
		{
			FText RecordText;
			if (SelectedActors.Num() == 1)
			{
			#if WITH_EDITOR
				RecordText = FText::Format(LOCTEXT("RecordSingleSelectedActorText", "Record {0} with Take Recorder"), FText::FromString(SelectedActors[0]->GetActorLabel()));
			#else // WITH_EDITOR
				RecordText = FText::Format(LOCTEXT("RecordSingleSelectedActorText", "Record {0} with Take Recorder"), FText::FromString(SelectedActors[0]->GetName()));
			#endif // WITH_EDITOR
			}
			else
			{
				RecordText = FText::Format(LOCTEXT("RecordSelectedActorsText", "Record {0} actors with Take Recorder"), SelectedActors.Num());
			}

			MenuBuilder.AddMenuEntry(FTakeRecorderSourcesCommands::Get().RecordSelectedActors, NAME_None, RecordText, TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Actors.TakeRecorder"));
		}
		
		));
	}

	return Extender;
}

#if WITH_EDITOR
void FTakeRecorderSourcesModule::ExtendSourcesMenu(TSharedRef<FExtender> Extender, UTakeRecorderSources* Sources)
{
	Extender->AddMenuExtension("Sources", EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateStatic(PopulateSourcesMenu, Sources));
}

void FTakeRecorderSourcesModule::PopulateSourcesMenu(FMenuBuilder& MenuBuilder, UTakeRecorderSources* Sources)
{
	FName ExtensionName = "ActorSourceSubMenu";
	MenuBuilder.AddSubMenu(
		LOCTEXT("ActorList_Label", "From Actor"),
		LOCTEXT("ActorList_Tip", "Add a new recording source from an actor in the current world"),
		FNewMenuDelegate::CreateStatic(PopulateActorSubMenu, Sources),
		FUIAction(),
		ExtensionName,
		EUserInterfaceActionType::Button
	);
}

void FTakeRecorderSourcesModule::PopulateActorSubMenu(FMenuBuilder& MenuBuilder, UTakeRecorderSources* Sources)
{
	TSet<const AActor*> ExistingActors;

	for (UTakeRecorderSource* Source : Sources->GetSources())
	{
		UTakeRecorderActorSource* ActorSource   = Cast<UTakeRecorderActorSource>(Source);
		AActor*                   ExistingActor = ActorSource ? ActorSource->Target.Get() : nullptr;

		if (ExistingActor)
		{
			ExistingActors.Add(ExistingActor);
		}
	}

	auto OutlinerFilterPredicate = [InExistingActors = MoveTemp(ExistingActors)](const AActor* InActor)
	{
		return !InExistingActors.Contains(InActor) && TakeRecorderSourcesUtils::IsActorRecordable(InActor);
	};

	// Set up a menu entry to add the selected actor(s) to the sequencer
	TArray<AActor*> SelectedActors;
	GEditor->GetSelectedActors()->GetSelectedObjects(SelectedActors);
	SelectedActors.RemoveAll([&](const AActor* In){ return !OutlinerFilterPredicate(In); });

	FText SelectedLabel;
	FSlateIcon ActorIcon = FSlateIconFinder::FindIconForClass(AActor::StaticClass());
	if (SelectedActors.Num() == 1)
	{
		SelectedLabel = FText::Format(LOCTEXT("AddSpecificActor", "Add '{0}'"), FText::FromString(SelectedActors[0]->GetActorLabel()));
		ActorIcon = FSlateIconFinder::FindIconForClass(SelectedActors[0]->GetClass());
	}
	else if (SelectedActors.Num() > 1)
	{
		SelectedLabel = FText::Format(LOCTEXT("AddCurrentActorSelection", "Add Current Selection ({0} actors)"), FText::AsNumber(SelectedActors.Num()));
	}

	if (!SelectedLabel.IsEmpty())
	{
		MenuBuilder.AddMenuEntry(
			SelectedLabel,
			FText(),
			ActorIcon,
			FExecuteAction::CreateLambda([Sources, SelectedActors]{
				TakeRecorderSourceHelpers::AddActorSources(Sources, SelectedActors);
			})
		);
	}

	MenuBuilder.BeginSection("ChooseActorSection", LOCTEXT("ChooseActor", "Choose Actor:"));
	{
		// Set up a menu entry to add any arbitrary actor to the sequencer
		FSceneOutlinerInitializationOptions InitOptions;
		{
			// We hide the header row to keep the UI compact.
			InitOptions.bShowHeaderRow = false;
			InitOptions.bShowSearchBox = true;
			InitOptions.bShowCreateNewFolder = false;
			InitOptions.bFocusSearchBoxWhenOpened = true;

			// Only want the actor label column
			InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));

			// Only display actors that are not possessed already
			InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda(OutlinerFilterPredicate));
		}

		// actor selector to allow the user to choose an actor
		FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
		TSharedRef< SWidget > MiniSceneOutliner =
			SNew(SBox)
			.MaxDesiredHeight(400.0f)
			.WidthOverride(300.0f)
			[
				SceneOutlinerModule.CreateActorPicker(
					InitOptions,
					FOnActorPicked::CreateLambda([Sources](AActor* Actor){
						// Create a new binding for this actor
						FSlateApplication::Get().DismissAllMenus();
						TakeRecorderSourceHelpers::AddActorSources(Sources, MakeArrayView(&Actor, 1));
					})
				)
			];

		MenuBuilder.AddWidget(MiniSceneOutliner, FText::GetEmpty(), true);
	}
	MenuBuilder.EndSection();
}
#endif // WITH_EDITOR

bool FTakeRecorderSourcesModule::Exec_Editor(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if WITH_EDITOR
	if (FParse::Command(&Cmd, TEXT("RecordTake")))
	{
		return HandleRecordTakeCommand(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("StopRecordingTake")))
	{
		return HandleStopRecordTakeCommand(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("CancelRecordingTake")))
	{
		return HandleCancelRecordTakeCommand(InWorld, Cmd, Ar);
	}
#endif
	return false;
}

bool FTakeRecorderSourcesModule::HandleRecordTakeCommand(UWorld* InWorld, const TCHAR* InStr, FOutputDevice& Ar)
{
#if WITH_EDITOR
	enum class EFilterType : int32
	{
		None,
		All,
		Actor,
		Class
	};

	bool bShowUsage = false;
	const TCHAR* Str = InStr;
	EFilterType FilterType = EFilterType::None;
	TCHAR Filter[128];
	if (FParse::Token(Str, Filter, UE_ARRAY_COUNT(Filter), 0))
	{
		FString const FilterStr = Filter;
		if (FilterStr == TEXT("all"))
		{
			FilterType = EFilterType::All;
		}
		else if (FilterStr == TEXT("actor"))
		{
			FilterType = EFilterType::Actor;
		}
		else if (FilterStr == TEXT("class"))
		{
			FilterType = EFilterType::Class;
		}
		else
		{
			Ar.Log(ELogVerbosity::Error, TEXT("Couldn't parse recording filter, using actor filters from settings."));
			bShowUsage = true;
		}
	}

	TArray<AActor*> ActorsToRecord;

	if (FilterType == EFilterType::Actor || FilterType == EFilterType::Class)
	{
		TCHAR Specifier[128];
		if (FParse::Token(Str, Specifier, UE_ARRAY_COUNT(Specifier), 0))
		{
			FString const SpecifierStr = FString(Specifier).TrimStart();

			TArray<FString> Splits;
			SpecifierStr.ParseIntoArray(Splits, TEXT(","));

			for (FString Split : Splits)
			{
				if (FilterType == EFilterType::Actor)
				{
					AActor* FoundActor = FindActorByLabel(Split, InWorld, true);
					if (FoundActor)
					{
						ActorsToRecord.Add(FoundActor);
					}
				}
				else
				{
					UClass* FoundClass = UClass::TryFindTypeSlow<UClass>(Split);
					if (FoundClass != nullptr)
					{
						FindActorsOfClass(FoundClass, InWorld, ActorsToRecord);
					}
					else
					{
						Ar.Log(ELogVerbosity::Error, TEXT("Couldn't parse class filter, aborting recording."));
						bShowUsage = true;
					}
				}
			}
		}
	}
	else
	{
		FindActorsOfClass(AActor::StaticClass(), InWorld, ActorsToRecord);
	}

	TOptional<ULevelSequence*> LevelSequence;
	TOptional<ULevelSequence*> RootLevelSequence;

	TCHAR SequenceAsset[128];
	if (FParse::Token(Str, SequenceAsset, UE_ARRAY_COUNT(SequenceAsset), 0))
	{
		FString const SequenceAssetStr = SequenceAsset;
		FString LeftS, RightS;
		if (SequenceAssetStr.Split(TEXT("sequence="), &LeftS, &RightS))
		{
			LevelSequence = LoadObject<ULevelSequence>(nullptr, *RightS);

			if (LevelSequence.IsSet() && !LevelSequence.GetValue())
			{
				Ar.Log(ELogVerbosity::Error, FString::Printf(TEXT("Couldn't find level sequence with path: %s, aborting recording."), *RightS));
				bShowUsage = true;
			}
		}
	}

	TCHAR RootSequenceAsset[128];
	if (FParse::Token(Str, SequenceAsset, UE_ARRAY_COUNT(RootSequenceAsset), 0))
	{
		FString const RootSequenceAssetStr = RootSequenceAsset;
		FString LeftS, RightS;
		if (RootSequenceAssetStr.Split(TEXT("root_sequence="), &LeftS, &RightS))
		{
			RootLevelSequence = LoadObject<ULevelSequence>(nullptr, *RightS);

			if (RootLevelSequence.IsSet() && !RootLevelSequence.GetValue())
			{
				Ar.Log(ELogVerbosity::Error, FString::Printf(TEXT("Couldn't find root level sequence with path: %s, aborting recording."), *RightS));
				bShowUsage = true;
			}
		}
	}

	if (ActorsToRecord.Num() == 0)
	{
		Ar.Log(ELogVerbosity::Error, TEXT("Couldn't find any actors to record, aborting recording."));
		bShowUsage = true;
	}

	// show usage if any errors
	if (bShowUsage)
	{
		Ar.Log(TEXT("Usage: RecordTake filterType options sequence="));
		Ar.Log(TEXT("         filterType = all, actor, class"));
		Ar.Log(TEXT("         options = comma separated options for filterType"));
		Ar.Log(TEXT("         sequence (optional) = path to sequence to record into"));
		Ar.Log(TEXT("         root sequence (optional) = path to root of the sequence being recorded into"));
		Ar.Log(TEXT("         example: RecordTake actor cube,sphere sequence=/Game/RecordSequence"));
		return false;
	}

	RecordActors(ActorsToRecord, LevelSequence, RootLevelSequence);
	return true;
#else
	return false;
#endif
}

bool FTakeRecorderSourcesModule::HandleStopRecordTakeCommand(UWorld* InWorld, const TCHAR* InStr, FOutputDevice& Ar)
{
#if WITH_EDITOR
	if (UTakeRecorder* ActiveRecorder = UTakeRecorder::GetActiveRecorder())
	{
		ULevelSequence* ActiveSequence = ActiveRecorder->GetSequence();
		if (ActiveSequence)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(ActiveSequence);
		}

		ActiveRecorder->Stop();
	}
	return true;
#else
	return false;
#endif
}

bool FTakeRecorderSourcesModule::HandleCancelRecordTakeCommand(UWorld* InWorld, const TCHAR* InStr, FOutputDevice& Ar)
{
#if WITH_EDITOR
	if (UTakeRecorder* ActiveRecorder = UTakeRecorder::GetActiveRecorder())
	{
		ULevelSequence* ActiveSequence = ActiveRecorder->GetSequence();
		if (ActiveSequence)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(ActiveSequence);
		}

		ActiveRecorder->Cancel();
	}
	return true;
#else
	return false;
#endif
}

void FTakeRecorderSourcesModule::RecordActors(const TArray<AActor*>& ActorsToRecord, TOptional<ULevelSequence*> LevelSequence, TOptional<ULevelSequence*> RootLevelSequence)
{
	ETakeRecorderMode TakeRecorderMode = LevelSequence.IsSet() && LevelSequence.GetValue() != nullptr ? ETakeRecorderMode::RecordIntoSequence : ETakeRecorderMode::RecordNewSequence;
			
	FString Slate = GetDefault<UTakeRecorderProjectSettings>()->Settings.DefaultSlate;

	ULevelSequence* RecordLevelSequence = nullptr;

	if (!LevelSequence.IsSet() || LevelSequence.GetValue() == nullptr)
	{
		RecordLevelSequence = NewObject<ULevelSequence>(GetTransientPackage(), NAME_None, RF_Transient);
		RecordLevelSequence->Initialize();
	}
	else
	{
		Slate = LevelSequence.GetValue()->GetName();
		
		if (RootLevelSequence.IsSet() && RootLevelSequence.GetValue() != nullptr)
		{
			RecordLevelSequence = RootLevelSequence.GetValue();
		}
		else
		{
			RecordLevelSequence = LevelSequence.GetValue();
		}
	}
	
	FText ErrorText = LOCTEXT("UnknownError", "An unknown error occurred when trying to start recording");

	if (!RecordLevelSequence)
	{
		return;
	}

	FTakeRecorderParameters Parameters;
	Parameters.User = GetDefault<UTakeRecorderUserSettings>()->Settings;
	Parameters.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;
	Parameters.TakeRecorderMode = TakeRecorderMode;

	// Overrides for what makes sense when recording into a level sequence with Sequencer and no Take Recorder panel
	Parameters.Project.bRecordToPossessable = true;
	Parameters.Project.bRecordSourcesIntoSubSequences = false;

	UTakeMetaData* MetaData = UTakeMetaData::CreateFromDefaults(GetTransientPackage(), NAME_None);
	MetaData->SetFlags(RF_Transactional | RF_Transient);

	MetaData->SetSlate(Slate);

	// Compute the correct starting take number
	int32 NextTakeNumber = UTakesCoreBlueprintLibrary::ComputeNextTakeNumber(MetaData->GetSlate());
	MetaData->SetTakeNumber(NextTakeNumber);

	UTakeRecorderSources* Sources = NewObject<UTakeRecorderSources>(GetTransientPackage(), NAME_None, RF_Transient);

	for (AActor* ActorToRecord : ActorsToRecord)
	{
		UTakeRecorderActorSource::AddSourceForActor(ActorToRecord, Sources);
	}

	UTakeRecorder* NewRecorder = NewObject<UTakeRecorder>(GetTransientPackage(), NAME_None, RF_Transient);
	if (!NewRecorder->Initialize(RecordLevelSequence, Sources, MetaData, Parameters, &ErrorText))
	{
		FNotificationInfo Info(ErrorText);
		Info.bUseLargeFont = false;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}

void FTakeRecorderSourcesModule::RecordSelectedActors()
{
#if WITH_EDITOR
	TArray<AActor*> SelectedActors;
	GEditor->GetSelectedActors()->GetSelectedObjects(SelectedActors);

	if (SelectedActors.Num() == 0)
	{
		return;
	}

	ULevelSequence* TempLevelSequence = NewObject<ULevelSequence>(GetTransientPackage(), NAME_None, RF_Transient);
	TempLevelSequence->Initialize();

	ULevelSequence* RootLevelSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
	ULevelSequence* LevelSequence = ULevelSequenceEditorBlueprintLibrary::GetFocusedLevelSequence();

	RecordActors(SelectedActors, LevelSequence, RootLevelSequence);
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void FTakeRecorderSourcesModule::OnSequencerCreated(TSharedRef<ISequencer> Sequencer)
{
	bool bSupportsSpawnableActorBinding = false;
	for (const TSubclassOf<UMovieSceneCustomBinding>& SupportedCustomBindingType : Sequencer->GetSupportedCustomBindingTypes())
	{
		if (SupportedCustomBindingType && SupportedCustomBindingType->IsChildOf(UMovieSceneSpawnableActorBinding::StaticClass()))
		{
			bSupportsSpawnableActorBinding = true;
		}
	}

	UTakeRecorderActorSource::SetAllowsSpawnableObjects(bSupportsSpawnableActorBinding);

	Sequencer->OnGetCanRecord().BindLambda([this] (FText& OutInfoText)
	{
		if (UTakeRecorderBlueprintLibrary::IsRecording())
		{
			return true; // can toggle record button to stop 
		}
		else
		{
			// If take recorder panel is open, the record button diverts to there. Any errors should be reported through the tooltip
			if (UTakeRecorderPanel* TakeRecorderPanel = UTakeRecorderBlueprintLibrary::GetTakeRecorderPanel())
			{
				bool bRetVal = TakeRecorderPanel->CanStartRecording(OutInfoText);
				if (!bRetVal && TakeRecorderPanel->GetLevelSequence() && TakeRecorderPanel->GetLevelSequence()->HasAnyFlags(RF_Transient)) 
				{
					OutInfoText = FText::Join(FText::FromString(" "), OutInfoText, LOCTEXT("CloseTakeRecorderToRecordIntoSequence", "If you want to record directly into the current sequence, please close Take Recorder."));
				}

				return bRetVal;
			}
			else
			{
				TArray<AActor*> SelectedActors;
				GEditor->GetSelectedActors()->GetSelectedObjects(SelectedActors);
				
				if (SelectedActors.Num() > 0)
				{
					OutInfoText = LOCTEXT("StartRecordingIntoSequence", "Start recording into the current sequence");
					return true;
				}
				else
				{
					OutInfoText = LOCTEXT("SelectActorsToRecord", "Select actors to record into the current sequence");
				}
			}
		}
		return false;
	} );

	Sequencer->OnGetIsRecording().BindLambda([this] 
	{
		return UTakeRecorderBlueprintLibrary::GetActiveRecorder() || UTakeRecorderBlueprintLibrary::IsRecording();
	} );

	Sequencer->OnRecordEvent().AddLambda([this]
	{
		if (UTakeRecorderBlueprintLibrary::GetActiveRecorder() != nullptr || UTakeRecorderBlueprintLibrary::IsRecording())
		{
			// StopRecording finalizes the take and closes the editor for the recording sequence
			// (and may open the finalized take), which tears down the current sequencer toolkit.
			// Defer for the same reason as the StartRecording branch below: avoid running the
			// teardown while Slate's in-flight FWidgetPath from the Record button click is still
			// holding SSequencer strongly.
			GEditor->GetTimerManager()->SetTimerForNextTick([]()
			{
				UTakeRecorderBlueprintLibrary::StopRecording();
			});
			return;
		}

		// Pop open a dialog asking whether to record
		FSuppressableWarningDialog::FSetupInfo Info(
			LOCTEXT("ShouldRecordPrompt", "Are you sure you want to start recording?"),
			LOCTEXT("ShouldRecordTitle", "Record Actors?"),
			TEXT("RecordActors") );
		Info.ConfirmText = LOCTEXT("ShouldRecord_ConfirmText", "Record");
		Info.CancelText = LOCTEXT("ShouldRecord_CancelText", "Cancel");
		Info.CheckBoxText = LOCTEXT("ShouldRecord_CheckBoxText", "Don't Ask Again");

		FSuppressableWarningDialog ShouldRecordDialog( Info );

		if (ShouldRecordDialog.ShowModal() == FSuppressableWarningDialog::EResult::Cancel)
		{
			return;
		}

		// StartRecording opens a new level sequence, which closes the existing sequencer toolkit
		// and destroys SSequencer. Doing that synchronously from inside the Record button's click
		// handler trips the ensure in FSequencer::DestroySequencerWidget, because Slate's in-flight
		// FWidgetPath is still holding SSequencer strongly until the event-routing call unwinds.
		// Defer to next tick so the routing path has dropped before the teardown runs.
		GEditor->GetTimerManager()->SetTimerForNextTick([this]()
		{
			if (UTakeRecorderPanel* TakeRecorderPanel = UTakeRecorderBlueprintLibrary::GetTakeRecorderPanel())
			{
				FText ErrorText;
				if (TakeRecorderPanel->CanStartRecording(ErrorText))
				{
					ULevelSequence* LevelSequence = TakeRecorderPanel->GetLevelSequence();
					UTakeRecorderSources* Sources = TakeRecorderPanel->GetSources();
					UTakeMetaData* MetaData = TakeRecorderPanel->GetTakeMetaData();

					FTakeRecorderParameters Parameters;
					Parameters.User    = GetDefault<UTakeRecorderUserSettings>()->Settings;
					Parameters.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;
					Parameters.TakeRecorderMode = TakeRecorderPanel->GetMode() == ETakeRecorderPanelMode::RecordingInto ? ETakeRecorderMode::RecordIntoSequence : ETakeRecorderMode::RecordNewSequence;

					UTakeRecorderBlueprintLibrary::StartRecording(LevelSequence, Sources, MetaData, Parameters);
					return;
				}
			}

			RecordSelectedActors();
		});
	});
}
#endif // WITH_EDITOR

UTakeRecorderSource* FTakeRecorderSourcesModule::OnNewActorReferenceBindingAdded(AActor* InActor, ULevelSequence* InLevelSequence)
{	
	if (UTakeRecorderActorSource* ActorSource = TakeRecorderSourcesUtils::StartRecordingActorDuringSession(InActor, InLevelSequence))
	{
		ActorSource->SetIsTemporarySource(true);
		return ActorSource;
	}
	
	return nullptr;
}

IMPLEMENT_MODULE(FTakeRecorderSourcesModule, TakeRecorderSources);
}



#undef LOCTEXT_NAMESPACE