// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorTakeRecorderDropHandler.h"
#include "ITakeRecorderSourcesModule.h"
#include "Misc/CoreMisc.h"
#include "Templates/SharedPointer.h"
#include "Framework/Commands/UICommandList.h"

#if WITH_EDITOR
#include "LevelEditor.h"
#endif // WITH_EDITOR

template<typename OptionalType> struct TOptional;

class FExtender;
class ULevelSequence;
class UTakeRecorderSources;
class UTakeRecorderSource;

#if WITH_EDITOR
class ISequencer;
#endif // WITH_EDITOR

namespace UE::TakeRecorderSources
{
class FTakeRecorderSourcesModule : public ITakeRecorderSourcesModule, private FSelfRegisteringExec
{
public:

	static FTakeRecorderSourcesModule& Get()
	{
		return FModuleManager::GetModuleChecked<FTakeRecorderSourcesModule>(TEXT("TakeRecorderSources"));
	}

	//~ Begin ITakeRecorderSourcesModule Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual void RegisterCanRecordDelegate(FName HandleId, FCanRecordDelegate InDelegate) override;
	virtual void UnregisterCanRecordDelegate(FName HandleId) override;
	//~ End ITakeRecorderSourcesModule Interface

	bool CanRecord(const FCanRecordArgs& InArgs);

private:
	
#if WITH_EDITOR
	FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors LevelEditorMenuExtenderDelegate;
#endif // WITH_EDITOR

	FActorTakeRecorderDropHandler ActorDropHandler;
	FDelegateHandle SourcesMenuExtension;

#if WITH_EDITOR
	FDelegateHandle LevelEditorExtenderDelegateHandle;
#endif // WITH_EDITOR

	FDelegateHandle OnSequencerCreatedHandle;

	TSharedPtr<FUICommandList> CommandList;

	TMap<FName, FCanRecordDelegate> CanRecordDelegates;
	
	void RegisterMenuExtensions();
	void UnregisterMenuExtensions();
	void BindCommands();

	TSharedRef<FExtender> ExtendLevelViewportContextMenu(const TSharedRef<FUICommandList> InCommandList, const TArray<AActor*> SelectedActors);

#if WITH_EDITOR
	static void ExtendSourcesMenu(TSharedRef<FExtender> Extender, UTakeRecorderSources* Sources);
	static void PopulateSourcesMenu(FMenuBuilder& MenuBuilder, UTakeRecorderSources* Sources);
	static void PopulateActorSubMenu(FMenuBuilder& MenuBuilder, UTakeRecorderSources* Sources);
#endif // WITH_EDITOR

	//~ Begin FSelfRegisteringExec Interface
	virtual bool Exec_Editor(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	//~ End FSelfRegisteringExec Interface
	bool HandleRecordTakeCommand(UWorld* InWorld, const TCHAR* InStr, FOutputDevice& Ar);
	bool HandleStopRecordTakeCommand(UWorld* InWorld, const TCHAR* InStr, FOutputDevice& Ar);
	bool HandleCancelRecordTakeCommand(UWorld* InWorld, const TCHAR* InStr, FOutputDevice& Ar);

	void RecordActors(const TArray<AActor*>& ActorsToRecord, TOptional<ULevelSequence*> LevelSequence, TOptional<ULevelSequence*> RootLevelSequence);
	void RecordSelectedActors();

#if WITH_EDITOR
	void OnSequencerCreated(TSharedRef<ISequencer> Sequencer);
#endif // WITH_EDITOR
	
	UTakeRecorderSource* OnNewActorReferenceBindingAdded(AActor* InActor, ULevelSequence* InLevelSequence);
};
}
