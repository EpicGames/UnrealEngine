// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderActorSource.h"
#include "TakeRecorderSources.h"
#include "TakesUtils.h"

#include "TrackRecorders/MovieSceneTrackRecorder.h"
#include "TrackRecorders/MovieSceneAnimationTrackRecorder.h"

#include "MovieSceneSequenceID.h"
#include "LevelSequence.h"

#include "ITakeRecorderSourcesManager.h"

class UWorld;
class ULevelSequence;
class UMovieScene;

namespace TakeRecorderSourcesUtils
{
	/*
	* Get the first PIE world, or first world from the actor sources, or the first world
	*/
	static UWorld* GetSourceWorld(ULevelSequence* InSequence)
	{
		// Get the first PIE world's world settings
		UWorld* World = TakesUtils::GetFirstPIEWorld();

		if (World)
		{
			return World;
		}

		// Otherwise any of the source's worlds
		UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindOrAddSources(InSequence);
		for (auto Source : Sources->GetSources())
		{
			if (Source->IsA<UTakeRecorderActorSource>())
			{
				UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source);
				if (ActorSource->Target.IsValid())
				{
					World = ActorSource->Target.Get()->GetWorld();
					if (World)
					{
						return World;
					}
				}
			}
		}

		// Otherwise, get the first world?
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			World = Context.World();
			if (World)
			{
				return World;
			}
		}

		return nullptr;
	}

	/** 
	* Is the specified actor part of the current recording? This allows us to do some discovery for attachments and hierarchies.
	*
	* @param  Actor	Actor to check
	* @return True if the specified actor is being recorded by another source.
	*/
	static bool IsActorBeingRecorded(const UTakeRecorderSource* InSource, AActor* Actor)
	{
		// If you're tripping this it means you constructed a UTakeRecorderActorSource without using a UTakeRecorderSources to create the instance.
		// cbb: This implementation (and the associated interface) can probably be moved up to UTakeRecorderSources and not on this level.
		UTakeRecorderSources* OwningSources = CastChecked<UTakeRecorderSources>(InSource->GetOuter());
		for (UTakeRecorderSource* Source : OwningSources->GetSources())
		{
			if (UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source))
			{
				if (ActorSource->bEnabled && ActorSource->Target.Get() == Actor)
				{
					return true;
				}
			}
		}

		return false;
	}

	static bool IsActorRecordable(const AActor* Actor)
	{
		return !Actor->GetClass()->HasAnyClassFlags(CLASS_NotPlaceable);
	};

	/**
	* Get the object binding for a given actor that is being recorded. An actor can either be a Possessable or a Spawnable but we only have pointers
	* to the original object being recorded. To solve this, we iterate through each actor being recorded and ask it what Guid it ended up with which
	* ends up abstracting away if it's a Spawnable or a Possessable.
	*
	* @param Actor Actor to look for.
	* @return A valid guid if the actor is being recorded otherwise an invalid guid.
	*/
	static FGuid GetRecordedActorGuid(const UTakeRecorderSource* InSource, class AActor* Actor)
	{
		// If you're tripping this it means you constructed a UTakeRecorderActorSource without using a UTakeRecorderSources to create the instance.
		// cbb: This implementation (and the associated interface) can probably be moved up to UTakeRecorderSources and not on this level.
		UTakeRecorderSources* OwningSources = CastChecked<UTakeRecorderSources>(InSource->GetOuter());
		for (UTakeRecorderSource* Source : OwningSources->GetSources())
		{
			if (UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source))
			{
				AActor* OtherTarget = ActorSource->Target.Get();
				if (OtherTarget == Actor)
				{
					return ActorSource->GetObjectBindingGuid();
				}
			}
		}

		return FGuid();
	}

	/** Returns original transform that may get set when recording a skeletal mesh animation. Needed to correctly transform attached children.
	* @param Actor Actor to look for.
	* @return The FTransform applied to the actor when recording animation, that the attached actor may need to apply to themselves to orient correctly.
	*/
	static FTransform GetRecordedActorAnimationInitialRootTransform(const UTakeRecorderSource* InSource, class AActor* Actor)
	{
		UTakeRecorderSources* OwningSources = CastChecked<UTakeRecorderSources>(InSource->GetOuter());
		for (UTakeRecorderSource* Source : OwningSources->GetSources())
		{
			if (UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source))
			{
				AActor* OtherTarget = ActorSource->Target.Get();
				if (OtherTarget && Actor && (OtherTarget == Actor || OtherTarget->GetName() == Actor->GetName()))
				{
					for (UMovieSceneTrackRecorder* TrackRecorder : ActorSource->TrackRecorders)
					{
						if (TrackRecorder && TrackRecorder->IsA<UMovieSceneAnimationTrackRecorder>())
						{
							return CastChecked<UMovieSceneAnimationTrackRecorder>(TrackRecorder)->GetInitialRootTransform();
						}
					}

				}
			}
		}
		return FTransform::Identity;
	}

	/**
	* Get the sequence of the level sequence the other actor is coming from.
	* Used for setting cross sequence bindings.
	* 
	* @param Actor Actor to look for.
	* @return The Sequence
	*/
	static ULevelSequence* GetLevelSequence(const UTakeRecorderSource* InSource, const AActor* Actor, ULevelSequence* RootLevelSequence)
	{
		check(InSource);
		const UTakeRecorderSources* OwningSources = CastChecked<UTakeRecorderSources>(InSource->GetOuter());

		for (UTakeRecorderSource* Source : OwningSources->GetSources())
		{
			if (UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source))
			{
				AActor* OtherTarget = ActorSource->Target.Get();
				if (OtherTarget && OtherTarget->GetName() == Actor->GetName()) //at the end the target's may have changed.
				{
					if (ActorSource->TargetLevelSequence != RootLevelSequence)
					{
						return ActorSource->TargetLevelSequence;
					}
				}
			}
		}

		return RootLevelSequence;
	}
	
	/**
	* Get the sequence id of the level sequence the other actor is coming from.
	* Used for setting cross sequence bindings.
	* 
	* @param Actor Actor to look for.
	* @return The Sequence ID
	*/
	static FMovieSceneSequenceID GetLevelSequenceID(const UTakeRecorderSource* InSource, class AActor* Actor, ULevelSequence* RootLevelSequence)
	{
		UTakeRecorderSources* OwningSources = CastChecked<UTakeRecorderSources>(InSource->GetOuter());

		for (UTakeRecorderSource* Source : OwningSources->GetSources())
		{
			if (UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source))
			{
				AActor* OtherTarget = ActorSource->Target.Get();
				if (OtherTarget && OtherTarget->GetName() == Actor->GetName()) //at the end the target's may have changed.
				{
					if (ActorSource->TargetLevelSequence != RootLevelSequence)
					{
						return ActorSource->GetSequenceID();
					}
				}
			}
		}

		return MovieSceneSequenceID::Root;
	}
	
	/** Create a new actor source for an actor which can be done during an active recording session. */
	static UTakeRecorderActorSource* StartRecordingActorDuringSession(AActor* InActor, ULevelSequence* InLevelSequence)
	{
		if (!IsValid(InActor) || !IsValid(InLevelSequence))
		{
			return nullptr;
		}
		
		UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindOrAddSources(InLevelSequence);

		UTakeRecorderActorSource* ActorSource = Sources->AddSource<UTakeRecorderActorSource>();
		// Setting the source actor through the function ensures internal property maps are also updated.
		ActorSource->SetSourceActor(InActor);

		// This has to be called after setting the Target and propagating the change event so that it has a chance to know what to record
		// about the actor.
		Sources->StartRecordingSource(TArray<UTakeRecorderSource*>({ ActorSource }), Sources->GetCachedFrameTime());
		return ActorSource;
	}
}