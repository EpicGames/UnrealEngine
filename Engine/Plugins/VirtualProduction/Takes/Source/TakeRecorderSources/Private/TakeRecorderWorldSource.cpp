// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderWorldSource.h"

#include "TakeRecorderSources.h"
#include "TakeRecorderActorSource.h"
#include "TakeRecorderSourcesUtils.h"

#include "LevelSequence.h"
#include "GameFramework/WorldSettings.h"

#include "ITakeRecorderSourcesManager.h"
#include "Engine/Level.h"

#if WITH_EDITOR
#include "ISequencer.h"
#include "ILevelSequenceEditorToolkit.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderWorldSource)

UTakeRecorderWorldSourceSettings::UTakeRecorderWorldSourceSettings(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, bRecordWorldSettings(true)
	, bAutotrackActors(true)
	, bAllowEditorOnlyActors(false)
	, bOnlyRecordMoveableActors(true)
{
}

#if WITH_EDITOR
void UTakeRecorderWorldSourceSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		SaveConfig();
	}
}
#endif // WITH_EDITOR

UTakeRecorderWorldSource::UTakeRecorderWorldSource(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	TrackTint = FColor(129, 129, 129);
}

TArray<UTakeRecorderSource*> UTakeRecorderWorldSource::PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer)
{
	TArray<UTakeRecorderSource*> NewSources;

	UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindOrAddSources(InSequence);

	// Get the first PIE world's world settings
	UWorld* World = TakeRecorderSourcesUtils::GetSourceWorld(InSequence);

	if (!World)
	{
		return NewSources;
	}

	if (bRecordWorldSettings)
	{
		AWorldSettings* WorldSettings = World ? World->GetWorldSettings() : nullptr;

		if (!WorldSettings)
		{
			return NewSources;
		}

		for (auto Source : Sources->GetSources())
		{
			if (Source->IsA<UTakeRecorderActorSource>())
			{
				UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source);
				if (ActorSource->Target.IsValid())
				{
					if (ActorSource->Target.Get() == WorldSettings)
					{
						return NewSources;
					}
				}
			}
		}

		UTakeRecorderActorSource* ActorSource = NewObject<UTakeRecorderActorSource>(Sources, UTakeRecorderActorSource::StaticClass(), NAME_None, RF_Transactional);
		// Setting the source actor through the function ensures internal property maps are also updated.
		ActorSource->SetSourceActor(WorldSettings);

		NewSources.Add(ActorSource);
		WorldSource = ActorSource;
	}

	if (bAutotrackActors)
	{
		AutotrackActors(InSequence, World);
		for (const TWeakObjectPtr<UTakeRecorderActorSource>& Source : AutotrackedSources)
		{
			if (Source.IsValid())
			{
				NewSources.Add(Source.Get());
			}
		}		
	}

	return NewSources;
}

TArray<UTakeRecorderSource*> UTakeRecorderWorldSource::PostRecording(class ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled)
{
	TArray<UTakeRecorderSource*> SourcesToRemove;

	if (WorldSource.IsValid())
	{
		SourcesToRemove.Add(WorldSource.Get());
	}

	for (const TWeakObjectPtr<UTakeRecorderActorSource>& Source : AutotrackedSources)
	{
		if (Source.IsValid())
		{
			SourcesToRemove.Add(Source.Get());
		}
	}

	return SourcesToRemove;
}

FText UTakeRecorderWorldSource::GetDisplayTextImpl() const
{
	return NSLOCTEXT("UTakeRecorderWorldSource", "Label", "World");
}

FText UTakeRecorderWorldSource::GetAddSourceDisplayTextImpl() const
{
    return NSLOCTEXT("UTakeRecorderWorldSource", "TakeRecorderDisplayName", "World Recorder");
}

bool UTakeRecorderWorldSource::CanAddSource(UTakeRecorderSources* InSources) const
{
	for (UTakeRecorderSource* Source : InSources->GetSources())
	{
		if (Source->IsA<UTakeRecorderWorldSource>())
		{
			return false;
		}
	}
	return true;
}

void UTakeRecorderWorldSource::AutotrackActors(class ULevelSequence* InSequence, UWorld* InWorld)
{
	if (!InWorld)
	{
		return;
	}

#if WITH_EDITOR
	if (!GEditor)
	{
		return;
	}

	// To review - this whole chunk seems a little pointless as we dont use it at all other than to check if a ISequencer is valid.
	// This is not though used elsewhere, and is bringing in Editor dependencies. I wonder if we need this at all as we look to move away
	// from ISequencer?
	IAssetEditorInstance*        AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(InSequence, false);
	ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);

	TSharedPtr<ISequencer> SequencerPtr = LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;

	if (!SequencerPtr.IsValid())
	{
		return;
	}
#endif // WITH_EDITOR

	UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindOrAddSources(InSequence);

	TArray<AActor*> ActorsBeingRecorded;
	if (WorldSource.IsValid() && WorldSource.Get()->Target.IsValid())
	{
		ActorsBeingRecorded.Add(WorldSource.Get()->Target.Get());
	}

	if (Sources)
	{
		for (UTakeRecorderSource* Source : Sources->GetSources())
		{
			if (Source->IsA<UTakeRecorderActorSource>())
			{
				UTakeRecorderActorSource* ActorSource = Cast<UTakeRecorderActorSource>(Source);
				if (ActorSource && ActorSource->Target.IsValid())
				{
					ActorsBeingRecorded.Add(ActorSource->Target.Get());
				}
			}
		}
	}

	AutotrackedSources.Reset();
	for (ULevel* Level : InWorld->GetLevels())
	{
		if (Level)
		{
			for (AActor* Actor : Level->Actors)
			{
				if (Actor && !ActorsBeingRecorded.Contains(Actor))
				{
					if (CanRecordActor(Actor))
					{
						UTakeRecorderActorSource* ActorSource = NewObject<UTakeRecorderActorSource>(Sources, UTakeRecorderActorSource::StaticClass(), NAME_None, RF_Transactional);
						// Setting the source actor through the function ensures internal property maps are also updated.
						ActorSource->SetSourceActor(Actor);

						// Add the source to our tracking table.
						AutotrackedSources.Add(ActorSource);
					}
				}				
			}
		}
	}
}

bool UTakeRecorderWorldSource::CanRecordActor(AActor* Actor) const
{
	if (!::IsValid(Actor))
	{
		return false;
	}

	// Skip editor-only actors
	if (!bAllowEditorOnlyActors && Actor->IsEditorOnly())
	{
		return false;
	}

	// Skip actors with no root component or root component not movable
	if (bOnlyRecordMoveableActors)
	{
		if (!Actor->GetRootComponent() || Actor->GetRootComponent()->Mobility == EComponentMobility::Static)
		{
			return false;
		}
	}

	return true;
}
