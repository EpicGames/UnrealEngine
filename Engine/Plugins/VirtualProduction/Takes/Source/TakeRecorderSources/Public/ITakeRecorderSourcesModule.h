// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

namespace UE::TakeRecorderSources
{
struct FCanRecordArgs
{
	TNotNull<UObject*> ObjectToRecord;

	explicit FCanRecordArgs(TNotNull<UObject*> InObjectToRecord) : ObjectToRecord(InObjectToRecord) {}
};
DECLARE_DELEGATE_RetVal_OneParam(bool, FCanRecordDelegate, const FCanRecordArgs&);
	
class ITakeRecorderSourcesModule : public IModuleInterface
{
public:

	static ITakeRecorderSourcesModule& Get()
	{
		return FModuleManager::GetModuleChecked<ITakeRecorderSourcesModule>(TEXT("TakeRecorderSources"));
	}
	
	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("TakeRecorderSources");
	}
	
	/**
	 * When building UTakeRecorderActorSource::RecordedProperties, decides whether an object should be included in the list of recorded objects.
	 * This is a conjunction, i.e. all registered delegates must return true for the object to be recorded; in other words, only one delegate need
	 * return false for the object to not be recorded.
	 */
	virtual void RegisterCanRecordDelegate(FName HandleId, FCanRecordDelegate InDelegate) = 0;
	virtual void UnregisterCanRecordDelegate(FName HandleId) = 0;
};
}
