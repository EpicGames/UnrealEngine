// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderSource.h"
#include "TakeRecorderSourceProperty.h"
#include "Containers/Ticker.h"
#include "UObject/SoftObjectPtr.h"
#include "TrackRecorders/IMovieSceneTrackRecorderHost.h"
#include "Serializers/MovieSceneActorSerialization.h"
#include "Serializers/MovieSceneManifestSerialization.h"
#include "MovieSceneSequenceID.h"
#include "TakeRecorderActorSource.generated.h"

class UActorComponent;
class UMovieSceneFolder;
class UMovieScene;
class UMovieSceneTrackRecorderSettings;
class UMovieSceneAnimationTrackRecorderEditorSettings;
class IMovieSceneTrackRecorderFactory;

#define UE_API TAKERECORDERSOURCES_API

DECLARE_LOG_CATEGORY_EXTERN(ActorSerialization, Verbose, All);

namespace UE::TakesCore
{
class FActorSequenceInformation;
} // namespace UE::TakesCore

UENUM(BlueprintType)
enum class ETakeRecorderActorRecordType : uint8
{
	Possessable,
	Spawnable,
	ProjectDefault
};

/**
* This Take Recorder Source can record an actor from the World's properties.
* Records the properties of the actor and the components on the actor and safely
* handles new components being spawned at runtime and the actor being destroyed.
*/
UCLASS(MinimalAPI, Category="Actors")
class UTakeRecorderActorSource : public UTakeRecorderSource, public IMovieSceneTrackRecorderHost
{
public:
	GENERATED_BODY()

	/** Reference to the actor in the world that should have it's properties recorded. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, DisplayName="Source Actor", Category="Actor Source")
	TSoftObjectPtr<AActor> Target;

	/**
	 * Should this actor be recorded as a Possessable in Sequencer? If so the resulting Object Binding	
	 * will not create a Spawnable copy of this object and instead will possess this object in the level.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Actor Source")
	ETakeRecorderActorRecordType RecordType;

	/**
	 * Should recording this actor as a spawnable overwrite existing spawnables or not.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Actor Source")
	ETakeRecorderSpawnableOverwriteBehavior SpawnableOverwriteBehavior;
	
	UE_DEPRECATED(5.8, "bRecordParentHierarchy has been deprecated. Please use AttachRecordBehaviour instead.")
	UPROPERTY(BlueprintReadWrite, BlueprintGetter = GetRecordParentHierarchy, BlueprintSetter = SetRecordParentHierarchy, Category = "Actor Source", 
		meta = (DeprecatedProperty = "Note", DeprecationMessage = "Please use AttachRecordBehaviour instead."))
	bool bRecordParentHierarchy;

	/**
	 * Control how attachments and hierarchies are recorded. If recording to possessable and the parent is not recorded or controlled by a sequence,
	 * the recorded transforms will be in local space since the child will still be attached to the parent in the level after
	 * recording.  If recording to spawnable and the parent is not recorded or controlled by a sequence, the recorded transforms will be in global space
	 * since the child will not be attached to the parent in the level.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintGetter = GetAttachRecordBehaviour, BlueprintSetter = SetAttachRecordBehaviour, Category = "Actor Source")
	ETakeRecorderAttachRecordBehaviour AttachRecordBehaviour;

	/** Whether to perform key-reduction algorithms as part of the recording */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Actor Source")
	bool bReduceKeys;

	/**
	 * Lists the properties and components on the current actor and whether or not each property will be
	 * recorded into a track in the resulting Level Sequence. 
	 */
	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, meta=(ShowInnerProperties), Category = "Actor Source")
	TObjectPtr<UActorRecorderPropertyMap> RecordedProperties;

	/** Include only the animation bones/curves that match this list */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Actor Source")
	TArray<FString> IncludeAnimationNames;

	/** Exclude all animation bones/curves that match this list */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Actor Source")
	TArray<FString> ExcludeAnimationNames;

	/** The level sequence that this source is being recorded into. Set during PreRecording, null after PostRecording. */
	UPROPERTY()
	TObjectPtr<ULevelSequence> TargetLevelSequence;

	/** The root or uppermost level sequence that this source is being recorded into. Set during PreRecording, null after PostRecording. */
	UPROPERTY()
	TObjectPtr<ULevelSequence> RootLevelSequence;

	/**
	* Dynamically created list of settings objects for the different factories that are recording something 
	* on this actor. If a Factory has no properties it can record the settings objects will not get created.
	* Only one instance of this object exists for a factory and the factory recorder will be passed the shared 
	* instance.
	*/
	UPROPERTY()
	TArray<TObjectPtr<UObject>> FactorySettings;
	
	/**
	* An array of section recorders created during the recording process that are capturing data about the actor/components.
	* Will be an empty list when a recording is not in progress.
	*/
	UPROPERTY()
	TArray<TObjectPtr<class UMovieSceneTrackRecorder>> TrackRecorders;

	/** The parent actor source that generated this actor source (ie. through parenting or as an attached component). Null after PostRecording */
	UPROPERTY()
	TObjectPtr<UTakeRecorderActorSource> ParentSource;

	/** Show Dialog during the (possibly) slow parts of the take recording */
	UPROPERTY()
	bool bShowProgressDialog;

public:

	/*
	 * Add a take recorder source for the given actor. 
	 *
	 * @param InActor The actor to add a source for
	 * @param InSources The sources to add the actor to
	 * @return The added source or the source already present with the same actor
	 */
	UFUNCTION(BlueprintCallable, Category = "Take Recorder")
	static UE_API UTakeRecorderSource* AddSourceForActor(AActor* InActor, UTakeRecorderSources* InSources);

	/*
	 * Remove the given actor from TakeRecorderSources.
	 *
	 * @param InActor The actor to remove from the sources
	 * @param InSources The sources from where to remove the actor
	 */
	UFUNCTION(BlueprintCallable, Category = "Take Recorder")
	static UE_API void RemoveActorFromSources(AActor* InActor, UTakeRecorderSources* InSources);

	/*
	 * External Sequencers can set whether spawnables are allowed for this Take Recorder 
	 */
	static bool AllowsSpawnableObjects();
	static void SetAllowsSpawnableObjects(bool bInAllowsSpawnableObjects);

public:
	UE_API UTakeRecorderActorSource(const FObjectInitializer& ObjInit);
	
	// UTakeRecorderSource Interface
	UE_API virtual void Initialize() override;
	UE_API virtual bool IsValid() const override;
	UE_API virtual TArray<UTakeRecorderSource*> PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer) override;
	UE_API virtual void OnAllSourcesPreRecordingFinished() override;
	UE_API virtual void StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence) override;
	UE_API virtual void TickRecording(const FQualifiedFrameTime& CurrentSequenceTime) override;
	UE_API virtual void StopRecording(class ULevelSequence* InSequence) override;
	UE_API virtual TArray<UTakeRecorderSource*> PostRecording(class ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled) override;
	UE_API virtual void FinalizeRecording() override;
	virtual TArray<UObject*> GetAdditionalSettingsObjects() const { return TArray<UObject*>(FactorySettings); }
	UE_API virtual FString GetSubsceneTrackName(ULevelSequence* InSequence) const override;
	UE_API virtual FString GetSubsceneAssetName(ULevelSequence* InSequence) const override;
	UE_API virtual void AddContentsToFolder(class UMovieSceneFolder* InFolder) override;
	UE_API virtual bool ShouldCreateFolderForContents() const override;
	virtual FMovieSceneSequenceID GetSequenceID() const override { return TargetSequenceID; }
	UE_API virtual const IMovieSceneTrackRecorderHost* AsTrackRecorderHost() const override;
	UE_API virtual bool IsActorSourceForActor(const AActor* Actor) const override;
	UE_API virtual bool OverwriteExistingTrackData() const override;
	UE_API virtual void PreRemove() override;
	// ~UTakeRecorderSource Interface

	/** Set the Target actor that we are going to record. Will reset the Recorded Property Map to defaults. */
	UFUNCTION(BlueprintCallable, Category = "Take Recorder Actor Source")
	UE_API void SetSourceActor(TSoftObjectPtr<AActor> InTarget);

	/** Get the target source actor. */
	UFUNCTION(BlueprintPure, Category = "Take Recorder Actor Source")
	TSoftObjectPtr<AActor> GetSourceActor() const { return Target; }

	/** Get the resolved attach record behaviour. This will resolve from Project Settings if needed. */
	UFUNCTION(BlueprintGetter)
	ETakeRecorderAttachRecordBehaviour GetAttachRecordBehaviour() const { return ResolveAttachRecordBehaviour(); }

	/** Set the current attach record behaviour. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetAttachRecordBehaviour(ETakeRecorderAttachRecordBehaviour InAttachRecordBehaviour);

	/** Get if we are recording the parent hierarchy. True if the resolved attach behaviour is equal to ETakeRecorderAttachRecordBehaviour::Hierarchy. */
	UFUNCTION(BlueprintGetter, meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use AttachRecordBehaviour instead."))
	bool GetRecordParentHierarchy() const { return ResolveAttachRecordBehaviour() == ETakeRecorderAttachRecordBehaviour::Hierarchy; }

	/** Set if the parent hierarchy should be recorded or not. Sets AttachRecordBehaviour. */
	UFUNCTION(BlueprintSetter, meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use AttachRecordBehaviour instead."))
	UE_API void SetRecordParentHierarchy(bool bValue);

	/** Get the Guid of the Object Binding that this Actor Source created in the resulting Level Sequence. */
	FGuid GetObjectBindingGuid() const
	{
		return CachedObjectBindingGuid;
	}

	/** Get the record type. If set to project default, gets the type from the project settings */
	UE_API bool GetRecordToPossessable() const;

	/** If this source should be cleaned up later. */
	void SetIsTemporarySource(bool bValue);
	
protected:

	/**
	* Request that we re-build the map of which properties to record. This should only be called when the target Actor is changed as it will
	* wipe out the users preference for which properties are going to get recorded and default them back to the Project Preferences for that
	* particular Actor/Component.
	*/
	void RebuildRecordedPropertyMap();
	void RebuildRecordedPropertyMapRecursive(const FFieldVariant& InObject, UActorRecorderPropertyMap* PropertyMap, const FString& OuterStructPath = FString());
	
	/**
	* Looks at the given component and determines what the parent of this component is. For the root component and Actor Components the
	* parent will be the root Property Map. For all other components, will attempt to find the Property Map which has a child Property Map
	* that references the given component.
	*
	* Used to find which property map a new property map should be added to while respecting the component hierarchy.
	*/
	UActorRecorderPropertyMap* GetParentPropertyMapForComponent(UActorComponent* InComponent);
	
	/** Looks through the given Property Map recursively to find a property map which references the given component. */
	UActorRecorderPropertyMap* GetPropertyMapForComponentRecursive(UActorComponent* InComponent, UActorRecorderPropertyMap* CurrentPropertyMap);

	/**
	* This is called when recording starts to generate the Section Recorders for the actor and all components that it currently has,
	* as well as again during runtime for any newly added components. This instantiates the TrackRecorders needed, but CreateTrack will
	* not be called until DoCreatePendingTrackRecorders() is called.
	*/
	void CreateSectionRecordersRecursive(UObject* ObjectToRecord, UActorRecorderPropertyMap* PropertyMap, TArray<UObject*>& TraversedObjects);

	/** Update our cached properties for what will be recorded. Done here so the UI doesn't have to iterate through map every frame. */
	void UpdateCachedNumberOfRecordedProperties();

	/** Returns the Guid of the Possessable in the specified sequence that represents the given actor, or an invalid Guid if the actor has no object binding in the sequence. */
	FGuid ResolveActorFromSequence(AActor* InActor, ULevelSequence* CurrentSequence) const;

	/** Remove the Possessable data for the given Guid from the sequence. Calls CleanExistingDataFromSequenceImpl afterwards for any other cleanup you may wish to do. */
	void CleanExistingDataFromSequence(const FGuid& ForGuid, ULevelSequence& InSequence);

	/** Called as part of PostRecording before Track Recorders are finalized. Calls PostProcessTrackRecordersImpl afterwards for any other post processing you wish to do before Track recorders are finalized. */
	void PostProcessTrackRecorders(ULevelSequence* InSequence);

	/** 
	* Ensure that the Object Template this recording is recording into has the specified component. Used to initialize dynamically added components that don't exist in the CDO.
	* @param InComponent - The component on the object that we want to check to see if it's a new component.
	* @param OutComponent - The newly duplicated component for the CDO (if any)
	* @return True if the given component was duplicated and OutComponent is valid.
	*/
	bool EnsureObjectTemplateHasComponent(UActorComponent* InComponent, UActorComponent*& OutComponent);

	/** 
	* Gets all components (both Scene and Actor) on the recorded Actor.
	* @param OutArray - Set of components to add found components to.
	* @param bUpdateReferencedActorList - If true will add any external Actors referenced to the NewReferencedActors list, and possibly print warnings.
	*/
	void GetAllComponents(TSet<TWeakObjectPtr<UActorComponent>>& OutArray, bool bUpdateReferencedActorList = false);

	/**
	* Gets all Scene components that are a child of the specified component. Use the Root Component if you want all
	* child components on an actor.
	* @param OnSceneComponent - The component who's children we look at.
	* @param OutArray - Set of components to add found components to.
	* @param bUpdateReferencedActorList - If true will add any external Actors referenced to the NewReferencedActors list, and possibly print warnings.
	*/
	void GetSceneComponents(USceneComponent* OnSceneComponent, TSet< TWeakObjectPtr<UActorComponent>>& OutArray, bool bUpdateReferencedActorList = false);

	/**
	* Returns the direct children of the specified scene component. Filters for ownership before returning.
	*/
	void GetChildSceneComponents(USceneComponent* OnSceneComponent, TSet< TWeakObjectPtr<UActorComponent>>& OutArray, bool bUpdateReferencedActorList = false);

	/**
	* Gets all Actor components on the recorded actor, not including Scene components.
	*/
	void GetActorComponents(AActor* OnActor, TSet< TWeakObjectPtr<UActorComponent>>& OutArray) const;

	/**
	* Iterates through the NewReferencedActors set and creates a new UTakeRecorderActorSource for each one. Only creates
	* a new source if that actor is not already being recorded by another Actor Source, will not create an Actor source 
	* for the actor we currently target.
	*/
	void CreateNewActorSourceForReferencedActors();
	
	UE_DEPRECATED(5.8, "Use the EnsureParentHierarchyIsReferenced that takes a ULevelSequence as an argument instead.")
	void EnsureParentHierarchyIsReferenced();

	/**
	* Walks the hierarchy of the Target actor and the given sequence to ensure all parents have been added to the NewReferencedActors set.
	* This allows us to ensure that we can record transforms in local space as all parents will be recorded and we can rebuild
	* the hierarchy through Attach Tracks in Sequencer.
	*/
	void EnsureParentHierarchyIsReferenced(ULevelSequence* InLevelSequence);

	// IMovieSceneTrackRecorderHost Interface
	/** Returns true if the other actor is also being recorded by the owning UTakeRecorderSources. Useful for checking if we're recording something we own is attached to. */
	UE_API bool IsOtherActorBeingRecorded(AActor* OtherActor) const override;
	/** Returns a valid guid if the other actor is also being recorded by the owning UTakeRecorderSources. Useful for knowing the guid of that Actor without knowing if it's a Possessable or a Spawnable. */
	UE_API FGuid GetRecordedActorGuid(class AActor* OtherActor) const override;
	/** Returns active level sequence for the given Actor */
	UE_API virtual ULevelSequence* GetLevelSequence(const AActor* OtherActor) override;
	/** Returns id of the active level sequence for the given Actor */
	UE_API virtual FMovieSceneSequenceID GetLevelSequenceID(class AActor* OtherActor) override;
	/** Returns generic track recorder settings */
	UE_API FTrackRecorderSettings GetTrackRecorderSettings() const override;
	/** Returns offset that may get set when recording a skeletal mesh animation. Needed to correctly transform attached children*/
	UE_API FTransform GetRecordedActorAnimationInitialRootTransform(class AActor* OtherActor) const override;
	/** Returns the root level sequence being recorded into */
	ULevelSequence* GetRootLevelSequence() const override { return RootLevelSequence; };
	// ~IMovieSceneTrackRecorderHost Interface
	
	/** Initializes an instance of the specified class if we don't already have it in our Settings array. */
	void InitializeFactorySettingsObject(TSubclassOf<UMovieSceneTrackRecorderSettings> InClass);
	/** Gets the already initialized instance of the specified class from the Settings array. */
	UMovieSceneTrackRecorderSettings* GetSettingsObjectForFactory(TSubclassOf<UMovieSceneTrackRecorderSettings> InClass) const;

	/** This is called after a Spawnable object template is created. Use this to modify any settings on the template object that need to be changed (ie: disabling auto-possession of pawns). */
	UE_API virtual void PostProcessCreatedObjectTemplateImpl(AActor* ObjectTemplate);
	virtual void CleanExistingDataFromSequenceImpl(const FGuid& ForGuid, ULevelSequence& InSequence) {}
	virtual void PostProcessTrackRecordersImpl() {}

public:
	// UObject Interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
#endif // WITH_EDITOR

	UE_API virtual void PostLoad() override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void BeginDestroy() override;
	// ~UObject Interface
private:

	/** Cleanup any attached delegates this source is connected to. */
	void CleanupAttachedDelegates();

#if WITH_EDITOR
	/** Weak pointer to the current attach parent. */
	TWeakObjectPtr<AActor> WeakAttachParent;

	/** Functions for handling when actors are attached or detatched in the level. */
	void HandleActorAttached(AActor* InChild, const AActor* InParent);
	void HandleActorDetached(AActor* InChild, const AActor* InParent);
#endif // WITH_EDITOR

	ETakeRecorderAttachRecordBehaviour ResolveAttachRecordBehaviour() const;
	ETakeRecorderSpawnableOverwriteBehavior ResolveSpawnableOverwriteBehaviour() const;
	ETakeRecorderActorRecordType ResolveRecordType() const;

	bool CheckIfBindingCanBeReused(const UE::TakesCore::FActorSequenceInformation& SequenceInfo) const;

	UE_API virtual const FSlateBrush* GetDisplayIconImpl() const override;
	UE_API virtual FText GetCategoryTextImpl() const;
	UE_API virtual FText GetDisplayTextImpl() const override;
	UE_API virtual FText GetDescriptionTextImpl() const override;
	UE_API virtual FText GetAddSourceDisplayTextImpl() const override;
	
	void ProcessRecordedTimes(ULevelSequence* InSequence);

	/** Return either instance or global animation track recorder editor settings. */
	UMovieSceneAnimationTrackRecorderEditorSettings* GetBestTrackRecorderEditorSettings() const;
	
	/** Create all pending TrackRecorders. */
	void DoCreatePendingTrackRecorders();
	
	/** Attempt to recover the target actor with most current actor. Generally used to repair sequencer spawnable targets. */
	void TryRecoverTarget(const ULevelSequence* InSequenceToSkip = nullptr, const ULevelSequence* InRootSequenceToSkip = nullptr);

	/** Ticker callback: calls TryRecoverTarget while Target is invalid and CachedActorLabel is set. */
	bool TickRecovery(float DeltaTime);

	/** Ticker handle for the editor-only per-tick recovery check. Active while this source is alive. */
	FTSTicker::FDelegateHandle RecoveryTickerHandle;

private:
	/** Object Binding guid that is created in the Level Sequence when recording starts.*/
	FGuid CachedObjectBindingGuid;

	/** Array of actors that have some sort of referenced link to our actor (such as an object attached to our hierarchy) that need Actor Source recorders initialized for them. List is emptied after sources are created. */
	TSet<AActor*> NewReferencedActors;
	/** Array of Actor Sources that we ended up creating that we need to clean up when we stop recording. */
	TArray<UTakeRecorderSource*> AddedActorSources;

	/** 
	* A pointer to the Object Template instance that was created inside the Target Level Sequence at the start of recording.
	* Can be null (if recording to a possessable) or when we are not recording. 
	*/
	TWeakObjectPtr<AActor> CachedObjectTemplate;
	/**
	* A set of components that we're currently recording. We compare the components from one frame to the next to see if any
	* components have been added or removed so we can appropriately update their Spawn tracked. Empty unless a recording is
	* in progress (but may still be empty if no components)
	*/
	TSet<TWeakObjectPtr<UActorComponent>> CachedComponentList;

	/**
	* ID of the TargetLevelSequence.
	*/
	FMovieSceneSequenceID TargetSequenceID;

	/**
	*  Serializer
	*/
	FActorSerializer ActorSerializer;
	
	/** Args for calling CreateTrack on TrackRecorders. */
	struct FCreateTrackArgs
	{
		TWeakObjectPtr<UMovieSceneTrackRecorder> TrackRecorder;
		TWeakObjectPtr<UObject> ObjectToRecord;
		TWeakObjectPtr<UMovieSceneTrackRecorderSettings> FactorySettings;
		FGuid ObjectGuid;
	};
	
	/** TrackRecorders we need to create after PreRecord has completed for all sources. */
	TArray<FCreateTrackArgs> PendingTrackRecorders;
	
	/** If this actor source should be considered temporary, and removed later. */
	bool bIsTemporarySource = false;
	
	/**
	 * True when this source's target is a Sequencer-managed actor (spawnable or possessable
	 * with the SequencerActor tag). Set in SetSourceActor and PreRecording. Serialized so
	 * recording copies and sources loaded from a take retain the flag.
	 */
	UPROPERTY()
	bool bTargetIsSequencerActor = false;

	/**
	 * True when we removed the spawn track from a Take Recorder-created spawnable to prevent a
	 * duplicate actor during recording. The track is re-added in FinalizeRecording.
	 */
	bool bRemovedSpawnTrack = false;
	
	/**
	 * Cached actor label from when Target was last set. Used to recover the target when the soft pointer
	 * becomes invalid. Primarily for when the actor was a Sequencer spawnable instance that was replaced by a new
	 * instance after the recording sequence was opened. Actor label matching is a known fragile approach;
	 * this is planned to be replaced with a more robust solution in a future pass.
	 */
	UPROPERTY()
	FString CachedActorLabel;
	
	/** 
	 * Weak pointer to the original (pre-recording) source object. Recording operates on a DuplicateObject
	 * copy of each source; this lets FinalizeRecording update the original's Target after recording ends.
	 */
	TWeakObjectPtr<UTakeRecorderActorSource> WeakOriginalSource;
};

#undef UE_API
