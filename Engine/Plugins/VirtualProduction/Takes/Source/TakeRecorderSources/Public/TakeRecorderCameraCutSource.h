// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderSource.h"
#include "UObject/SoftObjectPtr.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "MovieSceneSequenceID.h"

#include "TakeRecorderCameraCutSource.generated.h"

class UWorld;
class UTakeRecorderActorSource;

#define UE_API TAKERECORDERSOURCES_API

/** A recording source that detects camera switching and creates a camera cut track */
UCLASS(MinimalAPI, Category="Other")
class UTakeRecorderCameraCutSource : public UTakeRecorderSource
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderCameraCutSource(const FObjectInitializer& ObjInit);

private:

	// UTakeRecorderSource
	UE_API virtual TArray<UTakeRecorderSource*> PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer) override;
	UE_API virtual void TickRecording(const FQualifiedFrameTime& CurrentTime) override;
	UE_API virtual TArray<UTakeRecorderSource*> PostRecording(class ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled) override;
	UE_API virtual FText GetDisplayTextImpl() const override;
	UE_API virtual FText GetAddSourceDisplayTextImpl() const override;		
	// This source does not support subscenes
	virtual bool SupportsSubscenes() const override { return false; }

private:

	UPROPERTY()
	TObjectPtr<UWorld> World;

	/** The root or uppermost level sequence that this source is being recorded into. Set during PreRecording, null after PostRecording. */
	UPROPERTY()
	TObjectPtr<ULevelSequence> RootLevelSequence;


	struct FCameraCutData
	{
		FCameraCutData(FGuid InGuid, FMovieSceneSequenceID InSequenceID, FQualifiedFrameTime InTime) 
		: Guid(InGuid)
		, SequenceID(InSequenceID)
		, Time(InTime) {}

		FGuid Guid;
		FMovieSceneSequenceID SequenceID;
		FQualifiedFrameTime Time;
	};

	TArray<FCameraCutData> CameraCutData;

	/** Spawned actor sources to be removed at the end of recording */
	TArray<TWeakObjectPtr<UTakeRecorderActorSource> > NewActorSources;
};

#undef UE_API