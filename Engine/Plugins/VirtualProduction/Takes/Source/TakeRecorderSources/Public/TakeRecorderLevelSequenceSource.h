// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderSource.h"
#include "UObject/SoftObjectPtr.h"
#include "TakeRecorderLevelSequenceSource.generated.h"

class UTexture;
class ALevelSequenceActor;
class ULevelSequence;

#define UE_API TAKERECORDERSOURCES_API

/** Plays level sequence actors when recording starts */
UCLASS(MinimalAPI,Category="Other")
class UTakeRecorderLevelSequenceSource : public UTakeRecorderSource
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderLevelSequenceSource(const FObjectInitializer& ObjInit);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Source")
	TArray<TObjectPtr<ULevelSequence>> LevelSequencesToTrigger;

private:

	// UTakeRecorderSource
	UE_API virtual TArray<UTakeRecorderSource*> PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer) override;
	UE_API virtual void StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence) override;
	UE_API virtual void StopRecording(class ULevelSequence* InSequence) override;
	UE_API virtual FText GetDisplayTextImpl() const override;
	UE_API virtual FText GetDescriptionTextImpl() const override;
	UE_API virtual FText GetAddSourceDisplayTextImpl() const override;	

	// This source does not support subscenes since it's a playback source instead of a recording
	virtual bool SupportsSubscenes() const override { return false; }

	/** Transient level sequence actors to trigger, to be stopped and reset at the end of recording */
	TArray<TWeakObjectPtr<ALevelSequenceActor>> ActorsToTrigger;
};

#undef UE_API