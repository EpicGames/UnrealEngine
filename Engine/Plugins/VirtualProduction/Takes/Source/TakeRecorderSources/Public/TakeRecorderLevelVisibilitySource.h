// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderSource.h"
#include "TakeRecorderSources.h"
#include "Templates/SubclassOf.h"
#include "TakeRecorderLevelVisibilitySource.generated.h"

#define UE_API TAKERECORDERSOURCES_API

/** A recording source that records level visibilitiy */
UCLASS(MinimalAPI, Abstract, config = EditorSettings, DisplayName = "Level Visibility Recorder Defaults")
class UTakeRecorderLevelVisibilitySourceSettings : public UTakeRecorderSource
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderLevelVisibilitySourceSettings(const FObjectInitializer& ObjInit);

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	// UTakeRecorderSource Interface
	UE_API virtual FString GetSubsceneTrackName(ULevelSequence* InSequence) const override;
	UE_API virtual FString GetSubsceneAssetName(ULevelSequence* InSequence) const override;
	// ~UTakeRecorderSource Interface

	/** Name of the recorded level visibility track name */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Source")
	FText LevelVisibilityTrackName;
};

/** A recording source that records level visibility state */
UCLASS(MinimalAPI, Category="Other", config = EditorSettings)
class UTakeRecorderLevelVisibilitySource : public UTakeRecorderLevelVisibilitySourceSettings
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderLevelVisibilitySource(const FObjectInitializer& ObjInit);

private:

	// UTakeRecorderSource
	UE_API virtual TArray<UTakeRecorderSource*> PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer) override;
	UE_API virtual void TickRecording(const FQualifiedFrameTime& CurrentTime) override;
	UE_API virtual void AddContentsToFolder(class UMovieSceneFolder* InFolder) override;
	UE_API virtual FText GetDisplayTextImpl() const override;
	UE_API virtual FText GetAddSourceDisplayTextImpl() const override;
	UE_API virtual bool CanAddSource(UTakeRecorderSources* InSources) const override;

private:
	TWeakObjectPtr<class UMovieSceneLevelVisibilityTrack> CachedLevelVisibilityTrack;
};

#undef UE_API