// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderSourceProperty.h"

#include "TakeRecorderMicrophoneAudioManager.generated.h"

#define UE_API TAKERECORDERSOURCES_API

#if WITH_EDITOR
class IAudioCaptureEditor;
#endif // WITH_EDITOR

class USoundWave;
struct FTakeRecorderAudioSourceSettings;


DECLARE_MULTICAST_DELEGATE_OneParam(FOnNotifySourcesOfDeviceChange, int);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnAudioInputDevicePropertyChanged, const FAudioInputDeviceProperty&, InAudioInputDevice);

/** This class exposes the audio input device list via the project settings details. It does this in 
*   conjunction with FAudioInputDevicePropertyCustomization. It also manages the IAudioCaptureEditor 
*   object which handles the low level audio device recording.
*/
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, PerObjectConfig, DisplayName = "Audio Input Device")
class UTakeRecorderMicrophoneAudioManager : public UTakeRecorderAudioInputSettings
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderMicrophoneAudioManager(const FObjectInitializer& ObjInit);

	// Begin UObject Interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	// End UObject Interface

	/** Enumerates the audio devices present on the current machine */
	UFUNCTION(BlueprintCallable, Category = "Take Recorder|Microphone Audio")
	UE_API virtual void EnumerateAudioDevices(bool InForceRefresh = false) override;

	/** Returns input channel count for currently selected audio device */
	UFUNCTION(BlueprintPure, Category = "Take Recorder|Microphone Audio")
	UE_API virtual int32 GetDeviceChannelCount() override;

	/** The audio device to use for this microphone source */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, BlueprintGetter = GetAudioInputDevice, BlueprintSetter = SetAudioInputDevice,
	          Category = "Take Recorder|Microphone Audio", meta = (ShowOnlyInnerProperties))
	FAudioInputDeviceProperty AudioInputDevice;

	/** Returns the currently configured audio input device settings (device id, buffer size, default-device flag, cached device list). */
	UFUNCTION(BlueprintPure, Category = "Take Recorder|Microphone Audio")
	UE_API FAudioInputDeviceProperty GetAudioInputDevice() const;

	/**
	 * Sets the audio input device settings and notifies dependents — broadcasts OnNotifySourcesOfDeviceChange (so
	 * every microphone source re-evaluates its channel) and OnAudioInputDeviceChanged (so UI customizations refresh).
	 * Saves config when invoked on the CDO (the canonical instance returned by GetMutableDefault<>).
	 */
	UFUNCTION(BlueprintCallable, Category = "Take Recorder|Microphone Audio")
	UE_API void SetAudioInputDevice(const FAudioInputDeviceProperty& InAudioInputDevice);

	/** 
	*  Calls StartRecording on the AudioRecorder object. This is called multiple times (once for each
	*  microphone source, however, only the first call triggers the call to AudioRecorder.
	*/
	UE_API void StartRecording(int32 InChannelCount);
	/**
	 * Calls StopRecording on the AudioRecorder object. This is called multiple times (once for each
	 * microphone source, however, only the first call triggers the call to AudioRecorder.
	 */
	UE_API void StopRecording();
	/**
	 *  Calls FinalizeRecording on the AudioRecorder object. This is called multiple times (once for each
	 *  microphone source, however, only the first call triggers the call to AudioRecorder.
	 */
	UE_API void FinalizeRecording();

#if WITH_EDITOR
	/** Fetches the USoundWave for this source after a Take has been recorded */
	UE_API TObjectPtr<class USoundWave> GetRecordedSoundWave(const FTakeRecorderAudioSourceSettings& InSourceSettings);
#endif // WITH_EDITOR

	/** Accessor for the OnNotifySourcesOfDeviceChange delegate list */
	FOnNotifySourcesOfDeviceChange& GetOnNotifySourcesOfDeviceChange() { return OnNotifySourcesOfDeviceChange; }

	/** Fires after AudioInputDevice is modified — via editor edit or SetAudioInputDevice. */
	UPROPERTY(BlueprintAssignable, Category = "Take Recorder|Microphone Audio")
	FOnAudioInputDevicePropertyChanged OnAudioInputDevicePropertyChanged;

private:

	/** Multicast delegate which notifies clients when the currently selected audio device changes. */
	FOnNotifySourcesOfDeviceChange OnNotifySourcesOfDeviceChange;

#if WITH_EDITOR
	/** Calls factory to create the AudioRecorder object */
	UE_API TUniquePtr<IAudioCaptureEditor> CreateAudioRecorderObject();
#endif // WITH_EDITOR

	/** Builds the list of audio devices which will be used in the device menu */
	UE_API void BuildDeviceInfoArray();

	/** Returns whether audio device with given Id was found during enumeration */
	UE_API bool IsAudioDeviceAvailable(const FString& InDeviceId);

#if WITH_EDITOR
	/** The audio recorder object which manages low level recording of audio data */
	TUniquePtr<IAudioCaptureEditor> AudioRecorder;
#endif // WITH_EDITOR
};

#undef UE_API
