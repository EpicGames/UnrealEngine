// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderSource.h"
#include "TakeRecorderSources.h"
#include "TakeRecorderSourceProperty.h"

#include "GameFramework/Actor.h"
#include "Templates/SubclassOf.h"
#include "UObject/SoftObjectPtr.h"

#include "TakeRecorderMicrophoneAudioSource.generated.h"

#define UE_API TAKERECORDERSOURCES_API

class UMovieSceneAudioTrack;
class USoundWave;
class UTakeRecorderMicrophoneAudioManager;
struct FTakeRecorderAudioSourceSettings;

/** A recording source that records microphone audio */
UCLASS(MinimalAPI, Abstract, config=EditorSettings, DisplayName="Microphone Audio Recorder")
class UTakeRecorderMicrophoneAudioSourceSettings : public UTakeRecorderSource
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderMicrophoneAudioSourceSettings(const FObjectInitializer& ObjInit);

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	
	// UTakeRecorderSource Interface
	UE_API virtual FString GetSubsceneTrackName(ULevelSequence* InSequence) const override;
	UE_API virtual FString GetSubsceneAssetName(ULevelSequence* InSequence) const override;
	// ~UTakeRecorderSource Interface

	/** Name of the audio source */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Source")
	FText AudioSourceName;

	/** Name of the recorded audio track */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Source", meta = (NamingTokens))
	FText AudioTrackName;

	/** The name of the audio asset.
	 * Supports any of the following format specifiers that will be substituted when a take is recorded :
	 * {day} - The day of the timestamp for the start of the recording.
	 * {month} - The month of the timestamp for the start of the recording.
	 * {year} - The year of the timestamp for the start of the recording.
	 * {hour} - The hour of the timestamp for the start of the recording.
	 * {minute} - The minute of the timestamp for the start of the recording.
	 * {second} - The second of the timestamp for the start of the recording.
	 * {take} - The take number.
	 * {slate} - The slate string.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Source", meta = (NamingTokens))
	FString AudioAssetName;

	/** The name of the subdirectory audio will be placed in. Leave this empty to place into the same directory as the sequence base path 
	 * Supports any of the following format specifiers that will be substituted when a take is recorded :
	 * {day} - The day of the timestamp for the start of the recording.
	 * {month} - The month of the timestamp for the start of the recording.
	 * {year} - The year of the timestamp for the start of the recording.
	 * {hour} - The hour of the timestamp for the start of the recording.
	 * {minute} - The minute of the timestamp for the start of the recording.
	 * {second} - The second of the timestamp for the start of the recording.
	 * {take} - The take number.
	 * {slate} - The slate string.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Source", meta = (NamingTokens))
	FString AudioSubDirectory;
};

/** A recording source that records microphone audio */
UCLASS(MinimalAPI, Category="Audio", config=EditorSettings)
class UTakeRecorderMicrophoneAudioSource : public UTakeRecorderMicrophoneAudioSourceSettings
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderMicrophoneAudioSource(const FObjectInitializer& ObjInit);

#if WITH_EDITOR
	// Begin UObject Interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End UObject Interface
#endif // WITH_EDITOR

	/** Gain in decibels to apply to recorded audio */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Source", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "20.0", ClampMax = "40.0"))
	float AudioGain;

	/** Whether or not to split mic channels into separate audio tracks. If not true, a max of 2 input channels is supported. */
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "SplitAudioChannelsIntoSeparateTracks is deprecated."))
	bool bSplitAudioChannelsIntoSeparateTracks_DEPRECATED;
#endif

	/** Replace existing recorded audio with any newly recorded audio */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Source")
	bool bReplaceRecordedAudio;

	/** The audio device to use for this microphone source */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Source", meta = (ShowOnlyInnerProperties))
	FAudioInputDeviceChannelProperty AudioChannel;

	// UTakeRecorderSource
	UE_API virtual void Initialize() override;
	UE_API virtual FString GetSubsceneTrackName(ULevelSequence* InSequence) const override;
	UE_API virtual FString GetSubsceneAssetName(ULevelSequence* InSequence) const override;
	UE_API virtual TArray<UTakeRecorderSource*> PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer) override;
	UE_API virtual void StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence) override;
	UE_API virtual void TickRecording(const FQualifiedFrameTime& CurrentTime) override;
	UE_API virtual void StopRecording(class ULevelSequence* InSequence) override;
	UE_API virtual TArray<UTakeRecorderSource*> PostRecording(ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled) override;
	UE_API virtual void FinalizeRecording() override;
	UE_API virtual FText GetDisplayTextImpl() const override;
	UE_API virtual FText GetAddSourceDisplayTextImpl() const override;

	UE_API virtual void AddContentsToFolder(class UMovieSceneFolder* InFolder) override;
	UE_API virtual bool CanAddSource(UTakeRecorderSources* InSources) const override;
	// ~UTakeRecorderSource

	/** Returns track display name, replacing any tokens if needed */
	UE_API FString GetAudioTrackName(ULevelSequence* InSequence) const;
	/** Returns the fully expanded asset name */
	UE_API FString GetAudioAssetName(ULevelSequence* InSequence) const;

	/** Sets the channel count supported by the currently selected audio device. */
	UE_API void SetAudioDeviceChannelCount(int32 InChannelCount);
	/** Delegate which receives notifications when the audio input device changes. */
	UE_API void OnNotifySourcesOfDeviceChange(int32 InChannelCount);

private:

	/** Helper function for getting a pointer to the AudioInputManger object. */
	static UE_API UTakeRecorderMicrophoneAudioManager* GetAudioInputManager();
	
	/** Parses recorded timecode values */
	UE_API void ProcessRecordedTimes(ULevelSequence* InSequence);

	/** Returns an array of booleans indicating which channel indexes are currently in use */
	UE_API TUniquePtr<TArray<bool>> GetChannelsInUse(const int32 InDeviceChannelCount);
	/** Fetches the USoundWave for this source after a Take has been recorded */
	UE_API void GetRecordedSoundWave(ULevelSequence* InSequence);
	/** Called when the user selects a new channel in the combobox for this source */
	UE_API void SetCurrentInputChannel(int32 InChannelNumber);

	/** Returns whether given track is being used by any other source */
	UE_API bool IsTrackAssociatedWithAnySource(UMovieSceneAudioTrack* InAudioTrack);

	/** Build asset name, appending channel name if needed for uniqueness */
	UE_API FString GetUniqueAudioAssetName(ULevelSequence* InSequence) const;

	/** Static, parameterized helper function for buidling unique asset names */	
	static UE_API FString CreateUniqueAudioAssetName(ULevelSequence* InSequence, UTakeRecorderSources* InSources, const FString& InAssetName, const int32 InChannelNumber);

	/** Evaluate our tokens and automatically create an appropriate context. */
	UE_API bool TryEvaluateTokenString(const FString& InString, const ULevelSequence* InSequence, FString& OutString) const;

private:
	void DoAudioSetup();
		
	// Holds the Sequencer audio track for this source
	TWeakObjectPtr<class UMovieSceneAudioTrack> CachedAudioTrack;

	// Holds the USoundWave asset for a given take
	TWeakObjectPtr<USoundWave> RecordedSoundWave;
	// Caches the starting timecode for this take so it can be referenced when creating USoundWave assets
	FTimecode StartTimecode;

	// The user specified directory to store recorded audio assets in
	FDirectoryPath AudioDirectory;
	// The name of the UsoundWave asset
	FString AssetFileName;

	// Have we initialized our source.
	bool bHaveDoneAudioSetup = false;
};

#undef UE_API
