// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderSource.h"
#include "UObject/SoftObjectPtr.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "TakeRecorderNearbySpawnedActorSource.generated.h"

class UTexture;
class UTakeRecorderActorSource;

#define UE_API TAKERECORDERSOURCES_API

/** A recording source that detects actors spawned close to the current camera, and captures them as spawnables */
UCLASS(MinimalAPI, Category="Actors")
class UTakeRecorderNearbySpawnedActorSource : public UTakeRecorderSource
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderNearbySpawnedActorSource(const FObjectInitializer& ObjInit);

	/** The proximity to the current camera that an actor must be spawned in order to be recorded as a spawnable. If 0, proximity is disregarded. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, DisplayName="Spawn Proximity", Category="Source", meta=(units=cm))
	float Proximity;

	/** Should we only record actors that pass the filter list?*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Source")
	bool bFilterSpawnedActors;
	
	/** A type filter to apply to spawned objects */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Source", meta=(EditCondition="bFilterSpawnedActors"))
	TArray<TSubclassOf<AActor> > FilterTypes;

private:

	// UTakeRecorderSource
	UE_API virtual TArray<UTakeRecorderSource*> PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer) override;
	UE_API virtual TArray<UTakeRecorderSource*> PostRecording(class ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled) override;
	UE_API virtual FText GetDisplayTextImpl() const override;
	UE_API virtual FText GetDescriptionTextImpl() const override;
	UE_API virtual FText GetAddSourceDisplayTextImpl() const override;
	// This source does not support subscenes (ie. "NearbySpawnedActors subscene"), but each of the spawned actors would be placed in subscenes if the option is enabled
	virtual bool SupportsSubscenes() const override { return false; }

	/** Handle actors being spawned */
	void HandleActorSpawned(AActor* Actor, class ULevelSequence* InSequence);
	
	/** Is this actor valid for recording? Is it close enough? Is it a filtered type? */
	bool IsActorValid(AActor* Actor);

private:

	/** Delegate handles for FOnActorSpawned events */
	TMap<TWeakObjectPtr<UWorld>, FDelegateHandle> ActorSpawningDelegateHandles;

	/** Spawned actor sources to be removed at the end of recording */
	TArray<TWeakObjectPtr<UTakeRecorderActorSource> > SpawnedActorSources;
};

#undef UE_API