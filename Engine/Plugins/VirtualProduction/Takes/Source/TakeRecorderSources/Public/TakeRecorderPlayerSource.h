// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderSource.h"
#include "TakeRecorderSources.h"
#include "Templates/SubclassOf.h"
#include "TakeRecorderPlayerSource.generated.h"

class UTakeRecorderActorSource;

#define UE_API TAKERECORDERSOURCES_API

/** A recording source that records the current player */
UCLASS(MinimalAPI, Category="Actors")
class UTakeRecorderPlayerSource : public UTakeRecorderSource
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderPlayerSource(const FObjectInitializer& ObjInit);

private:

	// UTakeRecorderSource
	UE_API virtual TArray<UTakeRecorderSource*> PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer) override;
	UE_API virtual TArray<UTakeRecorderSource*> PostRecording(class ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled) override;
	UE_API virtual FText GetDisplayTextImpl() const override;
	UE_API virtual FText GetAddSourceDisplayTextImpl() const override;				
	UE_API virtual bool CanAddSource(UTakeRecorderSources* InSources) const override;

	// This source does not support subscenes (ie. "Player subscene"), but the player would be placed in subscenes if the option is enabled
	virtual bool SupportsSubscenes() const override { return false; }

	TWeakObjectPtr<UTakeRecorderActorSource> PlayerActorSource;
};

#undef UE_API