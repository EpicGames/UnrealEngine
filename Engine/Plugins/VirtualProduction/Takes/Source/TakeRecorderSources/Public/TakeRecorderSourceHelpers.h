// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "TakeRecorderSources.h"
#include "GameFramework/Actor.h"

class AActor;
class UMovieSceneTakeTrack;


namespace TakeRecorderSourceHelpers
{
	/**
	 * Adds a number of actors to the set of sources to record from.
	 * 
	 * @param TakeRecorderSources The list of sources used for the current take. 
	 * @param ActorsToRecord The list of Actors that should be added to Sources. Note that this can include ALevelSequenceActors.
	 * @param bReduceKeys Enable/disable key reduction on all the sources registered
	 * @param bShowProgress Enable/disable the dialog box showing progress for the potentially slow parts of finalizing the take
	 */
	TAKERECORDERSOURCES_API void AddActorSources(
		UTakeRecorderSources*     TakeRecorderSources,
		TArrayView<AActor* const> ActorsToRecord,
		bool                      bReduceKeys = true,
		bool                      bShowProgress = true);

	/**
	 * Remove specific actors as a source.
	 * @param TakeRecorderSources The list of sources used for the current take. 
	 * @param ActorsToRemove The list of Actors to be removed from Sources.
	 */
	TAKERECORDERSOURCES_API void RemoveActorSources(UTakeRecorderSources* TakeRecorderSources, TArrayView<AActor* const> ActorsToRemove);
	
	/**
	 * Removes all sources from a list of sources to record from.
	 */
	TAKERECORDERSOURCES_API void RemoveAllActorSources(UTakeRecorderSources* Sources);

	/**
	 * Retrieve the source actor if applicable.
	 * @param Source The source to check.
	 * @return The source actor, if one is set.
	 */
	TAKERECORDERSOURCES_API AActor* GetSourceActor(UTakeRecorderSource* Source);

	using FArrayOfRecordedTimePairs = const TArray<TPair<FQualifiedFrameTime, FQualifiedFrameTime>>;

	/**
	 * Creates a takes track to store timecode data on a take recorder source.
	 */
	UE_DEPRECATED(5.7, "Use UE::TakeCores::ProcessRecordedTimes instead.")
	TAKERECORDERSOURCES_API void ProcessRecordedTimes(ULevelSequence* InSequence, UMovieSceneTakeTrack* TakeTrack, const TOptional<TRange<FFrameNumber>>& FrameRange, const FArrayOfRecordedTimePairs& RecordedTimes);
};
