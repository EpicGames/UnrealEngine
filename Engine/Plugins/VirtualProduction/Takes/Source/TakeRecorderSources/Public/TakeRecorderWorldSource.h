// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderSource.h"
#include "TakeRecorderSources.h"
#include "UObject/SoftObjectPtr.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "TakeRecorderWorldSource.generated.h"

class UTexture;
class UTakeRecorderActorSource;

#define UE_API TAKERECORDERSOURCES_API

/** A recording source that records world state */
UCLASS(Abstract, MinimalAPI, config = EditorSettings, DisplayName = "World Recorder")
class UTakeRecorderWorldSourceSettings : public UTakeRecorderSource
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderWorldSourceSettings(const FObjectInitializer& ObjInit);

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	/** Record world settings */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Source")
	bool bRecordWorldSettings;

	/** Add a binding and track for all actors that aren't explicitly being recorded */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Source")
	bool bAutotrackActors;

	/** When autotracking actors to add, allow recording of editor only actor objects. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Source")
	bool bAllowEditorOnlyActors;

	/** When autotracking actors to add, only record actors that have a root component and are moveable. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Source")
	bool bOnlyRecordMoveableActors;
};


/** A recording source that records world state */
UCLASS(MinimalAPI, Category="Actors")
class UTakeRecorderWorldSource : public UTakeRecorderWorldSourceSettings
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderWorldSource(const FObjectInitializer& ObjInit);

private:

	// UTakeRecorderSource
	UE_API virtual TArray<UTakeRecorderSource*> PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer) override;
	UE_API virtual TArray<UTakeRecorderSource*> PostRecording(class ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled) override;
	virtual bool SupportsTakeNumber() const override { return false; }
	UE_API virtual FText GetDisplayTextImpl() const override;
	UE_API virtual FText GetAddSourceDisplayTextImpl() const override;		
	UE_API virtual bool CanAddSource(UTakeRecorderSources* InSources) const override;

	// This source does not support subscenes (ie. "World Settings subscene"), but the world settings actor would be placed in subscenes if the option is enabled
	virtual bool SupportsSubscenes() const override { return false; }

private:

	/**
	 * Autotrack actors in the world that aren't already being recorded
	 */
	void AutotrackActors(class ULevelSequence* InSequence, UWorld* InWorld);

	/**
	 * If Autotracked actors is enabled this helper will determine if the actor can be recorded.
	 */
	bool CanRecordActor(AActor* Actor) const;

	TWeakObjectPtr<UTakeRecorderActorSource> WorldSource;
	TSet<TWeakObjectPtr<UTakeRecorderActorSource>> AutotrackedSources;
};

#undef UE_API
