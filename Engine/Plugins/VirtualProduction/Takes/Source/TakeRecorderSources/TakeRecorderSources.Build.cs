// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TakeRecorderSources : ModuleRules
{
	public TakeRecorderSources(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "CinematicCamera",
                "Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"LevelSequence",
                "MovieScene",
				"MovieSceneTracks",
				"SerializedRecorderInterface",
                "Slate",
				"SlateCore",
				"TakesCore",
				"TakeRecorder",
				"TakeMovieScene"
			}
		);

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "TakeTrackRecorders", 
                "Core",
            }
        );

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"EditorFramework",
					"AudioCaptureEditor",
					"LevelEditor",
					"LevelSequenceEditor",
					"SceneOutliner",
					"SequenceRecorder", // For FTimecodeBoneMethod
					"UnrealEd",
					"Sequencer",
				}
			);
		}
	}
}
