// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeSequencerModule.h"
#include "Modules/ModuleManager.h"
#include "TakeTrackEditor.h"
#include "ISequencerModule.h"

IMPLEMENT_MODULE(FTakeSequencerModule, TakeSequencer)

void FTakeSequencerModule::StartupModule()
{
	// Register with the sequencer module
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	TakeTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FTakeTrackEditor::CreateTrackEditor));
}

void FTakeSequencerModule::ShutdownModule()
{
	// Unregister sequencer track creation delegates
	ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>( "Sequencer" );
	if ( SequencerModule != nullptr )
	{
		SequencerModule->UnRegisterTrackEditor( TakeTrackEditorHandle );
	}
}
