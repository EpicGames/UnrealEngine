// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeTrackEditor.h"

#include "Styling/AppStyle.h"
#include "SequencerSectionPainter.h"

#include "MovieSceneTakeSection.h"
#include "MovieSceneTakeTrack.h"
#include "MovieSceneTakeSettings.h"

#define LOCTEXT_NAMESPACE "FTakeTrackEditor"

FTakeSection::FTakeSection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
	: Section(*CastChecked<UMovieSceneTakeSection>(&InSection))
{ 
}


UMovieSceneSection* FTakeSection::GetSectionObject()
{
	return &Section;
}

FText FTakeSection::GetSectionToolTip() const
{
	if (Section.Slate.GetDefault().IsSet() && !Section.Slate.GetDefault().GetValue().IsEmpty())
	{
		return FText::FromString(TEXT("Take (") + Section.Slate.GetDefault().GetValue() + TEXT(")"));
	}
	return FText::GetEmpty();
}

int32 FTakeSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	int32 LayerId = Painter.PaintSectionBackground();

	return LayerId;
}


TSharedRef<ISequencerTrackEditor> FTakeTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> InSequencer )
{
	return MakeShareable( new FTakeTrackEditor( InSequencer ) );
}

bool FTakeTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneTakeTrack::StaticClass();
}

FText FTakeTrackEditor::GetDisplayName() const
{
	return LOCTEXT("TakeTrackEditor_DisplayName", "Take");
}

TSharedRef<ISequencerSection> FTakeTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));

	return MakeShareable(new FTakeSection(SectionObject, GetSequencer()));
}


const FSlateBrush* FTakeTrackEditor::GetIconBrush() const
{
	return FAppStyle::GetBrush("Sequencer.GenericDivider");
}

	
bool FTakeTrackEditor::ImportAnimatedProperty(const FString& InPropertyName, const FRichCurve& InCurve, FGuid InBinding, UMovieScene* InMovieScene)
{
	// check whether this property is something we can import
	if (InPropertyName != GetDefault<UMovieSceneTakeSettings>()->HoursName && 
		InPropertyName != GetDefault<UMovieSceneTakeSettings>()->MinutesName && 
		InPropertyName != GetDefault<UMovieSceneTakeSettings>()->SecondsName && 
		InPropertyName != GetDefault<UMovieSceneTakeSettings>()->FramesName && 
		InPropertyName != GetDefault<UMovieSceneTakeSettings>()->SubFramesName &&
		InPropertyName != GetDefault<UMovieSceneTakeSettings>()->RateName)
	{
		return false;
	}

	UMovieSceneTakeTrack* TakeTrack = InMovieScene->FindTrack<UMovieSceneTakeTrack>(InBinding);
	if (!TakeTrack)
	{
		InMovieScene->Modify();
		TakeTrack = InMovieScene->AddTrack<UMovieSceneTakeTrack>(InBinding);
	}
	check(TakeTrack);

	UMovieSceneTakeSection* TakeSection = nullptr;
	if (TakeTrack->GetAllSections().Num() == 0)
	{
		TakeSection = Cast<UMovieSceneTakeSection>(TakeTrack->CreateNewSection());
		TakeTrack->Modify();
		TakeTrack->AddSection(*TakeSection);
	}
	else
	{
		TakeSection = Cast<UMovieSceneTakeSection>(TakeTrack->GetAllSections()[0]);
	}
	check(TakeSection);

	FFrameRate FrameRate = InMovieScene->GetTickResolution();
	TArray<FFrameNumber> KeyTimes;
	TArray<int> KeyIntegerValues;
	TArray<FMovieSceneFloatValue> KeyFloatValues;
	for (TArray<FKeyHandle>::TConstIterator KeyHandle = InCurve.GetKeyHandleIterator(); KeyHandle; ++KeyHandle)
	{
		const FRichCurveKey &Key = InCurve.GetKey(*KeyHandle);
		FFrameNumber KeyTime = (Key.Time * FrameRate).RoundToFrame();
		FMovieSceneFloatValue KeyValue(Key.Value);
		KeyTimes.Add(KeyTime);
		KeyIntegerValues.Add((int)Key.Value);
		KeyFloatValues.Add(KeyValue);
	}

	TakeSection->Modify();

	if (InPropertyName == GetDefault<UMovieSceneTakeSettings>()->HoursName)
	{
		TakeSection->HoursCurve.Set(KeyTimes, KeyIntegerValues);
	}
	else if (InPropertyName == GetDefault<UMovieSceneTakeSettings>()->MinutesName)
	{
		TakeSection->MinutesCurve.Set(KeyTimes, KeyIntegerValues);
	}
	else if (InPropertyName == GetDefault<UMovieSceneTakeSettings>()->SecondsName)
	{
		TakeSection->SecondsCurve.Set(KeyTimes, KeyIntegerValues);
	}
	else if (InPropertyName == GetDefault<UMovieSceneTakeSettings>()->FramesName)
	{
		TakeSection->FramesCurve.Set(KeyTimes, KeyIntegerValues);
	}
	else if (InPropertyName == GetDefault<UMovieSceneTakeSettings>()->SubFramesName)
	{
		TakeSection->SubFramesCurve.Set(KeyTimes, KeyFloatValues);
	}
	else if (InPropertyName == GetDefault<UMovieSceneTakeSettings>()->RateName)
	{
		TakeSection->RateCurve.Set(KeyTimes, KeyFloatValues);
	}

	TOptional<TRange<FFrameNumber> > AutoSizeRange = TakeSection->GetAutoSizeRange();
	if (AutoSizeRange.IsSet())
	{
		TakeSection->SetRange(AutoSizeRange.GetValue());
	}

	return true;
}

bool FTakeTrackEditor::ImportStringProperty(const FString& InPropertyName, const FString& InStringValue, FGuid InBinding, UMovieScene* InMovieScene)
{
	// check whether this property is something we can import
	if (InPropertyName != GetDefault<UMovieSceneTakeSettings>()->SlateName)
	{
		return false;
	}

	UMovieSceneTakeTrack* TakeTrack = InMovieScene->FindTrack<UMovieSceneTakeTrack>(InBinding);
	if (!TakeTrack)
	{
		InMovieScene->Modify();
		TakeTrack = InMovieScene->AddTrack<UMovieSceneTakeTrack>(InBinding);
	}
	check(TakeTrack);

	UMovieSceneTakeSection* TakeSection = nullptr;
	if (TakeTrack->GetAllSections().Num() == 0)
	{
		TakeSection = Cast<UMovieSceneTakeSection>(TakeTrack->CreateNewSection());
		TakeTrack->Modify();
		TakeTrack->AddSection(*TakeSection);
	}
	else
	{
		TakeSection = Cast<UMovieSceneTakeSection>(TakeTrack->GetAllSections()[0]);
	}
	check(TakeSection);

	TakeSection->Modify();
	TakeSection->Slate.SetDefault(InStringValue);

	return true;
}


#undef LOCTEXT_NAMESPACE
