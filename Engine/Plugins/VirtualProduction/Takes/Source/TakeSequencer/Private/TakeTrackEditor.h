// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "IMovieSceneToolsTrackImporter.h"
#include "MovieSceneToolsModule.h"
#include "MovieSceneTakeTrack.h"
#include "MovieSceneTakeSection.h"
#include "MovieSceneTrackEditor.h"

class FTakeTrackEditor : public FMovieSceneTrackEditor, public IMovieSceneToolsTrackImporter
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	FTakeTrackEditor( TSharedRef<ISequencer> InSequencer )
		: FMovieSceneTrackEditor( InSequencer)
	{
		FMovieSceneToolsModule::Get().RegisterTrackImporter(this);
	}

	virtual ~FTakeTrackEditor() 
	{ 		
		FMovieSceneToolsModule::Get().UnregisterTrackImporter(this);
	}

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );


public:
		
	// ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding ) override;
	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;
	virtual const FSlateBrush* GetIconBrush() const override;

	// IMovieSceneToolsTrackImporter
	virtual bool ImportAnimatedProperty(const FString& InPropertyName, const FRichCurve& InCurve, FGuid InBinding, UMovieScene* InMovieScene) override;
	virtual bool ImportStringProperty(const FString& InPropertyName, const FString& InStringValue, FGuid InBinding, UMovieScene* InMovieScene) override;
};


/** Class for take sections */
class FTakeSection
	: public ISequencerSection
	, public TSharedFromThis<FTakeSection>
{
public:

	/** Constructor. */
	FTakeSection( UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FTakeSection() = default;

public:

	// ISequencerSection interface

	virtual UMovieSceneSection* GetSectionObject() override;
	virtual FText GetSectionToolTip() const override;
	virtual int32 OnPaintSection(FSequencerSectionPainter&) const override;

private:

	/** The section we are visualizing */
	UMovieSceneTakeSection& Section;
};

