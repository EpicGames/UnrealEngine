// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TakeSequencer : ModuleRules
{
	public TakeSequencer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				
				"Slate",
				"SlateCore",
				"UnrealEd"
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
                "MovieScene",
                "MovieSceneTracks",
                "MovieSceneTools",
				"Sequencer",
				"TakeMovieScene",
				"Engine",
            }
        );
	}
}
