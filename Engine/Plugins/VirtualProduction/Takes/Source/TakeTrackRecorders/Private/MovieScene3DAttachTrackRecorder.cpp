// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackRecorders/MovieScene3DAttachTrackRecorder.h"
#include "Tracks/MovieScene3DAttachTrack.h"
#include "Sections/MovieScene3DAttachSection.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "LevelSequence.h"
#include "TakeTrackRecordersUtils.h"
#include "ActorSequenceInformation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene3DAttachTrackRecorder)

bool FMovieScene3DAttachTrackRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return InObjectToRecord->IsA<AActor>();
}

UMovieSceneTrackRecorder* FMovieScene3DAttachTrackRecorderFactory::CreateTrackRecorderForObject() const
{
	return NewObject<UMovieScene3DAttachTrackRecorder>();
}

bool FMovieScene3DAttachTrackRecorderFactory::CanCreateTrackRecorderForHost(const IMovieSceneTrackRecorderHost* Host, UObject* ObjectToRecord) const
{
	if (Host == nullptr)
	{
		return IMovieSceneTrackRecorderFactory::CanCreateTrackRecorderForHost(Host, ObjectToRecord);
	}

	if (!IsValid(ObjectToRecord))
	{
		return false;
	}

	ETakeRecorderAttachRecordBehaviour AttachBehaviour = Host->GetTrackRecorderSettings().AttachRecordBehaviour;
	// The AttachBehaviour should have been resolved prior to calling this function. However, if it has not, we can safe out with false.
	if (!ensure(AttachBehaviour != ETakeRecorderAttachRecordBehaviour::ProjectDefault))
	{
		return false;
	}

	return AttachBehaviour != ETakeRecorderAttachRecordBehaviour::None;
}

void UMovieScene3DAttachTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentTime)
{
	AActor* ActorToRecord = Cast<AActor>(ObjectToRecord.Get());
	if (ActorToRecord)
	{
		FFrameRate TickResolution = MovieScene->GetTickResolution();
		FFrameNumber CurrentFrame = CurrentTime.ConvertTo(TickResolution).FloorToFrame();

		if (MovieSceneSection.IsValid())
		{
			MovieSceneSection->SetEndFrame(CurrentFrame);
		}

		// get attachment and check if the actor we are attached to is being recorded
		FName SocketName;
		FName ComponentName;
		AActor* AttachedToActor = UE::TakeTrackRecordersUtils::Private::GetAttachment(ActorToRecord, SocketName, ComponentName);

		// We need a target Guid for the parent. If the parent is being recorded, this needs to be the recording binding guid.
		// If it is not being recorded but it already exists within the sequence, then we need to get that guid.
		// If neither of the above is true, we can't record an attachment as the attachment relies on their being a binding.
		bool bIsControlledBySequence = false;
		const bool bTargetActorChanged = AttachedToActor == nullptr || AttachedToActor != ActorAttachedTo.Get();
		if (bTargetActorChanged)
		{
			Guid.Invalidate();
			CachedTargetSequenceID.Reset();
			if (AttachedToActor)
			{
				UE::TakesCore::FActorSequenceInformation SeqInfo(OwningTakeRecorderSource->GetRootLevelSequence(), AttachedToActor);
				if (OwningTakeRecorderSource->IsOtherActorBeingRecorded(AttachedToActor))
				{
					Guid = OwningTakeRecorderSource->GetRecordedActorGuid(AttachedToActor);
					CachedTargetSequenceID = OwningTakeRecorderSource->GetLevelSequenceID(AttachedToActor);
				}
				else if (SeqInfo.IsControlledBySequence())
				{
					bIsControlledBySequence = true;
					Guid = SeqInfo.GetCachedObjectBindingGuid();
					CachedTargetSequenceID = SeqInfo.GetControllingSequenceID();
				}
			}
		}

		if (Guid.IsValid())
		{
			// create the Section if we haven't already without reusing any existing tracks.
			if (!AttachTrack.IsValid())
			{
				AttachTrack = MovieScene->AddTrack<UMovieScene3DAttachTrack>(ObjectGuid);
			}

			// check if we need a section or if the actor we are attached to has changed
			if (!MovieSceneSection.IsValid() || bTargetActorChanged)
			{
				MovieSceneSection = Cast<UMovieScene3DAttachSection>(AttachTrack->CreateNewSection());
				AttachTrack->AddSection(*MovieSceneSection);

				MovieSceneSection->AttachSocketName = SocketName;
				MovieSceneSection->AttachComponentName = ComponentName;

				MovieSceneSection->TimecodeSource = MovieScene->GetEarliestTimecodeSource();
				MovieSceneSection->SetRange(TRange<FFrameNumber>(CurrentFrame, CurrentFrame));

				FMovieSceneSequenceID TargetSequenceID = CachedTargetSequenceID
					? *CachedTargetSequenceID
					: OwningTakeRecorderSource->GetLevelSequenceID(AttachedToActor);
				FMovieSceneSequenceID ThisSequenceID   = OwningTakeRecorderSource->GetSequenceID();

				FMovieSceneObjectBindingID NewBinding = UE::MovieScene::FRelativeObjectBindingID(ThisSequenceID, TargetSequenceID, Guid, OwningTakeRecorderSource->GetRootLevelSequence());
				MovieSceneSection->SetConstraintBindingID(NewBinding);
			}

			ActorAttachedTo = AttachedToActor;
		}
		else
		{
			// no attachment, so end the section recording if we have any
			MovieSceneSection = nullptr;
			// also ensure the cached actor to attach to is null'd out.
			ActorAttachedTo = nullptr;
		}
	}
}

void UMovieScene3DAttachTrackRecorder::FinalizeTrackImpl()
{
	AActor* ActorToRecord = Cast<AActor>(ObjectToRecord.Get());
	if (ActorToRecord)
	{
		FName SocketName;
		FName ComponentName;
		AActor* AttachedToActor = UE::TakeTrackRecordersUtils::Private::GetAttachment(ActorToRecord, SocketName, ComponentName);
		//note this actor may no longer exist BUT we need to do this in finalize since the compilation only happens there.
		//fix would be to have a way to get the sequence id differently then doing the compilation above
		//or have another way to do cleanup
		if (MovieSceneSection.IsValid() && AttachedToActor)
		{
			if (!Guid.IsValid())
			{				
			#if WITH_EDITOR
				const FString ActorId = ActorToRecord->GetActorLabel();
			#else // WITH_EDITOR
				const FString ActorId = ActorToRecord->GetName();
			#endif // WITH_EDITOR

				UE_LOGF(LogTemp, Warning, "Could not find binding to attach (%ls) to its parent (%ls), perhaps (%ls) was not recorded?", *ActorId, *ActorId, *ActorId);
			}

			FMovieSceneSequenceID TargetSequenceID = CachedTargetSequenceID
				? *CachedTargetSequenceID
				: OwningTakeRecorderSource->GetLevelSequenceID(AttachedToActor);
			FMovieSceneSequenceID ThisSequenceID = OwningTakeRecorderSource->GetSequenceID();

			FMovieSceneObjectBindingID NewBinding = UE::MovieScene::FRelativeObjectBindingID(ThisSequenceID, TargetSequenceID, Guid, OwningTakeRecorderSource->GetRootLevelSequence());
			MovieSceneSection->SetConstraintBindingID(NewBinding);
		}
	}
}

