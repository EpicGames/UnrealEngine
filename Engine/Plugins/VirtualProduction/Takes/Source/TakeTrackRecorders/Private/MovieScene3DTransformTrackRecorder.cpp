// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackRecorders/MovieScene3DTransformTrackRecorder.h"
#include "Animation/Skeleton.h"
#include "TrackRecorders/MovieSceneAnimationTrackRecorder.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Misc/ScopedSlowTask.h"
#include "Engine/SkeletalMesh.h"
#include "GameFramework/Character.h"
#include "KeyParams.h"
#include "Algo/Transform.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Components/SceneComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Evaluation/MovieSceneEvaluationState.h"
#include "TakeTrackRecordersUtils.h"
#include "ActorSequenceInformation.h"
#include "TakesUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene3DTransformTrackRecorder)


DEFINE_LOG_CATEGORY(TransformSerialization);

bool FMovieScene3DTransformTrackRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	if(USceneComponent* SceneComponent = Cast<USceneComponent>(InObjectToRecord))
	{
		// Dont record the root component transforms as this will be taken into account by the actor transform track
		// Also dont record transforms of skeletal mesh components as they will be taken into account in the actor transform
		bool bIsCharacterSkelMesh = false;
		if (SceneComponent->IsA<USkeletalMeshComponent>() && SceneComponent->GetOwner()->IsA<ACharacter>())
		{
			ACharacter* Character = CastChecked<ACharacter>(SceneComponent->GetOwner());
			bIsCharacterSkelMesh = SceneComponent == Character->GetMesh();
		}
		return (SceneComponent != SceneComponent->GetOwner()->GetRootComponent() && !bIsCharacterSkelMesh);
	}
	else 
	{
		return InObjectToRecord->IsA<AActor>();
	}
}

UMovieSceneTrackRecorder* FMovieScene3DTransformTrackRecorderFactory::CreateTrackRecorderForObject() const
{
	return NewObject<UMovieScene3DTransformTrackRecorder>();
}

void UMovieScene3DTransformTrackRecorder::CreateTrackImpl()
{
	bWasAttached = false;
	bSetFirstKey = true;
	static const FName Transform("Transform");
	MovieSceneTrack = MovieScene->FindTrack<UMovieScene3DTransformTrack>(ObjectGuid, Transform);
	if (!MovieSceneTrack.IsValid())
	{
		MovieSceneTrack = MovieScene->AddTrack<UMovieScene3DTransformTrack>(ObjectGuid);
	}
	else
	{
		// If the existing track is not empty, allow it to evaluate during the recording, and we will remove it at the end.
		if (TakesUtils::DoesTrackHaveAnyKeys(*MovieSceneTrack))
		{
			// Track the existing section to remove in finalize.
			// We cannot use the new section to drive the right attachment as it is set to not active so it is does not get evaluated in sequencer while recording.
			// Therefore if we have an attachment, we need to keep any existing data around, and then remove it once we have finished recording to allow
			// the recorded section to then take control.
			for (UMovieSceneSection* Section : MovieSceneTrack->GetAllSections())
			{
				SectionsToRemoveOnFinalize.Add(Section);
			}
		}
		else
		{
			MovieSceneTrack->RemoveAllAnimationData();
		}
	}

	if (MovieSceneTrack.IsValid())
	{
		MovieSceneSection = Cast<UMovieScene3DTransformSection>(MovieSceneTrack->CreateNewSection());
		
		// Disable the section after creation so that the track can't be evaluated by Sequencer while recording.
		MovieSceneSection->SetIsActive(false);

		MovieSceneTrack->AddSection(*MovieSceneSection);

		FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();

		FText Error;
		FString Name = ObjectToRecord.IsValid() ? ObjectToRecord->GetName() : TEXT("Unnamed_Actor");
		FName SerializedType("Transform");
		FString FileName = FString::Printf(TEXT("%s_%s"), *(SerializedType.ToString()), *Name);

		FTransformFileHeader Header(TickResolution, SerializedType, ObjectGuid);
		if (!TransformSerializer.OpenForWrite(FileName, Header, Error))
		{
			//UE_LOGF(LogFrameTransport, Error, "Cannot open frame debugger cache %ls. Failed to create archive.", *InFilename);
			UE_LOGF(TransformSerialization, Warning, "Error Opening Transform Sequencer File: Object '%ls' Error '%ls'", *(Name), *(Error.ToString()));
		}
	}
}

bool UMovieScene3DTransformTrackRecorder::ShouldAddNewKey(const FTransform& TransformToRecord)
{
	return (bSetFirstKey || !FTransform::AreTranslationsEqual(TransformToRecord, PreviousValue) || !FTransform::AreRotationsEqual(TransformToRecord, PreviousValue) || !FTransform::AreScale3DsEqual(TransformToRecord, PreviousValue));
}


void UMovieScene3DTransformTrackRecorder::StopRecordingImpl()
{
	TransformSerializer.Close();
}

void UMovieScene3DTransformTrackRecorder::FinalizeTrackImpl()
{
 	if (!MovieSceneSection.IsValid())
 	{
 		return;
 	}
 
 	FScopedSlowTask SlowTask(5.0f, NSLOCTEXT("TakeRecorder", "ProcessingTransforms", "Processing Transforms"));

	// Remove any sections we want to remove on finalize
	for (const TWeakObjectPtr<UMovieSceneSection>& WeakSection : SectionsToRemoveOnFinalize)
	{
		if (TStrongObjectPtr<UMovieSceneSection> StrongSection = WeakSection.Pin())
		{
			MovieSceneTrack->RemoveSection(*StrongSection.Get());
		}
	}
	SectionsToRemoveOnFinalize.Reset();

	SlowTask.EnterProgressFrame();
 
 	check (	BufferedTransforms.Times.Num() == BufferedTransforms.LocationX.Num());	
 	check (	BufferedTransforms.Times.Num() == BufferedTransforms.LocationY.Num());
 	check (	BufferedTransforms.Times.Num() == BufferedTransforms.LocationZ.Num());
 	check (	BufferedTransforms.Times.Num() == BufferedTransforms.RotationX.Num());
 	check (	BufferedTransforms.Times.Num() == BufferedTransforms.RotationY.Num());
 	check (	BufferedTransforms.Times.Num() == BufferedTransforms.RotationZ.Num());
 	check (	BufferedTransforms.Times.Num() == BufferedTransforms.ScaleX.Num());
 	check (	BufferedTransforms.Times.Num() == BufferedTransforms.ScaleY.Num());
 	check (	BufferedTransforms.Times.Num() == BufferedTransforms.ScaleZ.Num());
 
 	SlowTask.EnterProgressFrame();
 
	FTransform InvParentAnimationRootTransform = FTransform::Identity;
	FName SocketName;
	FName ComponentName;
	AActor* ActorToRecord = Cast<AActor>(ObjectToRecord.Get());
	if (ActorToRecord)
	{
		AActor* AttachedToActor = UE::TakeTrackRecordersUtils::Private::GetAttachment(ActorToRecord, SocketName, ComponentName);
		//If attached to an actor that has an initial root transform(skeletal mesh) that we negated out when moving over the anim keys to the transform
		//track we need to reapply that inverse here. But if it's attached to a socket we don't since we are in not in actor space but sill in component space.
		if (AttachedToActor  && SocketName == NAME_None)
		{
			InvParentAnimationRootTransform = OwningTakeRecorderSource->GetRecordedActorAnimationInitialRootTransform(AttachedToActor).Inverse();
			if (!InvParentAnimationRootTransform.Equals(FTransform::Identity))
			{
				if (DefaultTransform.IsSet())
				{
					FTransform DT = DefaultTransform.GetValue();
					DT = DT * InvParentAnimationRootTransform;
					DefaultTransform = DT;
					FVector Translation = DT.GetTranslation();
					FVector EulerRotation = DT.GetRotation().Rotator().Euler();
					FVector Scale = DT.GetScale3D();
					TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = MovieSceneSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
					DoubleChannels[0]->SetDefault(Translation.X);
					DoubleChannels[1]->SetDefault(Translation.Y);
					DoubleChannels[2]->SetDefault(Translation.Z);
					DoubleChannels[3]->SetDefault(EulerRotation.X);
					DoubleChannels[4]->SetDefault(EulerRotation.Y);
					DoubleChannels[5]->SetDefault(EulerRotation.Z);
					DoubleChannels[6]->SetDefault(Scale.X);
					DoubleChannels[7]->SetDefault(Scale.Y);
					DoubleChannels[8]->SetDefault(Scale.Z);
				}
				if (BufferedTransforms.Times.Num() > 0)
				{
					BufferedTransforms.PostMultTransform(InvParentAnimationRootTransform);
				}
			}
		}
		else if (BufferedTransforms.Times.Num() > 0)
		{
			BufferedTransforms.PostMultTransform(OwningTakeRecorderSource->GetTrackRecorderSettings().TransformOrigin.Inverse());
		}
	}

 	// Try to 're-wind' rotations that look like axis flips
 	// We need to do this as a post-process because the recorder cant reliably access 'wound' rotations:
 	// - Net quantize may use quaternions.
 	// - Scene components cache transforms as quaternions.
 	// - Gameplay is free to clamp/fmod rotations as it sees fit.
 	int32 TransformCount = BufferedTransforms.Times.Num();
 	for(int32 TransformIndex = 0; TransformIndex < TransformCount - 1; TransformIndex++)
 	{
 		FMath::WindRelativeAnglesDegrees(BufferedTransforms.RotationZ[TransformIndex], BufferedTransforms.RotationZ[TransformIndex+1]);
 		FMath::WindRelativeAnglesDegrees(BufferedTransforms.RotationY[TransformIndex], BufferedTransforms.RotationY[TransformIndex+1]);
 		FMath::WindRelativeAnglesDegrees(BufferedTransforms.RotationX[TransformIndex], BufferedTransforms.RotationX[TransformIndex+1]);
 	}
 
 	SlowTask.EnterProgressFrame();
  
 	// add buffered transforms
 	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = MovieSceneSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
 
	ERichCurveInterpMode LocalInterpMode = InterpolationMode;
 	auto Transformation = [LocalInterpMode](double In)
 	{
 		FMovieSceneDoubleValue NewValue(In);
 		NewValue.InterpMode = LocalInterpMode;
 		return NewValue;
 	};
 	TArray<FMovieSceneDoubleValue> DoubleValues;
 
 	DoubleValues.Reset(BufferedTransforms.LocationX.Num());
 	Algo::Transform(BufferedTransforms.LocationX, DoubleValues, Transformation);
 	DoubleChannels[0]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));
 
 	DoubleValues.Reset(BufferedTransforms.LocationY.Num());
 	Algo::Transform(BufferedTransforms.LocationY, DoubleValues, Transformation);
 	DoubleChannels[1]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));
 
 	DoubleValues.Reset(BufferedTransforms.LocationZ.Num());
 	Algo::Transform(BufferedTransforms.LocationZ, DoubleValues, Transformation);
 	DoubleChannels[2]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));
 
 	DoubleValues.Reset(BufferedTransforms.RotationX.Num());
 	Algo::Transform(BufferedTransforms.RotationX, DoubleValues, Transformation);
 	DoubleChannels[3]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));
 
 	DoubleValues.Reset(BufferedTransforms.RotationY.Num());
 	Algo::Transform(BufferedTransforms.RotationY, DoubleValues, Transformation);
 	DoubleChannels[4]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));
 
 	DoubleValues.Reset(BufferedTransforms.RotationZ.Num());
 	Algo::Transform(BufferedTransforms.RotationZ, DoubleValues, Transformation);
 	DoubleChannels[5]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));
 
 	DoubleValues.Reset(BufferedTransforms.ScaleX.Num());
 	Algo::Transform(BufferedTransforms.ScaleX, DoubleValues, Transformation);
 	DoubleChannels[6]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));
 
 	DoubleValues.Reset(BufferedTransforms.ScaleY.Num());
 	Algo::Transform(BufferedTransforms.ScaleY, DoubleValues, Transformation);
 	DoubleChannels[7]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));
 
 	DoubleValues.Reset(BufferedTransforms.ScaleZ.Num());
 	Algo::Transform(BufferedTransforms.ScaleZ, DoubleValues, Transformation);
 	DoubleChannels[8]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));
 
 	FTransform FirstTransform = FTransform::Identity;
	if (DefaultTransform.IsSet())
	{
		FirstTransform = DefaultTransform.GetValue();
	}
	else if (BufferedTransforms.Times.Num())
 	{
 		FirstTransform.SetTranslation(FVector(BufferedTransforms.LocationX[0], BufferedTransforms.LocationY[0], BufferedTransforms.LocationZ[0]));
 		FirstTransform.SetRotation(FQuat(FRotator(BufferedTransforms.RotationY[0], BufferedTransforms.RotationZ[0], BufferedTransforms.RotationX[0])));
 		FirstTransform.SetScale3D(FVector(BufferedTransforms.ScaleX[0], BufferedTransforms.ScaleY[0], BufferedTransforms.ScaleZ[0]));
 	}
 
 	BufferedTransforms = UE::TrackRecorders::FBufferedTransformKeys();
 
 	SlowTask.EnterProgressFrame();
 
	FTrackRecorderSettings TrackRecorderSettings = OwningTakeRecorderSource->GetTrackRecorderSettings();

	if (TrackRecorderSettings.bReduceKeys)
	{
		FKeyDataOptimizationParams Params;
		Params.bAutoSetInterpolation = true;
		Params.Tolerance = TrackRecorderSettings.ReduceKeysTolerance;
		for (FMovieSceneDoubleChannel* Channel : DoubleChannels)
		{
			Channel->Optimize(Params);
		}
	}
	else
	{
		for (FMovieSceneDoubleChannel* Channel : DoubleChannels)
		{
			Channel->AutoSetTangents();
		}
	}

	if (TrackRecorderSettings.bRemoveRedundantTracks)
	{
 		// we cant remove redundant tracks if we were attached as the playback relies on update order of
 		// transform tracks. Without this track, relative transforms would accumulate.
 		if(!bWasAttached)
 		{
 			bool bCanReset = true;
			for (FMovieSceneDoubleChannel* Channel : MovieSceneSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>())
			{
				if (Channel->GetNumKeys() > 1)
				{
					bCanReset = false;
					break;
				}
			}

			if (bCanReset)
			{
				for (FMovieSceneDoubleChannel* Channel : MovieSceneSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>())
				{
					Channel->Reset();
				}

				DoubleChannels[0]->SetDefault(FirstTransform.GetTranslation()[0]);
				DoubleChannels[1]->SetDefault(FirstTransform.GetTranslation()[1]);
				DoubleChannels[2]->SetDefault(FirstTransform.GetTranslation()[2]);
				DoubleChannels[3]->SetDefault(FirstTransform.GetRotation().Euler().X);
				DoubleChannels[4]->SetDefault(FirstTransform.GetRotation().Euler().Y);
				DoubleChannels[5]->SetDefault(FirstTransform.GetRotation().Euler().Z);
				DoubleChannels[6]->SetDefault(FirstTransform.GetScale3D()[0]);
				DoubleChannels[7]->SetDefault(FirstTransform.GetScale3D()[1]);
				DoubleChannels[8]->SetDefault(FirstTransform.GetScale3D()[2]);

				//no longer remove spawnable transform tracks since they may not match the template
			}
 		}
 	}
 
 	SlowTask.EnterProgressFrame();
 
	// If recording a spawnable, update the spawnable object template to the first keyframe
	if (MovieScene.IsValid() && ObjectGuid.IsValid())
	{
		UMovieSceneSequence* ThisSequence = MovieScene->GetTypedOuter<UMovieSceneSequence>();
		UObject* WorldContext = ObjectToRecord.Get() ? ObjectToRecord.Get() : GWorld.GetReference();
		if (WorldContext)
		{ 
			TSharedRef<UE::MovieScene::FSharedPlaybackState> TransientPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(WorldContext, ThisSequence);

			if (AActor* ActorTemplatePtr = Cast<AActor>(MovieSceneHelpers::GetObjectTemplate(ThisSequence, ObjectGuid, TransientPlaybackState, 0)))
			{
				if (USceneComponent* RootComponent = ActorTemplatePtr->GetRootComponent())
				{
					RootComponent->SetRelativeTransform(FirstTransform);
				}
			}
		}
	}
}

void UMovieScene3DTransformTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentTime)
{
 	if (!MovieSceneTrack.IsValid())
 	{
 		return;
 	}
 
 	if(ObjectToRecord.IsValid() && MovieSceneSection.IsValid())
 	{
 		FFrameRate   TickResolution  = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
 		FFrameNumber CurrentFrame    = CurrentTime.ConvertTo(TickResolution).FloorToFrame();
		
		MovieSceneSection->ExpandToFrame(CurrentFrame);

 		if(USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectToRecord.Get()))
 		{
 			// dont record non-registered scene components
 			if(!SceneComponent->IsRegistered())
 			{
 				return;
 			}
 		}
		if (!DefaultTransform.IsSet())
		{
			SetUpDefaultTransform();
		}

		FTransform TransformThisFrame = FTransform::Identity;
		if (ResolveTransformToRecord(TransformThisFrame))
		{
			if (ShouldAddNewKey(TransformThisFrame))
			{
				if (PreviousFrame.IsSet())
				{
					BufferedTransforms.Add(PreviousValue, PreviousFrame.GetValue());
					FSerializedTransform SerializedTransform(PreviousValue, PreviousFrame.GetValue());
					TransformSerializer.WriteFrameData(TransformSerializer.FramesWritten, SerializedTransform);
				}
				BufferedTransforms.Add(TransformThisFrame, CurrentFrame);
				FSerializedTransform SerializedTransform(TransformThisFrame, CurrentFrame);
				TransformSerializer.WriteFrameData(TransformSerializer.FramesWritten, SerializedTransform);

				PreviousValue = TransformThisFrame;
				PreviousFrame.Reset();
			}
			else
			{
				if (bSetFirstKey)
				{
					FSerializedTransform SerializedTransform(PreviousValue, CurrentFrame);
					TransformSerializer.WriteFrameData(TransformSerializer.FramesWritten, SerializedTransform);
				}
				PreviousFrame = CurrentFrame;
			}
			bSetFirstKey = false;

		}
 	}
}

void UMovieScene3DTransformTrackRecorder::SetUpDefaultTransform()
{
	FTransform DT = FTransform::Identity;
	ResolveTransformToRecord(DT);

	FVector Translation = DT.GetTranslation();
	FVector EulerRotation = DT.GetRotation().Rotator().Euler();
	FVector Scale = DT.GetScale3D();
	
	DefaultTransform = DT;
	PreviousValue = DT;
	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = MovieSceneSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	DoubleChannels[0]->SetDefault(Translation.X);
	DoubleChannels[1]->SetDefault(Translation.Y);
	DoubleChannels[2]->SetDefault(Translation.Z);
	DoubleChannels[3]->SetDefault(EulerRotation.X);
	DoubleChannels[4]->SetDefault(EulerRotation.Y);
	DoubleChannels[5]->SetDefault(EulerRotation.Z);
	DoubleChannels[6]->SetDefault(Scale.X);
	DoubleChannels[7]->SetDefault(Scale.Y);
	DoubleChannels[8]->SetDefault(Scale.Z);
}

bool UMovieScene3DTransformTrackRecorder::ResolveTransformToRecord(FTransform& OutTransform)
{
 	if(USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectToRecord.Get()))
 	{
 		OutTransform = SceneComponent->GetRelativeTransform();

 		return true;
 	}
 	else if(AActor* Actor = Cast<AActor>(ObjectToRecord.Get()))
 	{
 		bool bCaptureWorldSpaceTransform = false; //if not attached world = relative so we set this false so 
 
 		USceneComponent* RootComponent = Actor->GetRootComponent();
 		USceneComponent* AttachParent = RootComponent ? RootComponent->GetAttachParent() : nullptr;

		// if the attach parent has changed, recache its sequence information
		if (AttachParent != ActiveAttachParent)
		{
			if (AttachParent != nullptr && OwningTakeRecorderSource != nullptr)
			{
				ActiveAttachParent = AttachParent;
				AttachParentSequenceInformation = MakeShared<UE::TakesCore::FActorSequenceInformation>(OwningTakeRecorderSource->GetRootLevelSequence(), AttachParent->GetOwner());
			}
			else
			{
				ActiveAttachParent = nullptr;
				AttachParentSequenceInformation.Reset();
			}
		}
 
		// Only track if this attachment turns true so that we can compensate on Finalize
		if (!bWasAttached)
		{
	 		bWasAttached = AttachParent != nullptr;
		}

		if (AttachParent && OwningTakeRecorderSource)
 		{
			const bool bIsAttachParentControlledBySequence = AttachParentSequenceInformation.IsValid()
				? AttachParentSequenceInformation->IsControlledBySequence()
				: false;
			const bool bIsAttachParentBeingRecorded = OwningTakeRecorderSource->IsOtherActorBeingRecorded(AttachParent->GetOwner());

 			// We capture world space transforms for actors if they're attached, but we're not recording the attachment parent or the parent being controlled by a sequence.
			FTrackRecorderSettings TrackRecorderSettings = OwningTakeRecorderSource->GetTrackRecorderSettings();
			ETakeRecorderAttachRecordBehaviour AttachRecordBehaviour = TrackRecorderSettings.AttachRecordBehaviour;

			switch (AttachRecordBehaviour)
			{
				case ETakeRecorderAttachRecordBehaviour::None:
				{
					bCaptureWorldSpaceTransform = bIsAttachParentBeingRecorded || bIsAttachParentControlledBySequence
						? !TrackRecorderSettings.bRecordToPossessable
						: true;
				} break;
				case ETakeRecorderAttachRecordBehaviour::Attachments:
				{
					bCaptureWorldSpaceTransform = !bIsAttachParentBeingRecorded && !bIsAttachParentControlledBySequence;
				} break;
				default: bCaptureWorldSpaceTransform = false; break;
			}
 		}

 		if (!RootComponent)
 		{
 			return false;
 		}
 
	    if (bCaptureWorldSpaceTransform)
 		{
 			OutTransform = Actor->ActorToWorld();
 		}
 		else
 		{
			/*  Need to fix this for weapon mesh recording needs to be local to what it's attached.
			FName SocketName;
			FName ComponentName;
			if (bCaptureWorldSpaceTransform)
			{
				OutTransform = Actor->ActorToWorld();
				FTransform ParentTransform = AttachedToActor->ActorToWorld();
				OutTransform = ParentTransform.GetRelativeTransform(OutTransform);
			}
			else
				*/
			{
				OutTransform = RootComponent->GetRelativeTransform();
			}
 		}

 		return true;
 	}

	return false;
}
//  Move root from animation from animation sequence to transform track IF we are removing root animation (which doesn't happen with dynamic spawned skelmeshes)
void UMovieScene3DTransformTrackRecorder::PostProcessAnimationData(UMovieSceneAnimationTrackRecorder* AnimTrackRecorder)
{
	// Post processing of animation data is only done at editor time.
	// Runtime recording will handle this separately as it cannot use the FBufferedTransformKeys.
#if WITH_EDITOR
	ensure(AnimTrackRecorder);

	if (!MovieSceneSection.IsValid() || !AnimTrackRecorder)
	{
		return;
	}
	if (AnimTrackRecorder->RootWasRemoved())
	{
		//Get All Animation Keys
		UE::TrackRecorders::FBufferedTransformKeys  AnimationKeys;

		USkeletalMeshComponent* SkeletalMeshComponent = AnimTrackRecorder->GetSkeletalMeshComponent();
		bool bLayerCurrentBuffered = false;
		if (SkeletalMeshComponent)
		{
			FTransform Relative = FTransform::Identity;
			if (AActor* Actor = Cast<AActor>(ObjectToRecord.Get()))
			{
				if (bWasAttached && OwningTakeRecorderSource && DefaultTransform.IsSet())
				{
					if (BufferedTransforms.Num() == 0)
					{
						Relative = DefaultTransform.GetValue();
					}
					else
					{
						bLayerCurrentBuffered = true;
					}
				}

			}
			// Search for the Root Bone in the Skeleton
			const USkinnedAsset* SkinnedAsset = SkeletalMeshComponent->LeaderPoseComponent != nullptr ?
				SkeletalMeshComponent->LeaderPoseComponent->GetSkinnedAsset() : 
				SkeletalMeshComponent->GetSkeletalMeshAsset();
			const UAnimSequence* AnimSequence = AnimTrackRecorder->GetAnimSequence();
			if (AnimSequence && SkinnedAsset && AnimSequence->GetSkeleton())
			{
				// Find the root bone
				int32 RootIndex = INDEX_NONE;
				USkeleton* AnimSkeleton = AnimSequence->GetSkeleton();

				TArray<FName> TrackNames;
				// Need to revisit this when implementing the runtime implementation.
				AnimSequence->GetDataModelInterface()->GetBoneTrackNames(TrackNames);

				FName RootBoneName = NAME_None;
				for (const FName& TrackName : TrackNames)
				{
					// Verify if this bone exists in skeleton
					const int32 BoneTreeIndex = AnimSequence->GetSkeleton()->GetReferenceSkeleton().FindBoneIndex(TrackName);
					if (BoneTreeIndex != INDEX_NONE)
					{
						const int32 BoneIndex = AnimSkeleton->GetMeshBoneIndexFromSkeletonBoneIndex(SkinnedAsset, BoneTreeIndex);
						const int32 ParentIndex = SkinnedAsset->GetRefSkeleton().GetParentIndex(BoneIndex);
						if (ParentIndex == INDEX_NONE)
						{
							// We've found the root (root bones do not have a valid parent)
							RootIndex = BoneIndex;
							RootBoneName = TrackName;
							break;
						}
					}
				}

				if (RootBoneName == NAME_None)
				{
					FString ObjectToRecordName = ObjectToRecord.IsValid() ? ObjectToRecord->GetName() : TEXT("Unnamed_Actor");
					UE_LOGF(LogTakesCore, Log, "No Root Found for (%ls)", *ObjectToRecordName);
					return;
				}

				const FFrameRate TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
				const FFrameNumber StartTime = MovieSceneSection->GetInclusiveStartFrame();

				FTransform InvComponentTransform = AnimTrackRecorder->GetComponentTransform().Inverse();

				// Need to revisit this when implementing the runtime implementation.
				for (int32 KeyIndex = 0; KeyIndex < AnimSequence->GetDataModelInterface()->GetNumberOfKeys(); KeyIndex++)
				{
					const FTransform Transform = AnimSequence->GetDataModelInterface()->GetBoneTrackTransform(RootBoneName, FFrameNumber(KeyIndex));
					FFrameNumber AnimationFrame = (AnimSequence->GetTimeAtFrame(KeyIndex) * TickResolution).FloorToFrame();
					FTransform Total = InvComponentTransform * Transform * Relative;
					AnimationKeys.Add(Total, StartTime + AnimationFrame);
					if (KeyIndex == 0)
					{
						DefaultTransform = Total;
					}
				}
			}
		}

		if (!bLayerCurrentBuffered)
		{
			BufferedTransforms = MoveTemp(AnimationKeys);
		}
		else
		{
			// The current BufferedTransforms need to layer on top of the animation keys
			BufferedTransforms = AnimationKeys.Collapse(BufferedTransforms);
		}
	}
#endif // WITH_EDITOR
}

bool UMovieScene3DTransformTrackRecorder::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap,  TFunction<void()> InCompletionCallback)
{
	bool bFileExists = TransformSerializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FTransformFileHeader Header;

		if (TransformSerializer.OpenForRead(FileName, Header, Error))
		{
			TransformSerializer.GetDataRanges([this, InMovieScene, FileName, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
			{
				auto OnReadComplete = [this, InMovieScene, Header, InCompletionCallback]()
				{
					TArray<FTransformSerializedFrame> &InFrames = TransformSerializer.ResultData;
					if (InFrames.Num() > 0)
					{
						static FName Transform("Transform");
						MovieSceneTrack = InMovieScene->FindTrack<UMovieScene3DTransformTrack>(Header.Guid, Transform);
						if (!MovieSceneTrack.IsValid())
						{
							MovieSceneTrack = InMovieScene->AddTrack<UMovieScene3DTransformTrack>(Header.Guid);
						}
						else
						{
							MovieSceneTrack->RemoveAllAnimationData();
						}

						if (MovieSceneTrack.IsValid())
						{

							FFrameRate FileTickResolution = Header.TickResolution;
							FFrameRate TickResolution = InMovieScene->GetTickResolution();
							FFrameTime FrameTime;
							FFrameNumber CurrentFrame;

							MovieSceneSection = Cast<UMovieScene3DTransformSection>(MovieSceneTrack->CreateNewSection());

							MovieSceneTrack->AddSection(*MovieSceneSection);
							TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = MovieSceneSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

							bool InFirst = true;
							// @todo: sequencer-timecode: this was previously never actually used - should it be??
							const ERichCurveInterpMode Interpolation = RCIM_Cubic;//AnimRecorder.IsValid() ? RCIM_Linear : RCIM_Cubic;
							TArray<FMovieSceneDoubleValue> TransX;
							TransX.SetNumUninitialized(InFrames.Num());
							TArray<FMovieSceneDoubleValue> TransY;
							TransY.SetNumUninitialized(InFrames.Num());
							TArray<FMovieSceneDoubleValue> TransZ;
							TransZ.SetNumUninitialized(InFrames.Num());
							TArray<FMovieSceneDoubleValue> RotX;
							RotX.SetNumUninitialized(InFrames.Num());
							TArray<FMovieSceneDoubleValue> RotY;
							RotY.SetNumUninitialized(InFrames.Num());
							TArray<FMovieSceneDoubleValue> RotZ;
							RotZ.SetNumUninitialized(InFrames.Num());
							TArray<FMovieSceneDoubleValue> ScaleX;
							ScaleX.SetNumUninitialized(InFrames.Num());
							TArray<FMovieSceneDoubleValue> ScaleY;
							ScaleY.SetNumUninitialized(InFrames.Num());
							TArray<FMovieSceneDoubleValue> ScaleZ;
							ScaleZ.SetNumUninitialized(InFrames.Num());
							TArray<FFrameNumber> Times;
							Times.SetNumUninitialized(InFrames.Num());

							int Index = 0;
							FMovieSceneDoubleValue NewValue;
							NewValue.InterpMode = RCIM_Cubic; //Linear if animation ? mz

							for (const FTransformSerializedFrame& SerializedFrame : InFrames)
							{
								const FSerializedTransform &Frame = SerializedFrame.Frame;

								FrameTime = FFrameRate::TransformTime(Frame.Time, FileTickResolution, TickResolution);
								CurrentFrame = FrameTime.FrameNumber;
								Times[Index] = CurrentFrame;

								if (InFirst)
								{
									MovieSceneSection->SetRange(TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame));

									DoubleChannels[0]->SetDefault(Frame.Values[0]);
									DoubleChannels[1]->SetDefault(Frame.Values[1]);
									DoubleChannels[2]->SetDefault(Frame.Values[2]);
									DoubleChannels[3]->SetDefault(Frame.Values[3]);
									DoubleChannels[4]->SetDefault(Frame.Values[4]);
									DoubleChannels[5]->SetDefault(Frame.Values[5]);
									DoubleChannels[6]->SetDefault(Frame.Values[6]);
									DoubleChannels[7]->SetDefault(Frame.Values[7]);
									DoubleChannels[8]->SetDefault(Frame.Values[8]);
									InFirst = false;
								}

								NewValue.Value = Frame.Values[0];
								TransX[Index] = NewValue;
								NewValue.Value = Frame.Values[1];
								TransY[Index] = NewValue;
								NewValue.Value = Frame.Values[2];
								TransZ[Index] = NewValue;
								NewValue.Value = Frame.Values[3];
								RotX[Index] = NewValue;
								NewValue.Value = Frame.Values[4];
								RotY[Index] = NewValue;
								NewValue.Value = Frame.Values[5];
								RotZ[Index] = NewValue;
								NewValue.Value = Frame.Values[6];
								ScaleX[Index] = NewValue;
								NewValue.Value = Frame.Values[7];
								ScaleY[Index] = NewValue;
								NewValue.Value = Frame.Values[8];
								ScaleZ[Index] = NewValue;

								MovieSceneSection->ExpandToFrame(CurrentFrame);
								++Index;
							}
							DoubleChannels[0]->Set(Times, MoveTemp(TransX));
							DoubleChannels[1]->Set(Times, MoveTemp(TransY));
							DoubleChannels[2]->Set(Times, MoveTemp(TransZ));
							DoubleChannels[3]->Set(Times, MoveTemp(RotX));
							DoubleChannels[4]->Set(Times, MoveTemp(RotY));
							DoubleChannels[5]->Set(Times, MoveTemp(RotZ));
							DoubleChannels[6]->Set(Times, MoveTemp(ScaleX));
							DoubleChannels[7]->Set(Times, MoveTemp(ScaleY));
							DoubleChannels[8]->Set(Times, MoveTemp(ScaleZ));

						}
					}
					TransformSerializer.Close();
					InCompletionCallback();
				}; //callback

				TransformSerializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

			});
			return true;
		}
		else
		{
			TransformSerializer.Close();
		}
	}
	return false;
}

