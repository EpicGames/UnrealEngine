// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackRecorders/MovieSceneActorReferenceTrackRecorder.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "TakeRecorderSource.h"
#include "TakeRecorderSources.h"
#include "TakesCoreLog.h"
#include "TakesUtils.h"
#include "Tracks/MovieSceneActorReferenceTrack.h"
#include "ITakeRecorderSourcesManager.h"
#include "ActorSequenceInformation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneActorReferenceTrackRecorder)

bool FMovieSceneActorReferenceTrackRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return false;
}

UMovieSceneTrackRecorder* FMovieSceneActorReferenceTrackRecorderFactory::CreateTrackRecorderForObject() const
{
	return nullptr;
}

bool FMovieSceneActorReferenceTrackRecorderFactory::CanRecordProperty(UObject* InObjectToRecord, FProperty* InPropertyToRecord) const
{
	if (InPropertyToRecord && (InPropertyToRecord->IsA<FSoftObjectProperty>() || InPropertyToRecord->IsA<FObjectProperty>()))
	{
		return true;
	}
	return false;
}

UMovieSceneTrackRecorder* FMovieSceneActorReferenceTrackRecorderFactory::CreateTrackRecorderForProperty(UObject* InObjectToRecord,
	const FName& InPropertyToRecord) const
{
	UMovieSceneActorReferenceTrackRecorder* TrackRecorder = NewObject<UMovieSceneActorReferenceTrackRecorder>();
	TrackRecorder->CreateBinding(InPropertyToRecord);
	return TrackRecorder;
}

void UMovieSceneActorReferenceTrackRecorder::CreateBinding(const FName& InPropertyToRecord)
{
	PropertyBinding = MakeShared<FTrackInstancePropertyBindings>(InPropertyToRecord, InPropertyToRecord.ToString());
}

void UMovieSceneActorReferenceTrackRecorder::CreateTrackImpl()
{
	bSetFirstKey = true;

	check(PropertyBinding.IsValid())
	check(MovieScene.IsValid());
	check(ObjectGuid.IsValid());
	
	const TStrongObjectPtr<UMovieScene> MovieScenePin = MovieScene.Pin();
	
	const FName TrackName = *PropertyBinding->GetPropertyPath();
	UMovieSceneActorReferenceTrack* Track = MovieScenePin->FindTrack<UMovieSceneActorReferenceTrack>(ObjectGuid, TrackName);
	if (!Track)
	{
		Track = MovieScenePin->AddTrack<UMovieSceneActorReferenceTrack>(ObjectGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}
	if (!Track)
	{
		Track = MovieScenePin->AddTrack<UMovieSceneActorReferenceTrack>(ObjectGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}
	if (Track)
	{
		Track->SetPropertyNameAndPath(PropertyBinding->GetPropertyName(), PropertyBinding->GetPropertyPath());
		MovieSceneSection = Cast<UMovieSceneActorReferenceSection>(Track->CreateNewSection());
		// We only set the track defaults when we're not loading from a serialized recording. Serialized recordings don't store channel defaults but will always store a
		// key on the first frame which will accomplish the same.
		
		AActor* ActorValue = EvaluateBinding();
		LastBindingValue = ActorValue;
		
		const FMovieSceneActorReferenceKey ReferenceKey = ActorToReferenceKey(ActorValue);
		FMovieSceneActorReferenceChannel* ActorReferenceChannel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneActorReferenceChannel>(0);
		if (ensure(ActorReferenceChannel))
		{
			ActorReferenceChannel->SetDefault(ReferenceKey);
		}
		
		Track->AddSection(*MovieSceneSection);
	}
}

void UMovieSceneActorReferenceTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentTime)
{
	if (!MovieSceneSection.IsValid())
	{
		return;
	}
	
	bIsRecording = true;

	if (ObjectToRecord.IsValid())
	{
		if (bSetFirstKey)
		{
			bSetFirstKey = false;
			FMovieSceneActorReferenceChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneActorReferenceChannel>(0);
			if (ensure(Channel))
			{
				if (MovieSceneSection->HasStartFrame())
				{
					const FMovieSceneActorReferenceKey ReferenceKey = ActorToReferenceKey(LastBindingValue.Get());
					Channel->GetData().AddKey(MovieSceneSection->GetInclusiveStartFrame(), ReferenceKey);
				}
			}
		}

		const FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		const FFrameNumber CurrentFrame = CurrentTime.ConvertTo(TickResolution).FloorToFrame();

		MovieSceneSection->SetEndFrame(CurrentFrame);
		
		AActor* ActorValue = EvaluateBinding();
		if (LastBindingValue.Get() != ActorValue)
		{
			LastBindingValue = ActorValue;
			FMovieSceneActorReferenceChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneActorReferenceChannel>(0);
			if (ensure(Channel))
			{
				const FMovieSceneActorReferenceKey ReferenceKey = ActorToReferenceKey(ActorValue);
				Channel->GetData().AddKey(CurrentFrame, ReferenceKey);
			}
		}
	}
}


void UMovieSceneActorReferenceTrackRecorder::FinalizeTrackImpl()
{
	bIsRecording = false;
	
	if (!MovieSceneSection.IsValid())
	{
		return;
	}

	if (!ObjectToRecord.IsValid())
	{
		return;
	}

	const FTrackRecorderSettings TrackRecorderSettings = OwningTakeRecorderSource->GetTrackRecorderSettings();

	if (TrackRecorderSettings.bRemoveRedundantTracks)
	{
		RemoveRedundantTracks();
	}
}

void UMovieSceneActorReferenceTrackRecorder::RemoveRedundantTracks()
{
		if (!ObjectToRecord.IsValid() || !MovieSceneSection.IsValid() || !PropertyBinding.IsValid() || !MovieScene.IsValid())
		{
			return;
		}
		
		FTrackRecorderSettings TrackRecorderSettings = OwningTakeRecorderSource->GetTrackRecorderSettings();
			
		// The section can be removed if this is a spawnable since the spawnable template should have the same default values
		bool bRemoveSection = true;

		bool bExclude = FTrackRecorderSettings::IsExcludePropertyTrack(ObjectToRecord.Get(), PropertyBinding->GetPropertyPath(), TrackRecorderSettings.DefaultTracks);
		if (!bExclude)
		{
			// If any channel has more than 1 key, the track cannot be removed
			for (const FMovieSceneChannelEntry& Entry : MovieSceneSection->GetChannelProxy().GetAllEntries())
			{
				TArrayView<FMovieSceneChannel* const> Channels = Entry.GetChannels();

				for (int32 Index = 0; Index < Channels.Num(); ++Index)
				{
					if (Channels[Index]->GetNumKeys() > 1)
					{
						return;
					}
				}
			}

			// Assumes each channel is left with 1 or no keys, so the keys can be removed and the default value set
			FMovieSceneActorReferenceChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneActorReferenceChannel>(0);
			const FMovieSceneActorReferenceKey DefaultReferenceKey = Channel->GetDefault();

			// Reset channels
			for (const FMovieSceneChannelEntry& Entry : MovieSceneSection->GetChannelProxy().GetAllEntries())
			{
				TArrayView<FMovieSceneChannel* const> Channels = Entry.GetChannels();

				for (int32 Index = 0; Index < Channels.Num(); ++Index)
				{
					Channels[Index]->Reset();
				}
			}

			Channel->SetDefault(DefaultReferenceKey);

			// This section can only be removed if the CDO value is the same and it's not on the allow list of default property tracks.
			// NOTE: On the property recorder we restrict this just to TrackRecorderSettings.bRecordToPossessable, but with actor references this is
			// necessary in the event we start recording a spawnable as a default value, before starting the recording (a common scenario).
			{
				bRemoveSection = false;

				if (UObject* DefaultObject = ObjectToRecord->GetClass()->GetDefaultObject())
				{
					AActor* CurrentActorValue = EvaluateBinding(DefaultObject);
					const FMovieSceneActorReferenceKey ActorReferenceKey = ActorToReferenceKey(CurrentActorValue);
					if (ActorReferenceKey == DefaultReferenceKey)
					{
						bRemoveSection = true;
					}
				}

				if (bRemoveSection && FTrackRecorderSettings::IsDefaultPropertyTrack(ObjectToRecord.Get(), PropertyBinding->GetPropertyPath(), TrackRecorderSettings.DefaultTracks))
				{
					bRemoveSection = false;
				}
			}
		}

		if (bRemoveSection || bExclude)
		{
			UMovieSceneTrack* MovieSceneTrack = CastChecked<UMovieSceneTrack>(MovieSceneSection->GetOuter());

			UE_LOGF(LogTakesCore, Log, "Removed unused track (%ls) for (%ls)", *MovieSceneTrack->GetTrackName().ToString(), *ObjectToRecord->GetName());

			MovieSceneTrack->RemoveSection(*MovieSceneSection);
			MovieScene->RemoveTrack(*MovieSceneTrack);
		}
}

FMovieSceneActorReferenceKey UMovieSceneActorReferenceTrackRecorder::ActorToReferenceKey(AActor* InActor) const
{	
	if (InActor != nullptr)
	{
		ULevelSequence* RootLevelSequence = OwningTakeRecorderSource->GetRootLevelSequence();
		check(RootLevelSequence);
		
		FMovieSceneObjectBindingID Binding;
		
		ULevelSequence* TargetLevelSequence = OwningTakeRecorderSource->GetLevelSequence(InActor);
		if (ensure(TargetLevelSequence))
		{
			bool bIsActorBeingRecorded = false;
			if (UTakeRecorderSources* RootSources = ITakeRecorderSourcesManager::GetChecked().FindSources(RootLevelSequence))
			{
				for (const UTakeRecorderSource* Source : RootSources->GetSources())
				{
					if (Source && Source->IsActorSourceForActor(InActor))
					{
						bIsActorBeingRecorded = true;
						break;
					}
				}
			}

			UE::TakesCore::FActorSequenceInformation InActorSeqInfo(TargetLevelSequence, InActor);
			FGuid ActorId = InActorSeqInfo.GetCachedObjectBindingGuid();

			if (!ActorId.IsValid())
			{
				// Fallback, try resolve from the sequence directly
				ActorId = TakesUtils::ResolveActorFromSequence(InActor, TargetLevelSequence);
			}

			/*
			 * Only look to create new recording sources if it is not already being recorded from the root sources. In sub-sequences
			 * mode the root sources own this actor's recording; calling AddSourceForActorBinding
			 * would create a duplicate source whose UTakeRecorderSources has a null
			 * CachedLevelSequence, causing PreRecording to pass null as InRootSequence and
			 * ultimately crash in EnsureObjectTemplateHasComponent.
			 */
			if (!bIsActorBeingRecorded)
			{
				// Ensure a Take Recorder source exists for it so its properties get recorded.
				UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindOrAddSources(TargetLevelSequence);
				ULevelSequence* RecordingSequence = TargetLevelSequence = Sources->AddSourceForActorBinding(InActor, TargetLevelSequence);

				// If ActorId is already valid, it remains correct: StartRecording detects the existing binding via ResolveActorFromSequence 
				// and reuses its GUID, so no duplicate possessable is created and no re-resolve is needed here.
				// If it is not valid, we need to refind it now if it exists.
				if (!ActorId.IsValid())
				{
					check(RecordingSequence);
					TargetLevelSequence = RecordingSequence;
					ActorId = TakesUtils::ResolveActorFromSequence(InActor, TargetLevelSequence);
				}
			}
		
			// Actor reference tracks should always be able to find a valid guid -- all sources must be established by this point.
			if (ensure(ActorId.IsValid()))
			{
				const FMovieSceneSequenceID TargetSequenceID = OwningTakeRecorderSource->GetLevelSequenceID(InActor);
				const FMovieSceneSequenceID ThisSequenceID   = OwningTakeRecorderSource->GetSequenceID();
		
				// Setup binding relative to a target sequence, which may or may not be the root sequence, so we can support references in subsequences.
				Binding = UE::MovieScene::FRelativeObjectBindingID(ThisSequenceID, TargetSequenceID, ActorId, OwningTakeRecorderSource->GetRootLevelSequence());
			}
			}

		if (Binding.IsValid())
		{
			FMovieSceneActorReferenceKey NewKey;
			NewKey.Object = MoveTemp(Binding);
			return NewKey;
		}
	}
	
	return FMovieSceneActorReferenceKey();
}

AActor* UMovieSceneActorReferenceTrackRecorder::EvaluateBinding(const UObject* OwningObject) const
{
	if (OwningObject == nullptr)
	{
		OwningObject = ObjectToRecord.Get();
	}
	
	check(OwningObject);
	
	const FProperty* PropertyToRecord = PropertyBinding->GetProperty(*OwningObject);
	check(PropertyToRecord);
		
	AActor* ActorValue = nullptr;
	if (PropertyToRecord->IsA<FSoftObjectProperty>())
	{
		const TSoftObjectPtr<AActor> ActorValueSoftObjectPtr = PropertyBinding->GetCurrentValue<TSoftObjectPtr<AActor>>(*OwningObject);
		ActorValue = ActorValueSoftObjectPtr.LoadSynchronous();
	}
	else if (PropertyToRecord->IsA<FObjectProperty>())
	{
		ActorValue = Cast<AActor>(PropertyBinding->GetCurrentValue<TObjectPtr<UObject>>(*OwningObject));
	}
		
	return ActorValue;
}
