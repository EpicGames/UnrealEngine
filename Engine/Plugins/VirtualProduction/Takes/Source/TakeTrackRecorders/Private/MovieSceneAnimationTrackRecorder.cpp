// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackRecorders/MovieSceneAnimationTrackRecorder.h"
#include "Animation/Skeleton.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/SkeletalMesh.h"
#include "TrackRecorders/MovieSceneAnimationTrackRecorderSettings.h"
#include "TakesUtils.h"
#include "TakeMetaData.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "MovieScene.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimationRecordingSettings.h"
#include "Engine/TimecodeProvider.h"
#include "Engine/Engine.h"
#include "LevelSequence.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectBaseUtility.h"
#include "NamingTokenData.h"
#include "NamingTokens/TakeRecorderNamingTokensContext.h"

#if WITH_EDITOR
#include "AnimationRecorder.h"
#include "Logging/MessageLog.h"
#include "SequenceRecorderSettings.h"
#include "SequenceRecorderUtils.h"
#include "ObjectTools.h"
#include "Editor.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimationTrackRecorder)

#define LOCTEXT_NAMESPACE "MovieSceneAnimationTrackRecorder"

#if WITH_EDITOR
DEFINE_LOG_CATEGORY(AnimationSerialization);
#endif // WITH_EDITOR

bool FMovieSceneAnimationTrackRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	if (InObjectToRecord->IsA<USkeletalMeshComponent>())
	{
		USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InObjectToRecord);
		if (SkeletalMeshComponent && SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			return true;
		}
	}
	return false;
}

UMovieSceneTrackRecorder* FMovieSceneAnimationTrackRecorderFactory::CreateTrackRecorderForObject() const
{
	return NewObject<UMovieSceneAnimationTrackRecorder>();
}

void UMovieSceneAnimationTrackRecorder::CreateAnimationAssetAndSequence(const AActor* Actor, const FDirectoryPath& AnimationDirectory)
{
	UMovieSceneAnimationTrackRecorderSettings* AnimSettings = CastChecked<UMovieSceneAnimationTrackRecorderSettings>(Settings.Get());

	TStrongObjectPtr<USkeletalMeshComponent> StrongSkeletalMeshComponent = SkeletalMeshComponent.Pin();
	if (!StrongSkeletalMeshComponent.IsValid())
	{
		return;
	}

	SkeletalMesh = StrongSkeletalMeshComponent->GetSkeletalMeshAsset();

	if (SkeletalMesh.IsValid())
	{
		ComponentTransform = StrongSkeletalMeshComponent->GetComponentToWorld().GetRelativeTransform(Actor->GetTransform());
	#if WITH_EDITOR
		FString AnimationAssetName = Actor->GetActorLabel();
	#else // WITH_EDITOR
		FString AnimationAssetName = Actor->GetName();
	#endif // WITH_EDITOR

		FString DirectoryPath = AnimationDirectory.Path;

		if (ULevelSequence* LeaderLevelSequence = OwningTakeRecorderSource->GetRootLevelSequence())
		{
		#if WITH_EDITOR
			UTakeMetaData* AssetMetaData = LeaderLevelSequence->FindMetaData<UTakeMetaData>();

			UTakeRecorderNamingTokensContext* Context = NewObject<UTakeRecorderNamingTokensContext>();
			Context->Actor = Actor;
			
			AnimationAssetName = AssetMetaData->GenerateAssetPath(AnimSettings->AnimationAssetName, Context);
			DirectoryPath = AssetMetaData->GenerateAssetPath(DirectoryPath, Context);
		#endif // WITH_EDITOR
		}

		AnimSequence = TakesUtils::MakeNewAsset<UAnimSequence>(DirectoryPath, AnimationAssetName);
		if (AnimSequence.IsValid())
		{
			AnimSequence.Get()->MarkPackageDirty();

			FAssetRegistryModule::AssetCreated(AnimSequence.Get());

			// Assign the skeleton we're recording to the newly created Animation Sequence.
			AnimSequence->SetSkeleton(StrongSkeletalMeshComponent->GetSkeletalMeshAsset()->GetSkeleton());
		}
	}

}
// todo move to FTakeUtils?
static FGuid GetActorInSequence(AActor* InActor, UMovieScene* MovieScene)
{
#if WITH_EDITOR
	FString ActorTargetName = InActor->GetActorLabel();
#else // WITH_EDITOR
	FString ActorTargetName = InActor->GetName();
#endif // WITH_EDITOR

	for (int32 PossessableCount = 0; PossessableCount < MovieScene->GetPossessableCount(); ++PossessableCount)
	{
		const FMovieScenePossessable& Possessable = MovieScene->GetPossessable(PossessableCount);
		if (Possessable.GetName() == ActorTargetName || Possessable.Tags.Contains(*ActorTargetName))
		{
			return Possessable.GetGuid();
		}
	}
	return FGuid();
}

void UMovieSceneAnimationTrackRecorder::CreateTrackImpl()
{
	if (MovieScene.IsValid())
	{
		if (TStrongObjectPtr<UObject> StrongObjectToRecord = ObjectToRecord.Pin())
		{
			AActor* Actor = nullptr;
			SkeletalMeshComponent = CastChecked<USkeletalMeshComponent>(StrongObjectToRecord.Get());
			Actor = SkeletalMeshComponent->GetOwner();

			// Build an asset path to record our new animation asset to.
			FString PathToRecordTo = FPackageName::GetLongPackagePath(MovieScene->GetOutermost()->GetPathName());
			FString BaseName = MovieScene->GetName();

			FDirectoryPath AnimationDirectory;
			AnimationDirectory.Path = PathToRecordTo;

			UMovieSceneAnimationTrackRecorderSettings* AnimSettings = CastChecked<UMovieSceneAnimationTrackRecorderSettings>(Settings.Get());
			if (AnimSettings->AnimationSubDirectory.Len())
			{
				AnimationDirectory.Path /= AnimSettings->AnimationSubDirectory;
			}

			CreateAnimationAssetAndSequence(Actor, AnimationDirectory);

			if (AnimSequence.IsValid())
			{
				//If we are syncing to a timecode provider use that's frame rate as our frame rate since
				//otherwise use the displayrate.
				const TOptional<FQualifiedFrameTime> CurrentFrameTime = FApp::GetCurrentFrameTime();
				FFrameRate SampleRate = CurrentFrameTime.IsSet() ? CurrentFrameTime.GetValue().Rate : MovieScene->GetDisplayRate();

				FText Error;
				FString Name = SkeletalMeshComponent->GetName();
				FName SerializedType("Animation");
				FString FileName = FString::Printf(TEXT("%s_%s"), *(SerializedType.ToString()), *Name);

				float IntervalTime = SampleRate.AsDecimal() > 0.0f ? 1.0f / SampleRate.AsDecimal() : FAnimationRecordingSettings::DefaultSampleFrameRate.AsInterval();
				FAnimationFileHeader Header(SerializedType, ObjectGuid, IntervalTime);

				USkeleton* AnimSkeleton = AnimSequence->GetSkeleton();
				// add all frames

				const USkinnedMeshComponent* const LeaderPoseComponentInst = SkeletalMeshComponent->LeaderPoseComponent.Get();
				const TArray<FTransform>* SpaceBases;
				if (LeaderPoseComponentInst)
				{
					SpaceBases = &LeaderPoseComponentInst->GetComponentSpaceTransforms();
				}
				else
				{
					SpaceBases = &SkeletalMeshComponent->GetComponentSpaceTransforms();
				}
				for (int32 BoneIndex = 0; BoneIndex < SpaceBases->Num(); ++BoneIndex)
				{
					// verify if this bone exists in skeleton
					const int32 BoneTreeIndex = AnimSkeleton->GetSkeletonBoneIndexFromMeshBoneIndex(
						SkeletalMeshComponent->LeaderPoseComponent != nullptr ?
						SkeletalMeshComponent->LeaderPoseComponent->GetSkinnedAsset() :
						SkeletalMeshComponent->GetSkinnedAsset(), BoneIndex);
					if (BoneTreeIndex != INDEX_NONE)
					{
						// add tracks for the bone existing
						FName BoneTreeName = AnimSkeleton->GetReferenceSkeleton().GetBoneName(BoneTreeIndex);
						Header.AddNewRawTrack(BoneTreeName);
					}
				}
				Header.ActorGuid = GetActorInSequence(Actor, MovieScene.Get());
				Header.StartTime = 0.f; // ToDo: This should be assigned after the recording actually starts.

				if (!AnimationSerializer.OpenForWrite(FileName, Header, Error))
				{
				#if WITH_EDITOR
					//UE_LOGF(LogFrameTransport, Error, "Cannot open frame debugger cache %ls. Failed to create archive.", *InFilename);
					UE_LOGF(AnimationSerialization, Warning, "Error Opening Animation Sequencer File: Object '%ls' Error '%ls'", *(Name), *(Error.ToString()));
				#endif // WITH_EDITOR
				}
				bAnimationRecorderCreated = false;

				UMovieSceneSkeletalAnimationTrack* AnimTrack = MovieScene->FindTrack<UMovieSceneSkeletalAnimationTrack>(ObjectGuid);
				if (!AnimTrack)
				{
					AnimTrack = MovieScene->AddTrack<UMovieSceneSkeletalAnimationTrack>(ObjectGuid);
				}
				else
				{
					AnimTrack->RemoveAllAnimationData();
				}

				if (AnimTrack)
				{
					FText AnimationTrackDisplayName = AnimSettings->AnimationTrackName;
					if (const ULevelSequence* LeaderLevelSequence = OwningTakeRecorderSource->GetRootLevelSequence())
					{
					#if WITH_EDITOR
						const UTakeMetaData* AssetMetaData = LeaderLevelSequence->FindMetaData<UTakeMetaData>();

						UTakeRecorderNamingTokensContext* Context = NewObject<UTakeRecorderNamingTokensContext>();
						Context->Actor = Actor;

						AnimationTrackDisplayName = FText::FromString(AssetMetaData->GenerateAssetPath(AnimationTrackDisplayName.ToString(), Context));
					#endif // WITH_EDITOR
					}

				#if WITH_EDITOR
					AnimTrack->SetDisplayName(AnimationTrackDisplayName);
				#endif // WITH_EDITOR

					AnimTrack->AddNewAnimation(FFrameNumber(0), AnimSequence.Get());
					MovieSceneSection = Cast<UMovieSceneSkeletalAnimationSection>(AnimTrack->GetAllSections()[0]);
					MovieSceneSection->Params.bForceCustomMode = true;
				}
			}
		}
	}
}

void UMovieSceneAnimationTrackRecorder::StopRecordingImpl()
{
	AnimationSerializer.Close();

#if WITH_EDITOR
	if (AnimationRecorder.IsValid())
	{
		// Legacy Animation Recorder allowed recording into an animation asset directly and not creating an movie section
		const bool bShowAnimationAssetCreatedToast = false;
		InitialRootTransform = AnimationRecorder->GetRecorder().Get()->GetInitialRootTransform();
		AnimationRecorder->FinishRecording(bShowAnimationAssetCreatedToast);
	}
#endif // WITH_EDITOR
}

void UMovieSceneAnimationTrackRecorder::FinalizeTrackImpl()
{
 	if(MovieSceneSection.IsValid() && AnimSequence.IsValid() && MovieSceneSection->HasStartFrame())
 	{
 		FFrameRate   TickResolution  = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
 		FFrameNumber SequenceLength  = (AnimSequence->GetPlayLength() * TickResolution).FloorToFrame();

 		MovieSceneSection->SetEndFrame(TRangeBound<FFrameNumber>::Exclusive(MovieSceneSection->GetInclusiveStartFrame() + SequenceLength));
 	}

	FTrackRecorderSettings TrackRecorderSettings = OwningTakeRecorderSource->GetTrackRecorderSettings();
	if (TrackRecorderSettings.bSaveRecordedAssets)
	{
		TakesUtils::SaveAsset(GetAnimSequence(), TrackRecorderSettings.bRecordToPossessable);
	}

#if WITH_EDITOR
	if (bRecordingFailedToStart)
	{
		FMessageLog TakeRecorderMessageLog("TakeRecorder");
		TakeRecorderMessageLog.Warning(FText::Format(
			LOCTEXT("AnimRecorderInitFailed", "Animation recorder failed to be created, possibly due to the target object no longer being valid. Binding: '{0}'."),
			FText::FromString(ObjectGuid.ToString())));
		TakeRecorderMessageLog.Notify(LOCTEXT("AnimRecorderInitFailedNotify", "Animation recording was skipped for one or more bindings."));
	}
#endif // WITH_EDITOR
}

void UMovieSceneAnimationTrackRecorder::CancelTrackImpl()
{
	TArray<UObject*> AssetsToCleanUp;
	if (UAnimSequence* Anim = GetAnimSequence())
	{
		AssetsToCleanUp.Add(Anim);
	}
	
	// Revisit this cleanup as it still needs to happen in runtime so will need to update this.
#if WITH_EDITOR
	if (GEditor && AssetsToCleanUp.Num() > 0)
	{
		ObjectTools::ForceDeleteObjects(AssetsToCleanUp, false);
	}
#endif // WITH_EDITOR
}

void UMovieSceneAnimationTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentTime)
{
#if WITH_EDITOR
	// The animation recorder does most of the work here
	//  Note we wait for first tick so that we can make sure all of the attach tracks are set up .
	float CurrentSeconds = CurrentTime.AsSeconds();

	if (!bAnimationRecorderCreated && !bRecordingFailedToStart)
	{
		/*
		//Reset the start times based upon when the animation really starts.
		if (MovieSceneSection.IsValid())
		{
		#if WITH_EDITOR
			MovieSceneSection->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();
		#else // WITH_EDITOR
			// This mirrors what currently happens in SequenceRecorderUtils::GetTimecodeSource.
			MovieSceneSection->TimecodeSource = FMovieSceneTimecodeSource(FApp::GetTimecode());
		#endif // WITH_EDITOR
			FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
			FFrameNumber CurrentFrame = CurrentTime.ConvertTo(TickResolution).FloorToFrame();

			// Ensure we're expanded to at least the next frame so that we don't set the start past the end
			// when we set the first frame.
			MovieSceneSection->ExpandToFrame(CurrentFrame + FFrameNumber(1));
			MovieSceneSection->SetStartFrame(TRangeBound<FFrameNumber>::Inclusive(CurrentFrame));
		}
		*/
		if (TStrongObjectPtr<UObject> StrongObjectToRecord = ObjectToRecord.Pin())
		{
			bAnimationRecorderCreated = true;
			AActor* Actor = nullptr;
			SkeletalMeshComponent = CastChecked<USkeletalMeshComponent>(StrongObjectToRecord.Get());
			Actor = SkeletalMeshComponent->GetOwner();
			USceneComponent* RootComponent = Actor->GetRootComponent();
			USceneComponent* AttachParent = RootComponent ? RootComponent->GetAttachParent() : nullptr;

			UMovieSceneAnimationTrackRecorderSettings* AnimSettings = CastChecked<UMovieSceneAnimationTrackRecorderSettings>(Settings.Get());
			//In Sequence Recorder this would be done via checking if the component was dynamically created, due to changes in how the take recorder handles this, it no longer 
			//possible so it seems if it's native do root, otherwise use the setting
			//we what we did on the track recorder since later we need to actually remove the root and transfer to the transform track if needed.
			bRootWasRemoved = SkeletalMeshComponent->CreationMethod != EComponentCreationMethod::Native ? false : AnimSettings->bRemoveRootAnimation;

			//If not removing root we also don't record in world space ( not totally sure if it matters but matching up with Sequence Recorder)
			bool bRecordInWorldSpace = bRootWasRemoved == false ? false : true;

			FTrackRecorderSettings TrackRecorderSettings;
			if (OwningTakeRecorderSource)
			{
				TrackRecorderSettings = OwningTakeRecorderSource->GetTrackRecorderSettings();
			}

			if (bRecordInWorldSpace && AttachParent && OwningTakeRecorderSource)
			{
				// We capture world space transforms for actors if they're attached, but we're not recording the attachment parent
				bRecordInWorldSpace = !OwningTakeRecorderSource->IsOtherActorBeingRecorded(AttachParent->GetOwner());
			}

			FFrameRate SampleRate = MovieScene->GetDisplayRate();

			//Set this up here so we know that it's parent sources have also been added so we record in the correct space
			FAnimationRecordingSettings RecordingSettings;
			RecordingSettings.SampleFrameRate = SampleRate;
			RecordingSettings.InterpMode = AnimSettings->InterpMode;
			RecordingSettings.TangentMode = AnimSettings->TangentMode;
			RecordingSettings.bSetRetargetSourceAsset = AnimSettings->bSetRetargetSourceAsset;
			RecordingSettings.Length = 0;
			RecordingSettings.bRecordInWorldSpace = bRecordInWorldSpace;
			RecordingSettings.bRemoveRootAnimation = bRootWasRemoved;
			RecordingSettings.bCheckDeltaTimeAtBeginning = false;
			RecordingSettings.IncludeAnimationNames = TrackRecorderSettings.IncludeAnimationNames;
			RecordingSettings.ExcludeAnimationNames = TrackRecorderSettings.ExcludeAnimationNames;

			AnimationRecorder = MakeShared<FAnimRecorderInstance>();

			AnimationRecorder->Init(SkeletalMeshComponent.Get(), AnimSequence.Get(), &AnimationSerializer, RecordingSettings);
			AnimationRecorder->BeginRecording();
		}
		else
		{
			// Recorder failed to be created, possibly due to the target object no longer being valid.
			bRecordingFailedToStart = true;
		#if WITH_EDITOR
			UE_LOGF(AnimationSerialization, Warning, "Animation recorder failed to be created, possibly due to the target object no longer being valid.");
		#endif // WITH_EDITOR
		}
	}
	else if (bAnimationRecorderCreated)
	{
		float DeltaTime = CurrentSeconds - PreviousSeconds;
		AnimationRecorder->Update(DeltaTime);
	}

	PreviousSeconds = CurrentSeconds;


	if (SkeletalMeshComponent.IsValid())
	{
		// re-force updates on as gameplay can sometimes turn these back off!
		SkeletalMeshComponent->bEnableUpdateRateOptimizations = false;
		SkeletalMeshComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	}
#endif // WITH_EDITOR
}

void UMovieSceneAnimationTrackRecorder::RemoveRootMotion()
{
	 if(AnimSequence.IsValid())
	 {
		 if (bRootWasRemoved)
		 {
			 // Remove Root Motion by forcing the root lock on for now (which prevents the motion at evaluation time)
			 // In addition to set it to root lock we need to make sure it's to be zero'd since 
			 //	in all cases we expect the transform track to store either the absolute or relative transform for that skelmesh.

			 AnimSequence->bForceRootLock = true;
			 AnimSequence->RootMotionRootLock = ERootMotionRootLock::Zero;
		 }
	 }
}

void UMovieSceneAnimationTrackRecorder::ProcessRecordedTimes(const FString& HoursName, const FString& MinutesName, const FString& SecondsName, const FString& FramesName, const FString& SubFramesName, const FString& SlateName, const FString& Slate)
{
	FProcessRecordedTimeParams Params{
		 .HoursName = HoursName,
		 .MinutesName = MinutesName,
		 .SecondsName = SecondsName,
		 .FramesName = FramesName,
		 .SubFramesName = SubFramesName,
		 .SlateName = SlateName,
		 .Slate = Slate,
	};
	ProcessRecordedTimes(Params);
}
void UMovieSceneAnimationTrackRecorder::ProcessRecordedTimes(const FProcessRecordedTimeParams& InParams)
{
#if WITH_EDITOR
	if (AnimationRecorder.IsValid())
	{
		UMovieSceneAnimationTrackRecorderSettings* AnimSettings = CastChecked<UMovieSceneAnimationTrackRecorderSettings>(Settings.Get());
		AnimationRecorder->ProcessRecordedTimes(AnimSequence.Get(), SkeletalMeshComponent.Get(), AnimSettings->TimecodeBoneMethod, InParams);
	}
#endif // WITH_EDITOR
}

bool UMovieSceneAnimationTrackRecorder::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap,  TFunction<void()> InCompletionCallback)
{
	
	bool bFileExists = AnimationSerializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FAnimationFileHeader Header;

		if (AnimationSerializer.OpenForRead(FileName, Header, Error))
		{
			AnimationSerializer.GetDataRanges([this, InMovieScene, FileName, Header, ActorGuidToActorMap, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
			{
				auto OnReadComplete = [this, InMovieScene, FileName, Header, ActorGuidToActorMap, InCompletionCallback]()
				{
					TArray<FAnimationSerializedFrame> &InFrames = AnimationSerializer.ResultData;
					if (InFrames.Num() > 0)
					{

						UMovieSceneSkeletalAnimationTrack* AnimTrack = InMovieScene->FindTrack<UMovieSceneSkeletalAnimationTrack>(Header.Guid);
						if (!AnimTrack)
						{
							AnimTrack = InMovieScene->AddTrack<UMovieSceneSkeletalAnimationTrack>(Header.Guid);
						}
						else
						{
							AnimTrack->RemoveAllAnimationData();
						}
						if (AnimTrack)
						{
							AActor*const*  Actors = ActorGuidToActorMap.Find(Header.ActorGuid);
							if (Actors &&  Actors[0]->FindComponentByClass<USkeletalMeshComponent>())
							{
								const AActor* Actor = Actors[0];
								ObjectToRecord = Actor->FindComponentByClass<USkeletalMeshComponent>();
								MovieScene = InMovieScene;
								SkeletalMeshComponent = CastChecked<USkeletalMeshComponent>(ObjectToRecord.Get());

								FString PathToRecordTo = FPackageName::GetLongPackagePath(MovieScene->GetOutermost()->GetPathName());
								FString BaseName = MovieScene->GetName();
								FDirectoryPath AnimationDirectory;
								AnimationDirectory.Path = PathToRecordTo;

								CreateAnimationAssetAndSequence(Actor,AnimationDirectory);

								// Recording the curve data is only possible at editor time and not at runtime. As this is part of a load,
								// which is not going to be something that will be supported at runtime, for now this can simply be guarded.
							#if WITH_EDITOR
								AnimSequence->DeleteNotifyTrackData();

								IAnimationDataController& Controller = AnimSequence->GetController();
								{
									IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("LoadRecordedFile_Bracket", "Loading recorded animation file"));
									Controller.InitializeModel();
									Controller.ResetModel();

									const float FloatDenominator = 1000.0f;
									const float Numerator = FloatDenominator / Header.IntervalTime;
									const FFrameRate FrameRate(Numerator, FloatDenominator);
									Controller.SetFrameRate(FrameRate);

									int32 MaxNumberOfKeys = 0;
									for (int32 TrackIndex = 0; TrackIndex < Header.AnimationTrackNames.Num(); ++TrackIndex)
									{
										Controller.AddBoneCurve(Header.AnimationTrackNames[TrackIndex]);

										TArray<FVector3f> PosKeys;
										TArray<FQuat4f> RotKeys;
										TArray<FVector3f> ScaleKeys;

										// Generate key arrays
										for (const FAnimationSerializedFrame& SerializedFrame : InFrames)
										{
											const FSerializedAnimation& Frame = SerializedFrame.Frame;
											PosKeys.Add(FVector3f(Frame.AnimationData[TrackIndex].PosKey));
											RotKeys.Add(FQuat4f(Frame.AnimationData[TrackIndex].RotKey));
											ScaleKeys.Add(FVector3f(Frame.AnimationData[TrackIndex].ScaleKey));
										}

										MaxNumberOfKeys = FMath::Max(MaxNumberOfKeys, PosKeys.Num());
										MaxNumberOfKeys = FMath::Max(MaxNumberOfKeys, RotKeys.Num());
										MaxNumberOfKeys = FMath::Max(MaxNumberOfKeys, ScaleKeys.Num());

										Controller.SetBoneTrackKeys(Header.AnimationTrackNames[TrackIndex], PosKeys, RotKeys, ScaleKeys);
									}

									const FFrameNumber FrameNumber = FrameRate.AsFrameNumber((MaxNumberOfKeys > 1) ? (MaxNumberOfKeys - 1) * Header.IntervalTime : FrameRate.AsInterval());
									Controller.SetNumberOfFrames(FrameNumber);

									Controller.NotifyPopulated();
								}
							#endif // WITH_EDITOR

								AnimSequence->MarkPackageDirty();
								FFrameRate TickResolution = InMovieScene->GetTickResolution();;

								// save the package to disk, for convenience and so we can run this in standalone mod
								UPackage* const Package = AnimSequence->GetOutermost();
								FString const PackageName = Package->GetName();
								FString const PackageFileName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());
								FSavePackageArgs SaveArgs;
								SaveArgs.TopLevelFlags = RF_Standalone;
								SaveArgs.SaveFlags = SAVE_NoError;
								UPackage::SavePackage(Package, nullptr, *PackageFileName, SaveArgs);


								FFrameNumber SequenceLength = (AnimSequence->GetPlayLength() * TickResolution).FloorToFrame();
								FFrameNumber StartFrame = (Header.StartTime * TickResolution).FloorToFrame();
								AnimTrack->AddNewAnimation(StartFrame, AnimSequence.Get());
								MovieSceneSection = Cast<UMovieSceneSkeletalAnimationSection>(AnimTrack->GetAllSections()[0]);
								MovieSceneSection->SetEndFrame(TRangeBound<FFrameNumber>::Exclusive(MovieSceneSection->GetInclusiveStartFrame() + SequenceLength));

							}

						}
					}
					AnimationSerializer.Close();
					InCompletionCallback();
				}; //callback

				AnimationSerializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

			});
			return true;
		}
		else
		{
			AnimationSerializer.Close();
		}
	}
	
	return false;
}
#undef LOCTEXT_NAMESPACE // "MovieSceneAnimationTrackRecorder"
