// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackRecorders/MovieSceneAnimationTrackRecorderSettings.h"

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectBaseUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimationTrackRecorderSettings)

UMovieSceneAnimationTrackRecorderEditorSettings::UMovieSceneAnimationTrackRecorderEditorSettings(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, AnimationTrackName(NSLOCTEXT("UMovieSceneAnimationTrackRecorderSettings", "DefaultAnimationTrackName", "{actor}_anim"))
	, AnimationAssetName(TEXT("{actor}_{takeName}"))
	, AnimationSubDirectory(TEXT("Animation"))
	, InterpMode()
	, TangentMode()
	, bRemoveRootAnimation(true)
	, bSetRetargetSourceAsset(true)
{
	TimecodeBoneMethod.BoneMode = ETimecodeBoneMode::Root;
}

#if WITH_EDITOR
void UMovieSceneAnimationTrackRecorderEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		SaveConfig();
	}
}
#endif // WITH_EDITOR

