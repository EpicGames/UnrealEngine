// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackRecorders/MovieSceneParticleTrackRecorder.h"
#include "Tracks/MovieSceneParticleTrack.h"
#include "MovieScene.h"
#include "Channels/MovieSceneChannelProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneParticleTrackRecorder)

bool FMovieSceneParticleTrackRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return InObjectToRecord->IsA<UParticleSystemComponent>();
}

UMovieSceneTrackRecorder* FMovieSceneParticleTrackRecorderFactory::CreateTrackRecorderForObject() const
{
	return NewObject<UMovieSceneParticleTrackRecorder>();
}

void UMovieSceneParticleTrackRecorder::CreateTrackImpl()
{
	SystemToRecord = CastChecked<UParticleSystemComponent>(ObjectToRecord.Get());

	UMovieSceneParticleTrack* ParticleTrack = MovieScene->FindTrack<UMovieSceneParticleTrack>(ObjectGuid);
	if (!ParticleTrack)
	{
		ParticleTrack = MovieScene->AddTrack<UMovieSceneParticleTrack>(ObjectGuid);
	}
	else
	{
		ParticleTrack->RemoveAllAnimationData();
	}

	if (ParticleTrack)
	{
		MovieSceneSection = Cast<UMovieSceneParticleSection>(ParticleTrack->CreateNewSection());
		ParticleTrack->AddSection(*MovieSceneSection);


		bWasTriggered = false;

		UParticleSystemComponent::OnSystemPreActivationChange.AddUObject(this, &UMovieSceneParticleTrackRecorder::OnTriggered);
	}

	PreviousState = EParticleKey::Deactivate;
}

void UMovieSceneParticleTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentTime)
{
	if (!MovieSceneSection.IsValid())
	{
		return;
	}

	if(SystemToRecord.IsValid())
	{
		FFrameRate   TickResolution  = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber KeyTime         = CurrentTime.ConvertTo(TickResolution).FloorToFrame();

		// Expand our section to encompass the new frame so we can see it animating as you record.
		MovieSceneSection->ExpandToFrame(KeyTime);

		EParticleKey NewState = EParticleKey::Deactivate;
		if(SystemToRecord->IsRegistered() && SystemToRecord->IsActive() && !SystemToRecord->bWasDeactivated)
		{
			if(bWasTriggered)
			{
				NewState = EParticleKey::Trigger;
				bWasTriggered = false;
			}
			else
			{
				NewState = EParticleKey::Activate;
			}
		}
		else
		{
			NewState = EParticleKey::Deactivate;
		}

		if(NewState != PreviousState)
		{
			FMovieSceneParticleChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneParticleChannel>(0);
			if (ensure(Channel))
			{
				Channel->GetData().AddKey(KeyTime, (uint8)NewState);
			}
		}

		if(NewState == EParticleKey::Trigger)
		{
			NewState = EParticleKey::Activate;
		}
		PreviousState = NewState;
	}
}

void UMovieSceneParticleTrackRecorder::OnTriggered(UParticleSystemComponent* Component, bool bActivating)
{ 
	if(SystemToRecord.Get() == Component)
	{
		bWasTriggered = bActivating;
	}
}

