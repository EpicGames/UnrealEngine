// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackRecorders/MovieScenePropertyTrackRecorder.h"
#include "UObject/UnrealType.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePropertyTrackRecorder)

bool FMovieScenePropertyTrackRecorderFactory::CanRecordProperty(UObject* InObjectToRecord, FProperty* InPropertyToRecord) const
{
	if (InPropertyToRecord->IsA<FBoolProperty>())
	{
		return true;
	}
	else if (InPropertyToRecord->IsA<FByteProperty>())
	{
		return true;
	}
	else if (InPropertyToRecord->IsA<FEnumProperty>())
	{
		return true;
	}
	else if (InPropertyToRecord->IsA<FIntProperty>())
	{
		return true;
	}
	else if (InPropertyToRecord->IsA<FStrProperty>())
	{
		return true;
	}
	else if (InPropertyToRecord->IsA<FFloatProperty>())
	{
		return true;
	}
	else if (InPropertyToRecord->IsA<FDoubleProperty>())
	{
		return true;
	}
	else if (FStructProperty* StructProperty = CastField<FStructProperty>(InPropertyToRecord))
	{
		if (StructProperty->Struct->GetFName() == NAME_Vector)
		{
			return true;
		}
		else if (StructProperty->Struct->GetFName() == NAME_Color)
		{
			return true;
		}
		else if (StructProperty->Struct->GetFName() == NAME_LinearColor)
		{
			return true;
		}
	}

	// We only know how to make generic tracks for the types above
	return false;
}

UMovieSceneTrackRecorder* FMovieScenePropertyTrackRecorderFactory::CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const
{
	UMovieScenePropertyTrackRecorder* TrackRecorder = NewObject<UMovieScenePropertyTrackRecorder>();
	TrackRecorder->PropertyToRecord = InPropertyToRecord;
	return TrackRecorder;
}

UMovieSceneTrackRecorder* FMovieScenePropertyTrackRecorderFactory::CreateTrackRecorderForPropertyEnum(ESerializedPropertyType PropertyType, const FName& InPropertyToRecord) const
{
	UMovieScenePropertyTrackRecorder* TrackRecorder = NewObject<UMovieScenePropertyTrackRecorder>();
	TrackRecorder->PropertyToRecord = InPropertyToRecord;

	FTrackInstancePropertyBindings Binding(InPropertyToRecord, InPropertyToRecord.ToString());
	switch (PropertyType)
	{
	case ESerializedPropertyType::BoolType:
		TrackRecorder->PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<bool>(Binding));
		break;
	case ESerializedPropertyType::ByteType:
		TrackRecorder->PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<uint8>(Binding));
		break;
	case ESerializedPropertyType::EnumType:
		TrackRecorder->PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<uint8>(Binding));
		break;
	case ESerializedPropertyType::IntegerType:
		TrackRecorder->PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<int32>(Binding));
		break;
	case ESerializedPropertyType::StringType:
		TrackRecorder->PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<FString>(Binding));
		break;
	case ESerializedPropertyType::FloatType:
		TrackRecorder->PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<float>(Binding));
		break;
	case ESerializedPropertyType::DoubleType:
		TrackRecorder->PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<double>(Binding));
		break;
	case ESerializedPropertyType::Vector3fType:
		TrackRecorder->PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<FVector3f>(Binding));
		break;
	case ESerializedPropertyType::Vector3dType:
		TrackRecorder->PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<FVector3d>(Binding));
		break;
	case ESerializedPropertyType::ColorType:
		TrackRecorder->PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<FColor>(Binding));
		break;
	case ESerializedPropertyType::LinearColorType:
		TrackRecorder->PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<FLinearColor>(Binding));
		break;

	}
	return TrackRecorder;
}

void UMovieScenePropertyTrackRecorder::CreateTrackImpl()
{
 	FTrackInstancePropertyBindings Binding(PropertyToRecord, PropertyToRecord.ToString());
 	FProperty* Property = Binding.GetProperty(*ObjectToRecord);
 	if (Property != nullptr)
 	{
 		if (Property->IsA<FBoolProperty>())
 		{
 			PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<bool>(Binding));
 		}
 		else if (Property->IsA<FByteProperty>())
 		{
 			PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<uint8>(Binding));
 		}
		else if (Property->IsA<FEnumProperty>())
		{
			PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<uint8>(Binding));
		}
		else if (Property->IsA<FIntProperty>())
		{
			PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<int32>(Binding));
		}
		else if (Property->IsA<FStrProperty>())
		{
			PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<FString>(Binding));
		}
		else if (Property->IsA<FFloatProperty>())
		{
			PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<float>(Binding));
		}
		else if (Property->IsA<FDoubleProperty>())
		{
			PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<double>(Binding));
		}
		else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
 		{
			// LWC_TODO: vector types
			if (StructProperty->Struct->GetFName() == NAME_Vector3f)
 			{
 				PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<FVector3f>(Binding));
 			}
			else if (StructProperty->Struct->GetFName() == NAME_Vector3d
					|| StructProperty->Struct->GetFName() == NAME_Vector )
 			{
 				PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<FVector3d>(Binding));
 			}
			else if (StructProperty->Struct->GetFName() == NAME_Color)
 			{
				PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<FColor>(Binding));
 			}
 			else if (StructProperty->Struct->GetFName() == NAME_LinearColor)
 			{
				PropertyRecorder = MakeShareable(new FMovieSceneTrackPropertyRecorder<FLinearColor>(Binding));
 			}
 		} 
		ensure(PropertyRecorder);
		PropertyRecorder->SetSavedRecordingDirectory(Directory);
 		PropertyRecorder->Create(OwningTakeRecorderSource, ObjectToRecord.Get(), MovieScene.Get(), ObjectGuid, true);
 	}
}

void UMovieScenePropertyTrackRecorder::SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame)
{
	PropertyRecorder->SetSectionStartTimecode(InSectionStartTimecode, InSectionFirstFrame);
}

void UMovieScenePropertyTrackRecorder::FinalizeTrackImpl()
{
	PropertyRecorder->Finalize(ObjectToRecord.Get());
}

void UMovieScenePropertyTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentTime)
{
	PropertyRecorder->Record(ObjectToRecord.Get(), CurrentTime);
}

bool UMovieScenePropertyTrackRecorder::LoadRecordedFile(const FString& InFileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback)
{
	if (PropertyRecorder)
	{
		return PropertyRecorder->LoadRecordedFile(InFileName, InMovieScene, ActorGuidToActorMap, InCompletionCallback);
	}
	return false;
}

