// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackRecorders/MovieSceneSpawnTrackRecorder.h"
#include "GameFramework/Actor.h"
#include "KeyParams.h"
#include "MovieScene.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSpawnableAnnotation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSpawnTrackRecorder)

DEFINE_LOG_CATEGORY(SpawnSerialization);


UMovieSceneTrackRecorder* FMovieSceneSpawnTrackRecorderFactory::CreateTrackRecorderForObject() const
{
	return NewObject<UMovieSceneSpawnTrackRecorder>();
}

bool FMovieSceneSpawnTrackRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return InObjectToRecord->IsA<AActor>();
}

void UMovieSceneSpawnTrackRecorder::CreateTrackImpl()
{
	UMovieSceneSequence* Sequence = MovieScene->GetTypedOuter<UMovieSceneSequence>();
	if (!Sequence)
	{
		return;
	}
	if (!MovieSceneHelpers::IsBoundToAnySpawnable(Sequence, ObjectGuid, MovieSceneHelpers::CreateTransientSharedPlaybackState(this, Sequence)))
	{
		return;
	}

	UMovieSceneSpawnTrack* SpawnTrack = MovieScene->FindTrack<UMovieSceneSpawnTrack>(ObjectGuid);
	if (!SpawnTrack)
	{
		SpawnTrack = MovieScene->AddTrack<UMovieSceneSpawnTrack>(ObjectGuid);
	}
	else if (!SpawnTrack->GetAllSections().IsEmpty())
	{
		// For Sequencer-managed actors, existing sections are inherited
		// from the source sequence (CleanExistingDataFromSequence was intentionally skipped for them).
		// Removing them would cause Sequencer to destroy the spawnable on the next evaluation tick,
		// so preserve them and skip adding a new section.
		// For normal actors, fall through to RemoveAllAnimationData() and add a fresh section.
		if (TOptional<FMovieSceneSpawnableAnnotation> SpawnableAnnotation = FMovieSceneSpawnableAnnotation::Find(ObjectToRecord.Get()))
		{
			return;
		}
		SpawnTrack->RemoveAllAnimationData();
	}

	if (SpawnTrack)
	{
		MovieSceneSection = Cast<UMovieSceneBoolSection>(SpawnTrack->CreateNewSection());
		SpawnTrack->AddSection(*MovieSceneSection);
		SpawnTrack->SetObjectId(ObjectGuid);

		FMovieSceneBoolChannel* BoolChannel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
		check(BoolChannel);
		bSetFirstKey = true;
		bWasSpawned = ObjectToRecord.IsValid();
		BoolChannel->SetDefault(bWasSpawned);
		FText Error;
		FString Name = ObjectToRecord.IsValid() ? ObjectToRecord->GetName() : MovieSceneSection->GetName();
		FName SerializedType("Spawn");
		FString FileName = FString::Printf(TEXT("%s_%s"), *(SerializedType.ToString()), *Name);

		FFrameRate TickResolution = MovieScene->GetTickResolution();

		FSpawnFileHeader Header(TickResolution, SerializedType, ObjectGuid);
		if (!SpawnSerializer.OpenForWrite(FileName, Header, Error))
		{
			//UE_LOGF(LogFrameTransport, Error, "Cannot open frame debugger cache %ls. Failed to create archive.", *InFilename);
			UE_LOGF(SpawnSerialization, Warning, "Error Opening Spawn Sequencer File: Object '%ls' Error '%ls'", *(Name), *(Error.ToString()));
		}

	}
}

void UMovieSceneSpawnTrackRecorder::FinalizeTrackImpl()
{
	if (!MovieSceneSection.IsValid())
	{
		return;
	}

	// If the Track is degenerate, assume the actor was spawned and destroyed. Give it a 1 frame spawn Track.
	if (MovieSceneSection->GetRange().IsDegenerate() && MovieSceneSection->HasEndFrame())
	{
		FMovieSceneBoolChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
		if (ensure(Channel))
		{
			FFrameTime OneFrame = FFrameRate::TransformTime(FFrameTime(1), FFrameRate(1, 1), MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution());

			FFrameRate   TickResolution   = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
			FFrameNumber StartTime		  = MovieSceneSection->GetExclusiveEndFrame() - OneFrame.GetFrame();

			Channel->GetData().AddKey(StartTime, true);
			MovieSceneSection->SetStartFrame(StartTime);

			FSpawnProperty Property(StartTime, true);
			SpawnSerializer.WriteFrameData(SpawnSerializer.FramesWritten, Property);
		}
	}

	SpawnSerializer.Close();

}

void UMovieSceneSpawnTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentTime)
{
	if (!MovieSceneSection.IsValid())
	{
		return;
	}

	FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FFrameNumber FrameNumber = CurrentTime.ConvertTo(TickResolution).FloorToFrame();

	const bool bSpawned = ObjectToRecord.IsValid();
	if (bSpawned)
	{
		// Expand our section to encompass the new frame so we can see it animating as you record.
		MovieSceneSection->SetEndFrame(FrameNumber);
	}

	// Only add a new key if the value has changed.
	if (bSpawned != bWasSpawned || bSetFirstKey)
	{
		bSetFirstKey = false;
		FSpawnProperty Property(FrameNumber, bSpawned);
		SpawnSerializer.WriteFrameData(SpawnSerializer.FramesWritten, Property);
	}
}

bool UMovieSceneSpawnTrackRecorder::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback)
{
	bool bFileExists = SpawnSerializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FSpawnFileHeader Header;

		if (SpawnSerializer.OpenForRead(FileName, Header, Error))
		{
			SpawnSerializer.GetDataRanges([this, InMovieScene, FileName, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
			{
				auto OnReadComplete = [this, InMovieScene, Header, InCompletionCallback]()
				{
					UMovieSceneSpawnTrack* SpawnTrack = InMovieScene->FindTrack<UMovieSceneSpawnTrack>(Header.Guid);
					if (!SpawnTrack)
					{
						SpawnTrack = InMovieScene->AddTrack<UMovieSceneSpawnTrack>(Header.Guid);
					}
					else
					{
						SpawnTrack->RemoveAllAnimationData();
					}

					if (SpawnTrack)
					{
						MovieSceneSection = Cast<UMovieSceneBoolSection>(SpawnTrack->CreateNewSection());

						SpawnTrack->AddSection(*MovieSceneSection);
						SpawnTrack->SetObjectId(Header.Guid);

						FMovieSceneBoolChannel* BoolChannel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
						check(BoolChannel);
						BoolChannel->SetDefault(false);

						TArray<FSpawnSerializedFrame> &InFrames = SpawnSerializer.ResultData;
						if (InFrames.Num() > 0)
						{
							FFrameRate InFrameRate = Header.TickResolution;
							for (const FSpawnSerializedFrame& SerializedFrame : InFrames)
							{
								const FSpawnProperty& Frame = SerializedFrame.Frame;

								FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
								FFrameTime FrameTime = FFrameRate::TransformTime(Frame.Time, InFrameRate, TickResolution);
								FFrameNumber CurrentFrame = FrameTime.FrameNumber;
								BoolChannel->GetData().AddKey(CurrentFrame, Frame.bVal);
								MovieSceneSection->ExpandToFrame(CurrentFrame);
							}
						}
					}
					SpawnSerializer.Close();
					InCompletionCallback();
				}; //callback

				SpawnSerializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

			});
			return true;
		}
		else
		{
			SpawnSerializer.Close();
		}
	}

	return false;
}


