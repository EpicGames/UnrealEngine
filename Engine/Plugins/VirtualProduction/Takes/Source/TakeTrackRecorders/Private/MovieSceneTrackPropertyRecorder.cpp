// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackRecorders/MovieSceneTrackPropertyRecorder.h"
#include "MovieScene.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Tracks/MovieSceneBoolTrack.h"
#include "Sections/MovieSceneByteSection.h"
#include "Tracks/MovieSceneByteTrack.h"
#include "Sections/MovieSceneEnumSection.h"
#include "Tracks/MovieSceneEnumTrack.h"
#include "Sections/MovieSceneDoubleSection.h"
#include "Tracks/MovieSceneDoubleTrack.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Sections/MovieSceneColorSection.h"
#include "Tracks/MovieSceneColorTrack.h"
#include "Sections/MovieSceneVectorSection.h"
#include "Tracks/MovieSceneVectorTrack.h"
#include "Sections/MovieSceneIntegerSection.h"
#include "Tracks/MovieSceneIntegerTrack.h"
#include "Sections/MovieSceneStringSection.h"
#include "Tracks/MovieSceneStringTrack.h"
#include "Channels/MovieSceneChannelProxy.h"

// current set of compiled-in property types

DEFINE_LOG_CATEGORY(PropertySerialization);

template <>
bool FMovieSceneTrackPropertyRecorder<bool>::ShouldAddNewKey(const bool& InNewValue) const
{
	return InNewValue != PreviousValue;
}

template <>
UMovieSceneSection* FMovieSceneTrackPropertyRecorder<bool>::AddSection(const FString& TrackDisplayName, UMovieScene* InMovieScene, const FGuid& InGuid, bool bSetDefault )
{
	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneBoolTrack* Track = InMovieScene->FindTrack<UMovieSceneBoolTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneBoolTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneBoolTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneBoolSection* Section = Cast<UMovieSceneBoolSection>(Track->CreateNewSection());

		// We only set the track defaults when we're not loading from a serialized recording. Serialized recordings don't store channel defaults but will always store a
		// key on the first frame which will accomplish the same.
		if (bSetDefault)
		{
			FMovieSceneBoolChannel* BoolChannel = Section->GetChannelProxy().GetChannels<FMovieSceneBoolChannel>()[0];
			BoolChannel->SetDefault(PreviousValue);
		}

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieSceneTrackPropertyRecorder<bool>::PostCreate(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer)
{
}

template <>
void FMovieSceneTrackPropertyRecorder<bool>::AddKeysToSection(UMovieSceneSection* InSection, const TArray<FPropertyKey<bool>>& InKeys)
{
	TArray<FFrameNumber> Times;
	Times.SetNum(InKeys.Num());
	TArray<bool> Values;
	Values.SetNum(InKeys.Num());

	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		Times[Index] = InKeys[Index].Time;
		Values[Index] = InKeys[Index].Value;
	}

	InSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0)->AddKeys(Times, Values);
}

template <>
void FMovieSceneTrackPropertyRecorder<bool>::ReduceKeys(UMovieSceneSection* InSection, float ReduceKeysTolerance)
{
	// Reduce keys intentionally left blank
}

template <>
bool FMovieSceneTrackPropertyRecorder<bool>::GetDefaultValue(UMovieSceneSection* InSection)
{
	FMovieSceneBoolChannel* BoolChannel = InSection->GetChannelProxy().GetChannels<FMovieSceneBoolChannel>()[0];

	if (BoolChannel->GetNumKeys() > 0)
	{
		return BoolChannel->GetValues()[0];
	}
	else if (BoolChannel->GetDefault().IsSet())
	{
		return BoolChannel->GetDefault().GetValue();
	}

	return false;
}

template <>
void FMovieSceneTrackPropertyRecorder<bool>::SetDefaultValue(UMovieSceneSection* InSection, const bool& InDefaultValue)
{
	return InSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0)->SetDefault(InDefaultValue);
}

template<>
bool FMovieSceneTrackPropertyRecorder<bool>::OpenSerializer(const FString& InObjectName, const FName& InPropertyName, const FString& InTrackDisplayName, const FGuid& InGuid)
{
	FName SerializedType("Property");
	FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FPropertyFileHeader Header(TickResolution, SerializedType, InGuid);
	Header.PropertyName = InPropertyName;
	Header.TrackDisplayName = InTrackDisplayName;
	Header.PropertyType = ESerializedPropertyType::BoolType;
	FText Error;
	FString FileName = FString::Printf(TEXT("%s_%s_%s"), *(SerializedType.ToString()), *InObjectName, *(InPropertyName.ToString()));

	if (!Serializer.OpenForWrite(FileName, Header, Error))
	{
		UE_LOGF(PropertySerialization, Warning, "Error Opening Property File: Object '%ls' Property '%ls' Error: '%ls'", *InObjectName, *InPropertyName.ToString(), *Error.ToString());
		return false;
	}
	return true;
}

template <>
bool FMovieSceneTrackPropertyRecorder<bool>::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback)
{
	bool bFileExists = Serializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FPropertyFileHeader Header;

		if (Serializer.OpenForRead(FileName, Header, Error))
		{
			MovieSceneSection = AddSection(Header.TrackDisplayName, InMovieScene, Header.Guid, false);
			if (MovieSceneSection.IsValid())
			{
				Serializer.GetDataRanges([this, InMovieScene, FileName, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
				{
					auto OnReadComplete = [this, InMovieScene, Header, InCompletionCallback]()
					{
						TArray<FPropertySerializedBoolFrame> &InFrames = Serializer.ResultData;
						if (InFrames.Num() > 0)
						{
							FFrameRate InFrameRate = Header.TickResolution;
							TArray<FPropertyKey<bool>> NewKeys;
							for (const FPropertySerializedBoolFrame& SerializedFrame : InFrames)
							{
								const FPropertySerializedBool& Frame = SerializedFrame.Frame;

								FFrameRate   TickResolution = InMovieScene->GetTickResolution();
								FFrameTime FrameTime = FFrameRate::TransformTime(Frame.Time, InFrameRate, TickResolution);
								FFrameNumber CurrentFrame = FrameTime.FrameNumber;
								FPropertyKey<bool> Key;
								Key.Time = CurrentFrame;
								Key.Value = Frame.Value;
								NewKeys.Add(Key);
								MovieSceneSection->ExpandToFrame(CurrentFrame);
							}
							AddKeysToSection(MovieSceneSection.Get(), NewKeys);
						}
						Serializer.Close();
						InCompletionCallback();
					}; //callback

					Serializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

				});
				return true;
			}
			return false;
		}
		else
		{
			Serializer.Close();
		}
	}
	return false;
}



template <>
bool FMovieSceneTrackPropertyRecorder<uint8>::ShouldAddNewKey(const uint8& InNewValue) const
{
	return InNewValue != PreviousValue;
}


template <>
UMovieSceneSection* FMovieSceneTrackPropertyRecorder<uint8>::AddSection(const FString& TrackDisplayName, UMovieScene* InMovieScene, const FGuid& InGuid, bool bSetDefault)
{
	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneByteTrack* Track = InMovieScene->FindTrack<UMovieSceneByteTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneByteTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneByteSection* Section = Cast<UMovieSceneByteSection>(Track->CreateNewSection());

		// We only set the track defaults when we're not loading from a serialized recording. Serialized recordings don't store channel defaults but will always store a
		// key on the first frame which will accomplish the same.
		if (bSetDefault)
		{
			FMovieSceneByteChannel* Channel = Section->GetChannelProxy().GetChannel<FMovieSceneByteChannel>(0);
			Channel->SetDefault(PreviousValue);
		}

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieSceneTrackPropertyRecorder<uint8>::PostCreate(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer)
{
	UMovieSceneByteTrack* Track = Cast<UMovieSceneByteTrack>(MovieSceneSection->GetOuter());
	if (Track && !Track->GetEnum())
	{
		FProperty* PropertyToRecord = Binding.GetProperty(*InObjectToRecord);
		if (PropertyToRecord)
		{
			if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(PropertyToRecord))
			{
				Track->SetEnum(EnumProperty->GetEnum());
			}
			else if (FByteProperty* ByteProperty = CastField<FByteProperty>(PropertyToRecord))
			{
				Track->SetEnum(ByteProperty->Enum);
			}
		}
	}
}

template <>
void FMovieSceneTrackPropertyRecorder<uint8>::AddKeysToSection(UMovieSceneSection* InSection, const TArray<FPropertyKey<uint8>>& InKeys)
{
	TArray<FFrameNumber> Times;
	Times.SetNum(InKeys.Num());
	TArray<uint8> Values;
	Values.SetNum(InKeys.Num());

	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		Times[Index] = InKeys[Index].Time;
		Values[Index] = InKeys[Index].Value;
	}

	InSection->GetChannelProxy().GetChannel<FMovieSceneByteChannel>(0)->AddKeys(Times, Values);
}

template <>
void FMovieSceneTrackPropertyRecorder<uint8>::ReduceKeys(UMovieSceneSection* InSection, float ReduceKeysTolerance)
{
	// Reduce keys intentionally left blank
}

template <>
uint8 FMovieSceneTrackPropertyRecorder<uint8>::GetDefaultValue(UMovieSceneSection* InSection)
{
	FMovieSceneByteChannel* Channel = InSection->GetChannelProxy().GetChannel<FMovieSceneByteChannel>(0);
	
	if (Channel->GetNumKeys() > 0)
	{
		return Channel->GetValues()[0];
	}
	else if (Channel->GetDefault().IsSet())
	{
		return Channel->GetDefault().GetValue();
	}

	return 0;
}

template <>
void FMovieSceneTrackPropertyRecorder<uint8>::SetDefaultValue(UMovieSceneSection* InSection, const uint8& InDefaultValue)
{
	return InSection->GetChannelProxy().GetChannel<FMovieSceneByteChannel>(0)->SetDefault(InDefaultValue);
}

template<>
bool FMovieSceneTrackPropertyRecorder<uint8>::OpenSerializer(const FString& InObjectName, const FName& InPropertyName, const FString& InTrackDisplayName, const FGuid& InGuid)
{
	FName SerializedType("Property");
	FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FPropertyFileHeader Header(TickResolution, SerializedType, InGuid);
	Header.PropertyName = InPropertyName;
	Header.TrackDisplayName = InTrackDisplayName;
	Header.PropertyType = ESerializedPropertyType::ByteType;
	FText Error;
	FString FileName = FString::Printf(TEXT("%s_%s_%s"), *(SerializedType.ToString()), *InObjectName, *(InPropertyName.ToString()));

	if (!Serializer.OpenForWrite(FileName, Header, Error))
	{
		UE_LOGF(PropertySerialization, Warning, "Error Opening Property File: Object '%ls' Property '%ls' Error: '%ls'", *InObjectName, *InPropertyName.ToString(), *Error.ToString());
		return false;
	}
	return true;
}

template <>
bool FMovieSceneTrackPropertyRecorder<uint8>::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback)
{
	bool bFileExists = Serializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FPropertyFileHeader Header;

		if (Serializer.OpenForRead(FileName, Header, Error))
		{
			MovieSceneSection = AddSection(Header.TrackDisplayName, InMovieScene, Header.Guid, false);

			Serializer.GetDataRanges([this, InMovieScene, FileName, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
			{
				auto OnReadComplete = [this, InMovieScene, Header, InCompletionCallback]()
				{
					TArray<FPropertySerializedByteFrame> &InFrames = Serializer.ResultData;
					if (InFrames.Num() > 0)
					{
						FFrameRate InFrameRate = Header.TickResolution;
						TArray<FPropertyKey<uint8>> NewKeys;
						for (const FPropertySerializedByteFrame& SerializedFrame : InFrames)
						{
							const FPropertySerializedByte& Frame = SerializedFrame.Frame;
							FFrameRate   TickResolution = InMovieScene->GetTickResolution();
							FFrameTime FrameTime = FFrameRate::TransformTime(Frame.Time, InFrameRate, TickResolution);
							FFrameNumber CurrentFrame = FrameTime.FrameNumber;
							FPropertyKey<uint8> Key;
							Key.Time = CurrentFrame;
							Key.Value = Frame.Value;
							NewKeys.Add(Key);
							MovieSceneSection->ExpandToFrame(CurrentFrame);
						}
						AddKeysToSection(MovieSceneSection.Get(), NewKeys);
					}
					Serializer.Close();
					InCompletionCallback();
				}; //callback

				Serializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

			});
			return true;
		}
		else
		{
			Serializer.Close();
		}
	}
	return false;
}

template <>
bool FMovieSceneTrackPropertyRecorder<double>::ShouldAddNewKey(const double& InNewValue) const
{
	return !FMath::IsNearlyEqual(PreviousValue, InNewValue);
}

template <>
UMovieSceneSection* FMovieSceneTrackPropertyRecorder<double>::AddSection(const FString& TrackDisplayName, UMovieScene* InMovieScene, const FGuid& InGuid, bool bSetDefault)
{
	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneDoubleTrack* Track = InMovieScene->FindTrack<UMovieSceneDoubleTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneDoubleTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneDoubleSection* Section = Cast<UMovieSceneDoubleSection>(Track->CreateNewSection());

		// We only set the track defaults when we're not loading from a serialized recording. Serialized recordings don't store channel defaults but will always store a
		// key on the first frame which will accomplish the same.
		if (bSetDefault)
		{
			FMovieSceneDoubleChannel* Channel = Section->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(0);
			Channel->SetDefault(PreviousValue);
		}

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieSceneTrackPropertyRecorder<double>::PostCreate(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer)
{
}

template <>
void FMovieSceneTrackPropertyRecorder<double>::AddKeysToSection(UMovieSceneSection* InSection, const TArray<FPropertyKey<double>>& InKeys)
{
	TArray<FFrameNumber> Times;
	Times.SetNum(InKeys.Num());
	TArray<FMovieSceneDoubleValue> Values;
	Values.SetNum(InKeys.Num());

	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		Times[Index] = InKeys[Index].Time;
		Values[Index] = FMovieSceneDoubleValue(InKeys[Index].Value);
	}

	InSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(0)->AddKeys(Times, Values);
}

template <>
void FMovieSceneTrackPropertyRecorder<double>::ReduceKeys(UMovieSceneSection* InSection, float ReduceKeysTolerance)
{
	FKeyDataOptimizationParams Params;
	Params.bAutoSetInterpolation = true;
	Params.Tolerance = ReduceKeysTolerance;
	UE::MovieScene::Optimize(InSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(0), Params);
}

template <>
double FMovieSceneTrackPropertyRecorder<double>::GetDefaultValue(UMovieSceneSection* InSection)
{
	FMovieSceneDoubleChannel* DoubleChannel = InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>()[0];

	if (DoubleChannel->GetNumKeys() > 0)
	{
		return DoubleChannel->GetValues()[0].Value;
	}
	else if (DoubleChannel->GetDefault().IsSet())
	{
		return DoubleChannel->GetDefault().GetValue();
	}

	return 0.f;
}

template <>
void FMovieSceneTrackPropertyRecorder<double>::SetDefaultValue(UMovieSceneSection* InSection, const double& InDefaultValue)
{
	return InSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(0)->SetDefault(InDefaultValue);
}

template<>
bool FMovieSceneTrackPropertyRecorder<double>::OpenSerializer(const FString& InObjectName, const FName& InPropertyName, const FString& InTrackDisplayName, const FGuid& InGuid)
{
	FName SerializedType("Property");
	FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FPropertyFileHeader Header(TickResolution, SerializedType, InGuid);
	Header.PropertyName = InPropertyName;
	Header.TrackDisplayName = InTrackDisplayName;
	Header.PropertyType = ESerializedPropertyType::DoubleType;
	FText Error;
	FString FileName = FString::Printf(TEXT("%s_%s_%s"), *(SerializedType.ToString()), *InObjectName, *(InPropertyName.ToString()));

	if (!Serializer.OpenForWrite(FileName, Header, Error))
	{
		UE_LOGF(PropertySerialization, Warning, "Error Opening Property File: Object '%ls' Property '%ls' Error: '%ls'", *InObjectName, *InPropertyName.ToString(), *Error.ToString());
		return false;
	}
	return true;
}

template <>
bool FMovieSceneTrackPropertyRecorder<double>::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap,  TFunction<void()> InCompletionCallback)
{
	bool bFileExists = Serializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FPropertyFileHeader Header;

		if (Serializer.OpenForRead(FileName, Header, Error))
		{
			MovieSceneSection = AddSection(Header.TrackDisplayName, InMovieScene, Header.Guid, false);

			Serializer.GetDataRanges([this, InMovieScene, FileName, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
			{
				auto OnReadComplete = [this, InMovieScene, Header, InCompletionCallback]()
				{
					TArray<FPropertySerializedDoubleFrame> &InFrames = Serializer.ResultData;
					if (InFrames.Num() > 0)
					{
						FFrameRate InFrameRate = Header.TickResolution;
						TArray<FPropertyKey<double>> NewKeys;
						for (const FPropertySerializedDoubleFrame& SerializedFrame : InFrames)
						{
							const FPropertySerializedDouble& Frame = SerializedFrame.Frame;
							FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
							FFrameTime FrameTime = FFrameRate::TransformTime(Frame.Time, InFrameRate, TickResolution);
							FFrameNumber CurrentFrame = FrameTime.FrameNumber;
							FPropertyKey<double> Key;
							Key.Time = CurrentFrame;
							Key.Value = Frame.Value;
							NewKeys.Add(Key);
							MovieSceneSection->ExpandToFrame(CurrentFrame);
						}
						AddKeysToSection(MovieSceneSection.Get(), NewKeys);
					}
					Serializer.Close();
					InCompletionCallback();
				}; //callback

				Serializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

			});
			return true;
		}
		else
		{
			Serializer.Close();
		}
	}
	return false;
}

template <>
bool FMovieSceneTrackPropertyRecorder<float>::ShouldAddNewKey(const float& InNewValue) const
{
	return !FMath::IsNearlyEqual(PreviousValue, InNewValue);
}

template <>
UMovieSceneSection* FMovieSceneTrackPropertyRecorder<float>::AddSection(const FString& TrackDisplayName, UMovieScene* InMovieScene, const FGuid& InGuid, bool bSetDefault)
{
	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneFloatTrack* Track = InMovieScene->FindTrack<UMovieSceneFloatTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneFloatTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneFloatSection* Section = Cast<UMovieSceneFloatSection>(Track->CreateNewSection());

		// We only set the track defaults when we're not loading from a serialized recording. Serialized recordings don't store channel defaults but will always store a
		// key on the first frame which will accomplish the same.
		if (bSetDefault)
		{
			FMovieSceneFloatChannel* Channel = Section->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(0);
			Channel->SetDefault(PreviousValue);
		}

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieSceneTrackPropertyRecorder<float>::PostCreate(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer)
{
}

template <>
void FMovieSceneTrackPropertyRecorder<float>::AddKeysToSection(UMovieSceneSection* InSection, const TArray<FPropertyKey<float>>& InKeys)
{
	TArray<FFrameNumber> Times;
	Times.SetNum(InKeys.Num());
	TArray<FMovieSceneFloatValue> Values;
	Values.SetNum(InKeys.Num());
	
	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		Times[Index] = InKeys[Index].Time;
		Values[Index] = FMovieSceneFloatValue(InKeys[Index].Value);
	}

	InSection->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(0)->AddKeys(Times, Values);
}

template <>
void FMovieSceneTrackPropertyRecorder<float>::ReduceKeys(UMovieSceneSection* InSection, float ReduceKeysTolerance)
{
	FKeyDataOptimizationParams Params;
	Params.bAutoSetInterpolation = true;
	Params.Tolerance = ReduceKeysTolerance;
	UE::MovieScene::Optimize(InSection->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(0), Params);
}

template <>
float FMovieSceneTrackPropertyRecorder<float>::GetDefaultValue(UMovieSceneSection* InSection)
{
	FMovieSceneFloatChannel* FloatChannel = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>()[0];

	if (FloatChannel->GetNumKeys() > 0)
	{
		return FloatChannel->GetValues()[0].Value;
	}
	else if (FloatChannel->GetDefault().IsSet())
	{
		return FloatChannel->GetDefault().GetValue();
	}

	return 0.f;
}

template <>
void FMovieSceneTrackPropertyRecorder<float>::SetDefaultValue(UMovieSceneSection* InSection, const float& InDefaultValue)
{
	return InSection->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(0)->SetDefault(InDefaultValue);
}

template<>
bool FMovieSceneTrackPropertyRecorder<float>::OpenSerializer(const FString& InObjectName, const FName& InPropertyName, const FString& InTrackDisplayName, const FGuid& InGuid)
{
	FName SerializedType("Property");
	FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FPropertyFileHeader Header(TickResolution, SerializedType, InGuid);
	Header.PropertyName = InPropertyName;
	Header.TrackDisplayName = InTrackDisplayName;
	Header.PropertyType = ESerializedPropertyType::FloatType;
	FText Error;
	FString FileName = FString::Printf(TEXT("%s_%s_%s"), *(SerializedType.ToString()), *InObjectName, *(InPropertyName.ToString()));

	if (!Serializer.OpenForWrite(FileName, Header, Error))
	{
		UE_LOGF(PropertySerialization, Warning, "Error Opening Property File: Object '%ls' Property '%ls' Error: '%ls'", *InObjectName, *InPropertyName.ToString(), *Error.ToString());
		return false;
	}
	return true;
}

template <>
bool FMovieSceneTrackPropertyRecorder<float>::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap,  TFunction<void()> InCompletionCallback)
{
	bool bFileExists = Serializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FPropertyFileHeader Header;

		if (Serializer.OpenForRead(FileName, Header, Error))
		{
			MovieSceneSection = AddSection(Header.TrackDisplayName, InMovieScene, Header.Guid, false);

			Serializer.GetDataRanges([this, InMovieScene, FileName, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
			{
				auto OnReadComplete = [this, InMovieScene, Header, InCompletionCallback]()
				{
					TArray<FPropertySerializedFloatFrame> &InFrames = Serializer.ResultData;
					if (InFrames.Num() > 0)
					{
						FFrameRate InFrameRate = Header.TickResolution;
						TArray<FPropertyKey<float>> NewKeys;
						for (const FPropertySerializedFloatFrame& SerializedFrame : InFrames)
						{
							const FPropertySerializedFloat& Frame = SerializedFrame.Frame;
							FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
							FFrameTime FrameTime = FFrameRate::TransformTime(Frame.Time, InFrameRate, TickResolution);
							FFrameNumber CurrentFrame = FrameTime.FrameNumber;
							FPropertyKey<float> Key;
							Key.Time = CurrentFrame;
							Key.Value = Frame.Value;
							NewKeys.Add(Key);
							MovieSceneSection->ExpandToFrame(CurrentFrame);
						}
						AddKeysToSection(MovieSceneSection.Get(), NewKeys);
					}
					Serializer.Close();
					InCompletionCallback();
				}; //callback

				Serializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

			});
			return true;
		}
		else
		{
			Serializer.Close();
		}
	}
	return false;
}

template <>
bool FMovieSceneTrackPropertyRecorder<FColor>::ShouldAddNewKey(const FColor& InNewValue) const
{
	return PreviousValue.R != InNewValue.R  || PreviousValue.G != InNewValue.G || PreviousValue.B != InNewValue.B;
}

template <>
UMovieSceneSection* FMovieSceneTrackPropertyRecorder<FColor>::AddSection(const FString& TrackDisplayName, UMovieScene* InMovieScene, const FGuid& InGuid, bool bSetDefault)
{
	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneColorTrack* Track = InMovieScene->FindTrack<UMovieSceneColorTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneColorTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneColorSection* Section = Cast<UMovieSceneColorSection>(Track->CreateNewSection());

		// We only set the track defaults when we're not loading from a serialized recording. Serialized recordings don't store channel defaults but will always store a
			// key on the first frame which will accomplish the same.
		if (bSetDefault)
		{
			TArrayView<FMovieSceneFloatChannel*> FloatChannels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
		
			FloatChannels[0]->SetDefault(PreviousValue.R);
			FloatChannels[1]->SetDefault(PreviousValue.G);
			FloatChannels[2]->SetDefault(PreviousValue.B);
			FloatChannels[3]->SetDefault(PreviousValue.A);
		}

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieSceneTrackPropertyRecorder<FColor>::PostCreate(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer)
{
}

template <>
void FMovieSceneTrackPropertyRecorder<FColor>::AddKeysToSection(UMovieSceneSection* InSection, const TArray<FPropertyKey<FColor>>& InKeys)
{
	TArray<FFrameNumber> Times;
	Times.SetNum(InKeys.Num());
	TArray<FMovieSceneFloatValue> RedValues, GreenValues, BlueValues, AlphaValues;
	RedValues.SetNum(InKeys.Num());
	GreenValues.SetNum(InKeys.Num());
	BlueValues.SetNum(InKeys.Num());
	AlphaValues.SetNum(InKeys.Num());

	static const float InvColor = 1.0f / 255.0f;
	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		Times[Index] = InKeys[Index].Time;
		RedValues[Index] = FMovieSceneFloatValue(InKeys[Index].Value.R * InvColor);
		GreenValues[Index] = FMovieSceneFloatValue(InKeys[Index].Value.G * InvColor);
		BlueValues[Index] = FMovieSceneFloatValue(InKeys[Index].Value.B * InvColor);
		AlphaValues[Index] = FMovieSceneFloatValue(InKeys[Index].Value.A * InvColor);
	}

	InSection->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(0)->AddKeys(Times, RedValues);
	InSection->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(1)->AddKeys(Times, GreenValues);
	InSection->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(2)->AddKeys(Times, BlueValues);
	InSection->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(3)->AddKeys(Times, AlphaValues);
}

template <>
void FMovieSceneTrackPropertyRecorder<FColor>::ReduceKeys(UMovieSceneSection* InSection, float ReduceKeysTolerance)
{
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

	FKeyDataOptimizationParams Params;
	Params.bAutoSetInterpolation = true;
	Params.Tolerance = ReduceKeysTolerance;
	UE::MovieScene::Optimize(FloatChannels[0], Params);
	UE::MovieScene::Optimize(FloatChannels[1], Params);
	UE::MovieScene::Optimize(FloatChannels[2], Params);
	UE::MovieScene::Optimize(FloatChannels[3], Params);
}

template <>
FColor FMovieSceneTrackPropertyRecorder<FColor>::GetDefaultValue(UMovieSceneSection* InSection)
{
	FLinearColor DefaultValue(0.f, 0.f, 0.f, 1.f);

	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

	if (FloatChannels[0]->GetNumKeys() > 0)
	{
		DefaultValue.R = FloatChannels[0]->GetValues()[0].Value;
	}
	else if (FloatChannels[0]->GetDefault().IsSet())
	{
		DefaultValue.R = FloatChannels[0]->GetDefault().GetValue();
	}

	if (FloatChannels[1]->GetNumKeys() > 0)
	{
		DefaultValue.G = FloatChannels[1]->GetValues()[0].Value;
	}
	else if (FloatChannels[1]->GetDefault().IsSet())
	{
		DefaultValue.G = FloatChannels[1]->GetDefault().GetValue();
	}

	if (FloatChannels[2]->GetNumKeys() > 0)
	{
		DefaultValue.B = FloatChannels[2]->GetValues()[0].Value;
	}
	else if (FloatChannels[2]->GetDefault().IsSet())
	{
		DefaultValue.B = FloatChannels[2]->GetDefault().GetValue();
	}

	if (FloatChannels[3]->GetNumKeys() > 0)
	{
		DefaultValue.A = FloatChannels[3]->GetValues()[0].Value;
	}
	else if (FloatChannels[3]->GetDefault().IsSet())
	{
		DefaultValue.A = FloatChannels[3]->GetDefault().GetValue();
	}

	return DefaultValue.ToFColor(false);
}

template <>
void FMovieSceneTrackPropertyRecorder<FColor>::SetDefaultValue(UMovieSceneSection* InSection, const FColor& InDefaultValue)
{
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	FloatChannels[0]->SetDefault(InDefaultValue.R);
	FloatChannels[1]->SetDefault(InDefaultValue.G);
	FloatChannels[2]->SetDefault(InDefaultValue.B);
	FloatChannels[3]->SetDefault(InDefaultValue.A);
}


template<>
bool FMovieSceneTrackPropertyRecorder<FColor>::OpenSerializer(const FString& InObjectName, const FName& InPropertyName, const FString& InTrackDisplayName, const FGuid& InGuid)
{
	FName SerializedType("Property");
	FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FPropertyFileHeader Header(TickResolution, SerializedType, InGuid);
	Header.PropertyName = InPropertyName;
	Header.TrackDisplayName = InTrackDisplayName;
	Header.PropertyType = ESerializedPropertyType::ColorType;
	FText Error;
	FString FileName = FString::Printf(TEXT("%s_%s_%s"), *(SerializedType.ToString()), *InObjectName, *(InPropertyName.ToString()));

	if (!Serializer.OpenForWrite(FileName, Header, Error))
	{
		UE_LOGF(PropertySerialization, Warning, "Error Opening Property File: Object '%ls' Property '%ls' Error: '%ls'", *InObjectName, *InPropertyName.ToString(), *Error.ToString());
		return false;
	}
	return true;
}

template <>
bool FMovieSceneTrackPropertyRecorder<FColor>::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback)
{
	bool bFileExists = Serializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FPropertyFileHeader Header;

		if (Serializer.OpenForRead(FileName, Header, Error))
		{
			MovieSceneSection = AddSection(Header.TrackDisplayName, InMovieScene, Header.Guid, false);

			Serializer.GetDataRanges([this, InMovieScene, FileName, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
			{
				auto OnReadComplete = [this, InMovieScene, Header, InCompletionCallback]()
				{
					TArray<FPropertySerializedColorFrame> &InFrames = Serializer.ResultData;
					if (InFrames.Num() > 0)
					{
						FFrameRate InFrameRate = Header.TickResolution;
						TArray<FPropertyKey<FColor>> NewKeys;
						for (const FPropertySerializedColorFrame& SerializedFrame : InFrames)
						{
							const FPropertySerializedColor& Frame = SerializedFrame.Frame;
							FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
							FFrameTime FrameTime = FFrameRate::TransformTime(Frame.Time, InFrameRate, TickResolution);
							FFrameNumber CurrentFrame = FrameTime.FrameNumber;
							FPropertyKey<FColor> Key;
							Key.Time = CurrentFrame;
							Key.Value = FColor(Frame.Value); 
							NewKeys.Add(Key);
							MovieSceneSection->ExpandToFrame(CurrentFrame);
						}
						AddKeysToSection(MovieSceneSection.Get(), NewKeys);
					}
					Serializer.Close();
					InCompletionCallback();
				}; //callback

				Serializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

			});
			return true;
		}
		else
		{
			Serializer.Close();
		}
	}
	return false;
}

template <>
bool FMovieSceneTrackPropertyRecorder<FLinearColor>::ShouldAddNewKey(const FLinearColor& InNewValue) const
{
	return PreviousValue.R != InNewValue.R  || PreviousValue.G != InNewValue.G || PreviousValue.B != InNewValue.B;
}

template <>
UMovieSceneSection* FMovieSceneTrackPropertyRecorder<FLinearColor>::AddSection(const FString& TrackDisplayName, UMovieScene* InMovieScene, const FGuid& InGuid, bool bSetDefault)
{
	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneColorTrack* Track = InMovieScene->FindTrack<UMovieSceneColorTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneColorTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneColorSection* Section = Cast<UMovieSceneColorSection>(Track->CreateNewSection());

		// We only set the track defaults when we're not loading from a serialized recording. Serialized recordings don't store channel defaults but will always store a
			// key on the first frame which will accomplish the same.
		if (bSetDefault)
		{
			TArrayView<FMovieSceneFloatChannel*> FloatChannels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
		
			FloatChannels[0]->SetDefault(PreviousValue.R);
			FloatChannels[1]->SetDefault(PreviousValue.G);
			FloatChannels[2]->SetDefault(PreviousValue.B);
			FloatChannels[3]->SetDefault(PreviousValue.A);
		}

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieSceneTrackPropertyRecorder<FLinearColor>::PostCreate(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer)
{
}

template <>
void FMovieSceneTrackPropertyRecorder<FLinearColor>::AddKeysToSection(UMovieSceneSection* InSection, const TArray<FPropertyKey<FLinearColor>>& InKeys)
{
	TArray<FFrameNumber> Times;
	Times.SetNum(InKeys.Num());
	TArray<FMovieSceneFloatValue> RedValues, GreenValues, BlueValues, AlphaValues;
	RedValues.SetNum(InKeys.Num());
	GreenValues.SetNum(InKeys.Num());
	BlueValues.SetNum(InKeys.Num());
	AlphaValues.SetNum(InKeys.Num());

	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		Times[Index] = InKeys[Index].Time;
		RedValues[Index] = FMovieSceneFloatValue(InKeys[Index].Value.R);
		GreenValues[Index] = FMovieSceneFloatValue(InKeys[Index].Value.G);
		BlueValues[Index] = FMovieSceneFloatValue(InKeys[Index].Value.B);
		AlphaValues[Index] = FMovieSceneFloatValue(InKeys[Index].Value.A);
	}
	InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>()[0]->AddKeys(Times, RedValues);
	InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>()[1]->AddKeys(Times, GreenValues);
	InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>()[2]->AddKeys(Times, BlueValues);
	InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>()[3]->AddKeys(Times, AlphaValues);
}

template <>
void FMovieSceneTrackPropertyRecorder<FLinearColor>::ReduceKeys(UMovieSceneSection* InSection, float ReduceKeysTolerance)
{
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

	FKeyDataOptimizationParams Params;
	Params.bAutoSetInterpolation = true;
	Params.Tolerance = ReduceKeysTolerance;
	UE::MovieScene::Optimize(FloatChannels[0], Params);
	UE::MovieScene::Optimize(FloatChannels[1], Params);
	UE::MovieScene::Optimize(FloatChannels[2], Params);
	UE::MovieScene::Optimize(FloatChannels[3], Params);
}

template <>
FLinearColor FMovieSceneTrackPropertyRecorder<FLinearColor>::GetDefaultValue(UMovieSceneSection* InSection)
{
	FLinearColor DefaultValue(0.f, 0.f, 0.f, 1.f);

	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

	if (FloatChannels[0]->GetNumKeys() > 0)
	{
		DefaultValue.R = FloatChannels[0]->GetValues()[0].Value;
	}
	else if (FloatChannels[0]->GetDefault().IsSet())
	{
		DefaultValue.R = FloatChannels[0]->GetDefault().GetValue();
	}

	if (FloatChannels[1]->GetNumKeys() > 0)
	{
		DefaultValue.G = FloatChannels[1]->GetValues()[0].Value;
	}
	else if (FloatChannels[1]->GetDefault().IsSet())
	{
		DefaultValue.G = FloatChannels[1]->GetDefault().GetValue();
	}

	if (FloatChannels[2]->GetNumKeys() > 0)
	{
		DefaultValue.B = FloatChannels[2]->GetValues()[0].Value;
	}
	else if (FloatChannels[2]->GetDefault().IsSet())
	{
		DefaultValue.B = FloatChannels[2]->GetDefault().GetValue();
	}

	if (FloatChannels[3]->GetNumKeys() > 0)
	{
		DefaultValue.A = FloatChannels[3]->GetValues()[0].Value;
	}
	else if (FloatChannels[3]->GetDefault().IsSet())
	{
		DefaultValue.A = FloatChannels[3]->GetDefault().GetValue();
	}

	return DefaultValue;
}

template <>
void FMovieSceneTrackPropertyRecorder<FLinearColor>::SetDefaultValue(UMovieSceneSection* InSection, const FLinearColor& InDefaultValue)
{
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	FloatChannels[0]->SetDefault(InDefaultValue.R);
	FloatChannels[1]->SetDefault(InDefaultValue.G);
	FloatChannels[2]->SetDefault(InDefaultValue.B);
	FloatChannels[3]->SetDefault(InDefaultValue.A);
}


template<>
bool FMovieSceneTrackPropertyRecorder<FLinearColor>::OpenSerializer(const FString& InObjectName, const FName& InPropertyName, const FString& InTrackDisplayName, const FGuid& InGuid)
{
/*
	FName SerializedType("Property");
	FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FPropertyFileHeader Header(TickResolution, SerializedType, InGuid);
	Header.PropertyName = InPropertyName;
	Header.TrackDisplayName = InTrackDisplayName;
	Header.PropertyType = ESerializedPropertyType::ColorType;
	FText Error;
	FString FileName = FString::Printf(TEXT("%s_%s_%s"), *(SerializedType.ToString()), *InObjectName, *(InPropertyName.ToString()));

	if (!Serializer.OpenForWrite(FileName, Header, Error))
	{
		UE_LOGF(PropertySerialization, Warning, "Error Opening Property File: Object '%ls' Property '%ls' Error: '%ls'", *InObjectName, *InPropertyName.ToString(), *Error.ToString());
		return false;
	}
	*/
	return true;
}

template <>
bool FMovieSceneTrackPropertyRecorder<FLinearColor>::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback)
{
/*
	bool bFileExists = Serializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FPropertyFileHeader Header;

		if (Serializer.OpenForRead(FileName, Header, Error))
		{
			MovieSceneSection = AddSection(Header.TrackDisplayName, InMovieScene, Header.Guid, false);

			Serializer.GetDataRanges([this, InMovieScene, FileName, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
			{
				auto OnReadComplete = [this, InMovieScene, Header, InCompletionCallback]()
				{
					TArray<FPropertySerializedColorFrame> &InFrames = Serializer.ResultData;
					if (InFrames.Num() > 0)
					{
						FFrameRate InFrameRate = Header.TickResolution;
						TArray<FPropertyKey<FColor>> NewKeys;
						for (const FPropertySerializedColorFrame& SerializedFrame : InFrames)
						{
							const FPropertySerializedColor& Frame = SerializedFrame.Frame;
							FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
							FFrameTime FrameTime = FFrameRate::TransformTime(Frame.Time, InFrameRate, TickResolution);
							FFrameNumber CurrentFrame = FrameTime.FrameNumber;
							FPropertyKey<FColor> Key;
							Key.Time = CurrentFrame;
							Key.Value = FColor(Frame.Value); 
							NewKeys.Add(Key);
							MovieSceneSection->ExpandToFrame(CurrentFrame);
						}
						AddKeysToSection(MovieSceneSection.Get(), NewKeys);
					}
					Serializer.Close();
					InCompletionCallback();
				}; //callback

				Serializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

			});
			return true;
		}
		else
		{
			Serializer.Close();
		}
	}
	*/
	return false;
}

template <>
bool FMovieSceneTrackPropertyRecorder<FVector3f>::ShouldAddNewKey(const FVector3f& InNewValue) const
{
	return !FMath::IsNearlyEqual(PreviousValue.X, InNewValue.X) || !FMath::IsNearlyEqual(PreviousValue.Y, InNewValue.Y) || !FMath::IsNearlyEqual(PreviousValue.Z, InNewValue.Z);
}

template <>
UMovieSceneSection* FMovieSceneTrackPropertyRecorder<FVector3f>::AddSection(const FString& TrackDisplayName, UMovieScene* InMovieScene, const FGuid& InGuid, bool bSetDefault)
{
	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneFloatVectorTrack* Track = InMovieScene->FindTrack<UMovieSceneFloatVectorTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneFloatVectorTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}


	if (Track)
	{
		Track->SetNumChannelsUsed(3);
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneFloatVectorSection* Section = Cast<UMovieSceneFloatVectorSection>(Track->CreateNewSection());

		// We only set the track defaults when we're not loading from a serialized recording. Serialized recordings don't store channel defaults but will always store a
		// key on the first frame which will accomplish the same.
		if (bSetDefault)
		{
			TArrayView<FMovieSceneFloatChannel*> FloatChannels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
			FloatChannels[0]->SetDefault(PreviousValue.X);
			FloatChannels[1]->SetDefault(PreviousValue.Y);
			FloatChannels[2]->SetDefault(PreviousValue.Z);
		}

		Track->AddSection(*Section);
		return Section;
	}

	return nullptr;
}

template <>
void FMovieSceneTrackPropertyRecorder<FVector3f>::PostCreate(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer)
{
}

template <>
void FMovieSceneTrackPropertyRecorder<FVector3f>::AddKeysToSection(UMovieSceneSection* InSection, const TArray<FPropertyKey<FVector3f>>& InKeys)
{
	TArray<FFrameNumber> Times;
	Times.SetNum(InKeys.Num());
	TArray<FMovieSceneFloatValue> XValues, YValues, ZValues;
	XValues.SetNum(InKeys.Num());
	YValues.SetNum(InKeys.Num());
	ZValues.SetNum(InKeys.Num());

	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		Times[Index] = InKeys[Index].Time;
		XValues[Index] = FMovieSceneFloatValue(InKeys[Index].Value.X);
		YValues[Index] = FMovieSceneFloatValue(InKeys[Index].Value.Y);
		ZValues[Index] = FMovieSceneFloatValue(InKeys[Index].Value.Z);
	}
	InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>()[0]->AddKeys(Times, XValues);
	InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>()[1]->AddKeys(Times, YValues);
	InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>()[2]->AddKeys(Times, ZValues);
}

template <>
void FMovieSceneTrackPropertyRecorder<FVector3f>::ReduceKeys(UMovieSceneSection* InSection, float ReduceKeysTolerance)
{
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

	FKeyDataOptimizationParams Params;
	Params.bAutoSetInterpolation = true;
	Params.Tolerance = ReduceKeysTolerance;
	UE::MovieScene::Optimize(FloatChannels[0], Params);
	UE::MovieScene::Optimize(FloatChannels[1], Params);
	UE::MovieScene::Optimize(FloatChannels[2], Params);
}

template <>
FVector3f FMovieSceneTrackPropertyRecorder<FVector3f>::GetDefaultValue(UMovieSceneSection* InSection)
{
	FVector3f DefaultValue(0.f);

	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

	for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
	{
		if (FloatChannels[ChannelIndex]->GetNumKeys() > 0)
		{
			DefaultValue[ChannelIndex] = FloatChannels[ChannelIndex]->GetValues()[0].Value;
		}
		else if (FloatChannels[ChannelIndex]->GetDefault().IsSet())
		{
			DefaultValue[ChannelIndex] = FloatChannels[ChannelIndex]->GetDefault().GetValue();
		}
	}

	return DefaultValue;
}

template <>
void FMovieSceneTrackPropertyRecorder<FVector3f>::SetDefaultValue(UMovieSceneSection* InSection, const FVector3f& InDefaultValue)
{
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	FloatChannels[0]->SetDefault(InDefaultValue[0]);
	FloatChannels[1]->SetDefault(InDefaultValue[1]);
	FloatChannels[2]->SetDefault(InDefaultValue[2]);
}

template<>
bool FMovieSceneTrackPropertyRecorder<FVector3f>::OpenSerializer(const FString& InObjectName, const FName& InPropertyName, const FString& InTrackDisplayName, const FGuid& InGuid)
{
	FName SerializedType("Property");
	FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FPropertyFileHeader Header(TickResolution, SerializedType, InGuid);
	Header.PropertyName = InPropertyName;
	Header.TrackDisplayName = InTrackDisplayName;
	Header.PropertyType = ESerializedPropertyType::Vector3fType;
	
	FText Error;
	FString FileName = FString::Printf(TEXT("%s_%s_%s"), *(SerializedType.ToString()), *InObjectName, *(InPropertyName.ToString()));

	if (!Serializer.OpenForWrite(FileName, Header, Error))
	{
		UE_LOGF(PropertySerialization, Warning, "Error Opening Property File: Object: '%ls' Property '%ls' Error: '%ls'", *InObjectName, *InPropertyName.ToString(), *Error.ToString());
		return false;
	}
	return true;
}

template <>
bool FMovieSceneTrackPropertyRecorder<FVector3f>::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback)
{
	bool bFileExists = Serializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FPropertyFileHeader Header;

		if (Serializer.OpenForRead(FileName, Header, Error))
		{
			MovieSceneSection = AddSection(Header.TrackDisplayName, InMovieScene, Header.Guid, false);

			Serializer.GetDataRanges([this, InMovieScene, FileName, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
			{
				auto OnReadComplete = [this, InMovieScene, Header, InCompletionCallback]()
				{
					TArray<FPropertySerializedVector3fFrame> &InFrames = Serializer.ResultData;
					if (InFrames.Num() > 0)
					{
						FFrameRate InFrameRate = Header.TickResolution;
						TArray<FPropertyKey<FVector3f>> NewKeys;
						for (const FPropertySerializedVector3fFrame& SerializedFrame : InFrames)
						{
							const FPropertySerializedVector3f& Frame = SerializedFrame.Frame;
							FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
							FFrameTime FrameTime = FFrameRate::TransformTime(Frame.Time, InFrameRate, TickResolution);
							FFrameNumber CurrentFrame = FrameTime.FrameNumber;
							FPropertyKey<FVector3f> Key;
							Key.Time = CurrentFrame;
							Key.Value = Frame.Value;
							NewKeys.Add(Key);
							MovieSceneSection->ExpandToFrame(CurrentFrame);
						}
						AddKeysToSection(MovieSceneSection.Get(), NewKeys);
					}
					Serializer.Close();
					InCompletionCallback();
				}; //callback

				Serializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

			});
			return true;
		}
		else
		{
			Serializer.Close();
		}
	}
	return false;
}

template <>
bool FMovieSceneTrackPropertyRecorder<FVector3d>::ShouldAddNewKey(const FVector3d& InNewValue) const
{
	return !FMath::IsNearlyEqual(PreviousValue.X, InNewValue.X) || !FMath::IsNearlyEqual(PreviousValue.Y, InNewValue.Y) || !FMath::IsNearlyEqual(PreviousValue.Z, InNewValue.Z);
}

template <>
UMovieSceneSection* FMovieSceneTrackPropertyRecorder<FVector3d>::AddSection(const FString& TrackDisplayName, UMovieScene* InMovieScene, const FGuid& InGuid, bool bSetDefault)
{
	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneDoubleVectorTrack* Track = InMovieScene->FindTrack<UMovieSceneDoubleVectorTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneDoubleVectorTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}


	if (Track)
	{
		Track->SetNumChannelsUsed(3);
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneDoubleVectorSection* Section = Cast<UMovieSceneDoubleVectorSection>(Track->CreateNewSection());

		// We only set the track defaults when we're not loading from a serialized recording. Serialized recordings don't store channel defaults but will always store a
		// key on the first frame which will accomplish the same.
		if (bSetDefault)
		{
			TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = Section->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
			DoubleChannels[0]->SetDefault(PreviousValue.X);
			DoubleChannels[1]->SetDefault(PreviousValue.Y);
			DoubleChannels[2]->SetDefault(PreviousValue.Z);
		}

		Track->AddSection(*Section);
		return Section;
	}

	return nullptr;
}

template <>
void FMovieSceneTrackPropertyRecorder<FVector3d>::PostCreate(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer)
{
}

template <>
void FMovieSceneTrackPropertyRecorder<FVector3d>::AddKeysToSection(UMovieSceneSection* InSection, const TArray<FPropertyKey<FVector3d>>& InKeys)
{
	TArray<FFrameNumber> Times;
	Times.SetNum(InKeys.Num());
	TArray<FMovieSceneDoubleValue> XValues, YValues, ZValues;
	XValues.SetNum(InKeys.Num());
	YValues.SetNum(InKeys.Num());
	ZValues.SetNum(InKeys.Num());

	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		Times[Index] = InKeys[Index].Time;
		XValues[Index] = FMovieSceneDoubleValue(InKeys[Index].Value.X);
		YValues[Index] = FMovieSceneDoubleValue(InKeys[Index].Value.Y);
		ZValues[Index] = FMovieSceneDoubleValue(InKeys[Index].Value.Z);
	}
	InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>()[0]->AddKeys(Times, XValues);
	InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>()[1]->AddKeys(Times, YValues);
	InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>()[2]->AddKeys(Times, ZValues);
}

template <>
void FMovieSceneTrackPropertyRecorder<FVector3d>::ReduceKeys(UMovieSceneSection* InSection, float ReduceKeysTolerance)
{
	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

	FKeyDataOptimizationParams Params;
	Params.bAutoSetInterpolation = true;
	Params.Tolerance = ReduceKeysTolerance;
	UE::MovieScene::Optimize(DoubleChannels[0], Params);
	UE::MovieScene::Optimize(DoubleChannels[1], Params);
	UE::MovieScene::Optimize(DoubleChannels[2], Params);
}

template <>
FVector3d FMovieSceneTrackPropertyRecorder<FVector3d>::GetDefaultValue(UMovieSceneSection* InSection)
{
	FVector3d DefaultValue(0.f);

	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

	for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
	{
		if (DoubleChannels[ChannelIndex]->GetNumKeys() > 0)
		{
			DefaultValue[ChannelIndex] = DoubleChannels[ChannelIndex]->GetValues()[0].Value;
		}
		else if (DoubleChannels[ChannelIndex]->GetDefault().IsSet())
		{
			DefaultValue[ChannelIndex] = DoubleChannels[ChannelIndex]->GetDefault().GetValue();
		}
	}

	return DefaultValue;
}

template <>
void FMovieSceneTrackPropertyRecorder<FVector3d>::SetDefaultValue(UMovieSceneSection* InSection, const FVector3d& InDefaultValue)
{
	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	DoubleChannels[0]->SetDefault(InDefaultValue[0]);
	DoubleChannels[1]->SetDefault(InDefaultValue[1]);
	DoubleChannels[2]->SetDefault(InDefaultValue[2]);
}

template<>
bool FMovieSceneTrackPropertyRecorder<FVector3d>::OpenSerializer(const FString& InObjectName, const FName& InPropertyName, const FString& InTrackDisplayName, const FGuid& InGuid)
{
	FName SerializedType("Property");
	FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FPropertyFileHeader Header(TickResolution, SerializedType, InGuid);
	Header.PropertyName = InPropertyName;
	Header.TrackDisplayName = InTrackDisplayName;
	Header.PropertyType = ESerializedPropertyType::Vector3dType;
	
	FText Error;
	FString FileName = FString::Printf(TEXT("%s_%s_%s"), *(SerializedType.ToString()), *InObjectName, *(InPropertyName.ToString()));

	if (!Serializer.OpenForWrite(FileName, Header, Error))
	{
		UE_LOGF(PropertySerialization, Warning, "Error Opening Property File: Object: '%ls' Property '%ls' Error: '%ls'", *InObjectName, *InPropertyName.ToString(), *Error.ToString());
		return false;
	}
	return true;
}

template <>
bool FMovieSceneTrackPropertyRecorder<FVector3d>::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback)
{
	bool bFileExists = Serializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FPropertyFileHeader Header;

		if (Serializer.OpenForRead(FileName, Header, Error))
		{
			MovieSceneSection = AddSection(Header.TrackDisplayName, InMovieScene, Header.Guid, false);

			Serializer.GetDataRanges([this, InMovieScene, FileName, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
			{
				auto OnReadComplete = [this, InMovieScene, Header, InCompletionCallback]()
				{
					TArray<FPropertySerializedVector3dFrame> &InFrames = Serializer.ResultData;
					if (InFrames.Num() > 0)
					{
						FFrameRate InFrameRate = Header.TickResolution;
						TArray<FPropertyKey<FVector3d>> NewKeys;
						for (const FPropertySerializedVector3dFrame& SerializedFrame : InFrames)
						{
							const FPropertySerializedVector3d& Frame = SerializedFrame.Frame;
							FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
							FFrameTime FrameTime = FFrameRate::TransformTime(Frame.Time, InFrameRate, TickResolution);
							FFrameNumber CurrentFrame = FrameTime.FrameNumber;
							FPropertyKey<FVector3d> Key;
							Key.Time = CurrentFrame;
							Key.Value = Frame.Value;
							NewKeys.Add(Key);
							MovieSceneSection->ExpandToFrame(CurrentFrame);
						}
						AddKeysToSection(MovieSceneSection.Get(), NewKeys);
					}
					Serializer.Close();
					InCompletionCallback();
				}; //callback

				Serializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

			});
			return true;
		}
		else
		{
			Serializer.Close();
		}
	}
	return false;
}

template <>
bool FMovieSceneTrackPropertyRecorder<int32>::ShouldAddNewKey(const int32& InNewValue) const
{
	return InNewValue != PreviousValue;
}

template <>
UMovieSceneSection* FMovieSceneTrackPropertyRecorder<int32>::AddSection(const FString& TrackDisplayName, UMovieScene* InMovieScene, const FGuid& InGuid, bool bSetDefault)
{
	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneIntegerTrack* Track = InMovieScene->FindTrack<UMovieSceneIntegerTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneIntegerTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneIntegerTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneIntegerSection* Section = Cast<UMovieSceneIntegerSection>(Track->CreateNewSection());

		// We only set the track defaults when we're not loading from a serialized recording. Serialized recordings don't store channel defaults but will always store a
		// key on the first frame which will accomplish the same.
		if (bSetDefault)
		{
			FMovieSceneIntegerChannel* IntegerChannel = Section->GetChannelProxy().GetChannels<FMovieSceneIntegerChannel>()[0];
			IntegerChannel->SetDefault(PreviousValue);
		}

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieSceneTrackPropertyRecorder<int32>::PostCreate(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer)
{
}

template <>
void FMovieSceneTrackPropertyRecorder<int32>::AddKeysToSection(UMovieSceneSection* InSection, const TArray<FPropertyKey<int32>>& InKeys)
{
	TArray<FFrameNumber> Times;
	Times.SetNum(InKeys.Num());
	TArray<int32> Values;
	Values.SetNum(InKeys.Num());

	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		Times[Index] = InKeys[Index].Time;
		Values[Index] = InKeys[Index].Value;
	}

	InSection->GetChannelProxy().GetChannel<FMovieSceneIntegerChannel>(0)->AddKeys(Times, Values);
}

template <>
void FMovieSceneTrackPropertyRecorder<int32>::ReduceKeys(UMovieSceneSection* InSection, float ReduceKeysTolerance)
{
	// Reduce keys intentionally left blank
}

template <>
int32 FMovieSceneTrackPropertyRecorder<int32>::GetDefaultValue(UMovieSceneSection* InSection)
{
	FMovieSceneIntegerChannel* IntegerChannel = InSection->GetChannelProxy().GetChannels<FMovieSceneIntegerChannel>()[0];

	if (IntegerChannel->GetNumKeys() > 0)
	{
		return IntegerChannel->GetValues()[0];
	}
	else if (IntegerChannel->GetDefault().IsSet())
	{
		return IntegerChannel->GetDefault().GetValue();
	}

	return 0;
}

template <>
void FMovieSceneTrackPropertyRecorder<int32>::SetDefaultValue(UMovieSceneSection* InSection, const int32& InDefaultValue)
{
	return InSection->GetChannelProxy().GetChannel<FMovieSceneIntegerChannel>(0)->SetDefault(InDefaultValue);
}

template<>
bool FMovieSceneTrackPropertyRecorder<int32>::OpenSerializer(const FString& InObjectName, const FName& InPropertyName, const FString& InTrackDisplayName, const FGuid& InGuid)
{
	FName SerializedType("Property");
	FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FPropertyFileHeader Header(TickResolution, SerializedType, InGuid);
	Header.PropertyName = InPropertyName;
	Header.TrackDisplayName = InTrackDisplayName;
	Header.PropertyType = ESerializedPropertyType::IntegerType;
	FText Error;
	FString FileName = FString::Printf(TEXT("%s_%s_%s"), *(SerializedType.ToString()), *InObjectName, *(InPropertyName.ToString()));

	if (!Serializer.OpenForWrite(FileName, Header, Error))
	{
		UE_LOGF(PropertySerialization, Warning, "Error Opening Property File: Object '%ls' Property '%ls' Error: '%ls'", *InObjectName, *InPropertyName.ToString(), *Error.ToString());
		return false;
	}
	return true;
}

template <>
bool FMovieSceneTrackPropertyRecorder<int32>::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback)
{
	bool bFileExists = Serializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FPropertyFileHeader Header;

		if (Serializer.OpenForRead(FileName, Header, Error))
		{
			MovieSceneSection = AddSection(Header.TrackDisplayName, InMovieScene, Header.Guid, false);
			if (MovieSceneSection.IsValid())
			{
				Serializer.GetDataRanges([this, InMovieScene, FileName, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
				{
					auto OnReadComplete = [this, InMovieScene, Header, InCompletionCallback]()
					{
						TArray<FPropertySerializedIntegerFrame> &InFrames = Serializer.ResultData;
						if (InFrames.Num() > 0)
						{
							FFrameRate InFrameRate = Header.TickResolution;
							TArray<FPropertyKey<int32>> NewKeys;
							for (const FPropertySerializedIntegerFrame& SerializedFrame : InFrames)
							{
								const FPropertySerializedInteger& Frame = SerializedFrame.Frame;

								FFrameRate   TickResolution = InMovieScene->GetTickResolution();
								FFrameTime FrameTime = FFrameRate::TransformTime(Frame.Time, InFrameRate, TickResolution);
								FFrameNumber CurrentFrame = FrameTime.FrameNumber;
								FPropertyKey<int32> Key;
								Key.Time = CurrentFrame;
								Key.Value = Frame.Value;
								NewKeys.Add(Key);
								MovieSceneSection->ExpandToFrame(CurrentFrame);
							}
							AddKeysToSection(MovieSceneSection.Get(), NewKeys);
						}
						Serializer.Close();
						InCompletionCallback();
					}; //callback

					Serializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

				});
				return true;
			}
			return false;
		}
		else
		{
			Serializer.Close();
		}
	}
	return false;
}


template <>
bool FMovieSceneTrackPropertyRecorder<FString>::ShouldAddNewKey(const FString& InNewValue) const
{
	return InNewValue != PreviousValue;
}

template <>
UMovieSceneSection* FMovieSceneTrackPropertyRecorder<FString>::AddSection(const FString& TrackDisplayName, UMovieScene* InMovieScene, const FGuid& InGuid, bool bSetDefault)
{
	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneStringTrack* Track = InMovieScene->FindTrack<UMovieSceneStringTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneStringTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneStringTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneStringSection* Section = Cast<UMovieSceneStringSection>(Track->CreateNewSection());

		// We only set the track defaults when we're not loading from a serialized recording. Serialized recordings don't store channel defaults but will always store a
		// key on the first frame which will accomplish the same.
		if (bSetDefault)
		{
			FMovieSceneStringChannel* StringChannel = Section->GetChannelProxy().GetChannels<FMovieSceneStringChannel>()[0];
			StringChannel->SetDefault(PreviousValue);
		}

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieSceneTrackPropertyRecorder<FString>::PostCreate(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer)
{
}

template <>
void FMovieSceneTrackPropertyRecorder<FString>::AddKeysToSection(UMovieSceneSection* InSection, const TArray<FPropertyKey<FString>>& InKeys)
{
	TArray<FFrameNumber> Times;
	Times.SetNum(InKeys.Num());
	TArray<FString> Values;
	Values.SetNum(InKeys.Num());

	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		Times[Index] = InKeys[Index].Time;
		Values[Index] = InKeys[Index].Value;
	}

	InSection->GetChannelProxy().GetChannel<FMovieSceneStringChannel>(0)->AddKeys(Times, Values);
}

template <>
void FMovieSceneTrackPropertyRecorder<FString>::ReduceKeys(UMovieSceneSection* InSection, float ReduceKeysTolerance)
{
	// Reduce keys intentionally left blank
}

template <>
FString FMovieSceneTrackPropertyRecorder<FString>::GetDefaultValue(UMovieSceneSection* InSection)
{
	FMovieSceneStringChannel* StringChannel = InSection->GetChannelProxy().GetChannels<FMovieSceneStringChannel>()[0];

	if (StringChannel->GetNumKeys() > 0)
	{
		return StringChannel->GetData().GetValues()[0];
	}
	else if (StringChannel->GetDefault().IsSet())
	{
		return StringChannel->GetDefault().GetValue();
	}

	return FString();
}

template <>
void FMovieSceneTrackPropertyRecorder<FString>::SetDefaultValue(UMovieSceneSection* InSection, const FString& InDefaultValue)
{
	return InSection->GetChannelProxy().GetChannel<FMovieSceneStringChannel>(0)->SetDefault(InDefaultValue);
}

template<>
bool FMovieSceneTrackPropertyRecorder<FString>::OpenSerializer(const FString& InObjectName, const FName& InPropertyName, const FString& InTrackDisplayName, const FGuid& InGuid)
{
	FName SerializedType("Property");
	FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FPropertyFileHeader Header(TickResolution, SerializedType, InGuid);
	Header.PropertyName = InPropertyName;
	Header.TrackDisplayName = InTrackDisplayName;
	Header.PropertyType = ESerializedPropertyType::StringType;
	FText Error;
	FString FileName = FString::Printf(TEXT("%s_%s_%s"), *(SerializedType.ToString()), *InObjectName, *(InPropertyName.ToString()));

	if (!Serializer.OpenForWrite(FileName, Header, Error))
	{
		UE_LOGF(PropertySerialization, Warning, "Error Opening Property File: Object '%ls' Property '%ls' Error: '%ls'", *InObjectName, *InPropertyName.ToString(), *Error.ToString());
		return false;
	}
	return true;
}

template <>
bool FMovieSceneTrackPropertyRecorder<FString>::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback)
{
	bool bFileExists = Serializer.DoesFileExist(FileName);
	if (bFileExists)
	{
		FText Error;
		FPropertyFileHeader Header;

		if (Serializer.OpenForRead(FileName, Header, Error))
		{
			MovieSceneSection = AddSection(Header.TrackDisplayName, InMovieScene, Header.Guid, false);
			if (MovieSceneSection.IsValid())
			{
				Serializer.GetDataRanges([this, InMovieScene, FileName, Header, InCompletionCallback](uint64 InMinFrameId, uint64 InMaxFrameId)
				{
					auto OnReadComplete = [this, InMovieScene, Header, InCompletionCallback]()
					{
						TArray<FPropertySerializedStringFrame> &InFrames = Serializer.ResultData;
						if (InFrames.Num() > 0)
						{
							FFrameRate InFrameRate = Header.TickResolution;
							TArray<FPropertyKey<FString>> NewKeys;
							for (const FPropertySerializedStringFrame& SerializedFrame : InFrames)
							{
								const FPropertySerializedString& Frame = SerializedFrame.Frame;

								FFrameRate   TickResolution = InMovieScene->GetTickResolution();
								FFrameTime FrameTime = FFrameRate::TransformTime(Frame.Time, InFrameRate, TickResolution);
								FFrameNumber CurrentFrame = FrameTime.FrameNumber;
								FPropertyKey<FString> Key;
								Key.Time = CurrentFrame;
								Key.Value = Frame.Value;
								NewKeys.Add(Key);
								MovieSceneSection->ExpandToFrame(CurrentFrame);
							}
							AddKeysToSection(MovieSceneSection.Get(), NewKeys);
						}
						Serializer.Close();
						InCompletionCallback();
					}; //callback

					Serializer.ReadFramesAtFrameRange(InMinFrameId, InMaxFrameId, OnReadComplete);

				});
				return true;
			}
			return false;
		}
		else
		{
			Serializer.Close();
		}
	}
	return false;
}


template class FMovieSceneTrackPropertyRecorder<bool>;
template class FMovieSceneTrackPropertyRecorder<uint8>;
template class FMovieSceneTrackPropertyRecorder<float>;
template class FMovieSceneTrackPropertyRecorder<FColor>;
template class FMovieSceneTrackPropertyRecorder<FLinearColor>;
template class FMovieSceneTrackPropertyRecorder<FVector>;
template class FMovieSceneTrackPropertyRecorder<int32>;
template class FMovieSceneTrackPropertyRecorder<FString>;

template struct FSerializedProperty<bool>;
template struct FSerializedProperty<uint8>;
template struct FSerializedProperty<float>;
template struct FSerializedProperty<FColor>;
template struct FSerializedProperty<FLinearColor>;
template struct FSerializedProperty<FVector>;
template struct FSerializedProperty<int32>;
template struct FSerializedProperty<FString>;

