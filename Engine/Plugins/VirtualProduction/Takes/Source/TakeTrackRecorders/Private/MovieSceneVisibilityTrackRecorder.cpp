// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackRecorders/MovieSceneVisibilityTrackRecorder.h"
#include "TakesCoreLog.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "KeyParams.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Tracks/MovieSceneVisibilityTrack.h"
#include "MovieScene.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneTimeHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneVisibilityTrackRecorder)

static const FName ActorVisibilityTrackName = TEXT("bHidden");
static const FName ComponentVisibilityTrackName = TEXT("bHiddenInGame");

bool FMovieSceneVisibilityTrackRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return InObjectToRecord->IsA<AActor>() || InObjectToRecord->IsA<USceneComponent>();
}

bool FMovieSceneVisibilityTrackRecorderFactory::CanRecordProperty(UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const
{
	// This returns true for the visibility properties so that the generic bool recorder does not record them
	if (InPropertyToRecord->GetFName() == ActorVisibilityTrackName || InPropertyToRecord->GetFName() == ComponentVisibilityTrackName)
	{
		return true;
	}
	return false;
}

UMovieSceneTrackRecorder* FMovieSceneVisibilityTrackRecorderFactory::CreateTrackRecorderForObject() const
{
	return NewObject<UMovieSceneVisibilityTrackRecorder>();
}

void UMovieSceneVisibilityTrackRecorder::CreateTrackImpl()
{
	bSetFirstKey = true;

	UMovieSceneVisibilityTrack* VisibilityTrack = MovieScene->FindTrack<UMovieSceneVisibilityTrack>(ObjectGuid);
	if (!VisibilityTrack)
	{
		VisibilityTrack = MovieScene->AddTrack<UMovieSceneVisibilityTrack>(ObjectGuid);
	}
	else
	{
		VisibilityTrack->RemoveAllAnimationData();
	}

	if (VisibilityTrack)
	{
		USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectToRecord.Get());

		if (SceneComponent)
		{
			VisibilityTrack->SetPropertyNameAndPath(ComponentVisibilityTrackName, ComponentVisibilityTrackName.ToString());
		}
		else
		{
			VisibilityTrack->SetPropertyNameAndPath(ActorVisibilityTrackName, ActorVisibilityTrackName.ToString());
		}

		MovieSceneSection = Cast<UMovieSceneBoolSection>(VisibilityTrack->CreateNewSection());
		VisibilityTrack->AddSection(*MovieSceneSection);

		bWasVisible = false;
		if (SceneComponent)
		{
			bWasVisible = SceneComponent->IsVisible() && SceneComponent->IsRegistered();
		}
		else if (AActor* Actor = Cast<AActor>(ObjectToRecord.Get()))
		{
			bWasVisible = !Actor->IsHidden();
		}

		FMovieSceneBoolChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
		if (ensure(Channel))
		{
			Channel->SetDefault(bWasVisible);
		}
	}
}

void UMovieSceneVisibilityTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentTime)
{
	if (!MovieSceneSection.IsValid())
	{
		return;
	}

	if (ObjectToRecord.IsValid())
	{
		if (bSetFirstKey)
		{
			// Set the first key as the track will extrapolate backwards if the object was 
			// initially visible and then hidden
			bSetFirstKey = false;
			FMovieSceneBoolChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
			if (ensure(Channel))
			{
				if (MovieSceneSection->HasStartFrame())
				{
					Channel->GetData().AddKey(MovieSceneSection->GetInclusiveStartFrame(), bWasVisible);
				}
			}
		}

		FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame = CurrentTime.ConvertTo(TickResolution).FloorToFrame();

		MovieSceneSection->SetEndFrame(CurrentFrame);

		bool bVisible = IsObjectVisible();
		if (bVisible != bWasVisible)
		{
			FMovieSceneBoolChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
			if (ensure(Channel))
			{
				Channel->GetData().AddKey(CurrentFrame, bVisible);
			}
		}
		bWasVisible = bVisible;
	}
}


void UMovieSceneVisibilityTrackRecorder::FinalizeTrackImpl()
{
	if (!MovieSceneSection.IsValid())
	{
		return;
	}

	if (!ObjectToRecord.IsValid())
	{
		return;
	}

	FTrackRecorderSettings TrackRecorderSettings = OwningTakeRecorderSource->GetTrackRecorderSettings();

	// Reduce keys intentionally left blank

	if (TrackRecorderSettings.bRemoveRedundantTracks)
	{
		RemoveRedundantTracks();
	}
}

void UMovieSceneVisibilityTrackRecorder::RemoveRedundantTracks()
{
	FTrackRecorderSettings TrackRecorderSettings = OwningTakeRecorderSource->GetTrackRecorderSettings();

	FMovieSceneBoolChannel* Channel = MovieSceneSection.Get()->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
	if (!Channel)
	{
		return;
	}

	// If less than 1 key, remove the key and ensure the default value is the same
	if (Channel->GetData().GetValues().Num() > 1)
	{
		return;
	}

	bool bDefaultValue = Channel->GetData().GetValues().Num() == 1 ? Channel->GetData().GetValues()[0] : Channel->GetDefault().GetValue();
	Channel->GetData().Reset();
	Channel->SetDefault(bDefaultValue);

	// The section can be removed if this is a spawnable since the spawnable template should have the same default values
	bool bRemoveSection = !TrackRecorderSettings.bRecordToPossessable;

	// Otherwise the section can be removed if the CDO value is the same
	if (!bRemoveSection)
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectToRecord.Get()))
		{
			USceneComponent* DefaultSceneComponent = CastChecked<USceneComponent>(SceneComponent->GetClass()->GetDefaultObject());
			if (DefaultSceneComponent->IsVisible() == bDefaultValue)
			{
				bRemoveSection = true;
			}
		}
		else if (AActor* Actor = Cast<AActor>(ObjectToRecord.Get()))
		{
			AActor* DefaultActor = CastChecked<AActor>(Actor->GetClass()->GetDefaultObject());
			if (!DefaultActor->IsHidden() == bDefaultValue)
			{
				bRemoveSection = true;
			}
		}
	}

	if (bRemoveSection)
	{
		UMovieSceneTrack* MovieSceneTrack = CastChecked<UMovieSceneTrack>(MovieSceneSection->GetOuter());
		FString ObjectToRecordName = ObjectToRecord.IsValid() ? ObjectToRecord->GetName() : TEXT("Unnamed_Actor");

		UE_LOGF(LogTakesCore, Log, "Removed unused track (%ls) for (%ls)", *MovieSceneTrack->GetTrackName().ToString(), *ObjectToRecordName);

		MovieSceneTrack->RemoveSection(*MovieSceneSection.Get());
		MovieScene->RemoveTrack(*MovieSceneTrack);
	}
}

bool UMovieSceneVisibilityTrackRecorder::IsObjectVisible() const
{
	bool bVisible = false;
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectToRecord.Get()))
	{
		bVisible = SceneComponent->IsVisible() && SceneComponent->IsRegistered();
	}
	else if (AActor* Actor = Cast<AActor>(ObjectToRecord.Get()))
	{
		bVisible = !Actor->IsHidden();
	}
	
	return bVisible;
}

