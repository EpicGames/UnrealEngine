// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "TrackRecorders/MovieScene3DTransformTrackRecorder.h"
#include "TrackRecorders/MovieScene3DAttachTrackRecorder.h"
#include "TrackRecorders/MovieSceneActorReferenceTrackRecorder.h"
#include "TrackRecorders/MovieSceneParticleTrackRecorder.h"
#include "TrackRecorders/MovieSceneSpawnTrackRecorder.h"
#include "TrackRecorders/MovieSceneVisibilityTrackRecorder.h"
#include "TrackRecorders/MovieSceneAnimationTrackRecorder.h"

static const FName MovieSceneTrackRecorderFactoryName("MovieSceneTrackRecorderFactory");

class FTakeTrackRecordersModule : public IModuleInterface
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
		// Register built-in recorders
		IModularFeatures::Get().RegisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieScene3DTransformTrackRecorder);
		IModularFeatures::Get().RegisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneAnimationTrackRecorder);
		IModularFeatures::Get().RegisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieScene3DAttachTrackRecorder);
		IModularFeatures::Get().RegisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneParticleTrackRecorder);
		IModularFeatures::Get().RegisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneSpawnTrackRecorder);
		IModularFeatures::Get().RegisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneVisibilityTrackRecorder);
		IModularFeatures::Get().RegisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneActorReferenceTrackRecorder);
	}

	virtual void ShutdownModule() override
	{
		// Unregister built-in recorders
		IModularFeatures::Get().UnregisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieScene3DTransformTrackRecorder);
		IModularFeatures::Get().UnregisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneAnimationTrackRecorder);
		IModularFeatures::Get().UnregisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieScene3DAttachTrackRecorder);
		IModularFeatures::Get().UnregisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneParticleTrackRecorder);
		IModularFeatures::Get().UnregisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneSpawnTrackRecorder);
		IModularFeatures::Get().UnregisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneVisibilityTrackRecorder);
		IModularFeatures::Get().UnregisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneActorReferenceTrackRecorder);
	}

	FMovieScene3DTransformTrackRecorderFactory MovieScene3DTransformTrackRecorder;
	FMovieSceneAnimationTrackRecorderFactory MovieSceneAnimationTrackRecorder;
	FMovieScene3DAttachTrackRecorderFactory MovieScene3DAttachTrackRecorder;
	FMovieSceneParticleTrackRecorderFactory MovieSceneParticleTrackRecorder;
	FMovieSceneSpawnTrackRecorderFactory MovieSceneSpawnTrackRecorder;
	FMovieSceneVisibilityTrackRecorderFactory MovieSceneVisibilityTrackRecorder;
	FMovieSceneActorReferenceTrackRecorderFactory MovieSceneActorReferenceTrackRecorder;
};

IMPLEMENT_MODULE(FTakeTrackRecordersModule, TakeTrackRecorders)
