// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeTrackRecordersUtils.h"

#include "GameFramework/Actor.h"
#include "Components/SceneComponent.h"
#include "UObject/NameTypes.h"

namespace UE::TakeTrackRecordersUtils::Private
{

AActor* GetAttachment(AActor* InActor, FName& SocketName, FName& ComponentName)
{
	AActor* AttachedActor = nullptr;
	SocketName = NAME_None;
	ComponentName = NAME_None;
	if (InActor)
	{
		USceneComponent* RootComponent = InActor->GetRootComponent();
		if (RootComponent && RootComponent->GetAttachParent() != nullptr)
		{
			AttachedActor = RootComponent->GetAttachParent()->GetOwner();
			SocketName = RootComponent->GetAttachSocketName();
			ComponentName = RootComponent->GetAttachParent()->GetFName();
		}
	}

	return AttachedActor;
}

} // namespace UE::TakeTrackRecordersUtils::Private
