// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class AActor;
class FName;

namespace UE::TakeTrackRecordersUtils::Private
{

/**
 * Get the attachment owner of the given actor.
 * 
 * @param InActor The actor to find the attachment for.
 * @param SocketName Out name of the socket of the attachment.
 * @param ComponentName Out name of the component of the attachment. 
 * 
 * @return The attach owning actor.
 */
AActor* GetAttachment(AActor* InActor, FName& SocketName, FName& ComponentName);

} // namespace UE::TakeTrackRecordersUtils::Private
