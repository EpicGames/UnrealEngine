// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "Features/IModularFeature.h"

class ISequencer;
class IMovieSceneTrackRecorderHost;
class ULevelSequence;
class IMovieSceneCachedTrack;
class UMovieSceneTrackRecorderSettings;

/**  
 * Factory class interface that allows the recorder to determine what recorders to apply to 
 * actors/components/objects it is presented with.
 */
class IMovieSceneTrackRecorderFactory : public IModularFeature
{
public:
	/**
	 * Check whether an object can be recorded by this section recorder. If so then the actor recorder
	 * will call CreateTrackRecorder() to acquire a new instance to use in recording. This should only
	 * be implemented for recorders that record something about the Object that isn't tracked via a specific
	 * property. It is unlikely (but possible) that a factory should return true for both CanRecordObject
	 * and CanRecordProperty.
	 * 
	 * @param	InObjectToRecord	The object to check.
	 * @return true if the object can be recorded by this recorder
	 */
	virtual bool CanRecordObject(class UObject* InObjectToRecord) const = 0;

	/**
	 * Create a track recorder for this factory. Only called if CanRecordObject returns true.
	 * Should not return nullptr if CanRecordObject returned true.
	 *
	 * @return A new property recorder instance.
	 */
	virtual class UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const = 0;

	/**
	* Check whether or not the specific property on the given object can be recorded. If so, then the actor
	* recorder will call CreateTrackRecorderForProperty to create a new instance to use for recording. This
	* should only be implemented for recorders that record the specific property on the object and claim ownership
	* over recording that property.
	*
	* @param InObjectToRecord - The object to check to see if we can record.
	* @param InPropertyToRecord - The specific property to try and record from that object.
	* @return True if the specific object can be recorded 
	*/
	virtual bool CanRecordProperty(class UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const = 0;


	/**
	* Create a track recorder for this factory. Only called if CanRecordProperty returns true.
	* Should not return nullptr if CanRecordProperty returned true.
	*
	* @return A new property recorder instance.
	*/
	virtual class UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const = 0;

	virtual class UMovieSceneTrackRecorder* CreateTrackRecorderForCacheTrack(IMovieSceneCachedTrack* CachedTrack, const TObjectPtr<ULevelSequence>& Sequence, const TSharedPtr<ISequencer>& Sequencer) const { return nullptr; };

	/**
	* Get the human readable display name for the recorder. Used for debugging purposes to help identify which factory
	* is recording a given property.
	* @return Display name for use in the UI for debugging purposes.
	*/
	virtual FText GetDisplayName() const = 0;

	/**
	* Get the settings class for this factory. If this factory is used to record something on an Actor, an instance will be created and passed to created track recorders.
	*/
	virtual TSubclassOf<UMovieSceneTrackRecorderSettings> GetSettingsClass() const { return nullptr; }
	/**
	* Whether or not the created section recorder is serializable. If so when it creates a section we will serialize the section's information into a manifest.
	* @return an object used for user settings for this recorder
	*/
	virtual bool IsSerializable() const { return false; }

	/**
	*  Unique Name that's stored in the serialized manifest when it's recorded
	*/
	virtual FName GetSerializedType() const { return FName(); }
	
	/**
	 * Allow a factory to handle individual properties on an object, regardless of the property name.
	 */
	virtual bool CanHandlePropertyInstance() const { return false; }
	
	/**
	 * Allows a factory to determine if it wants to be created for a given host.
	 */
	virtual bool CanCreateTrackRecorderForHost(const IMovieSceneTrackRecorderHost* Host, UObject* ObjectToRecord) const { return true; }
};
