// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieSceneSequenceID.h"
#include "IMovieSceneTrackRecorderHost.generated.h"

class ULevelSequence;

UENUM(BlueprintType)
enum class ETakeRecorderAttachRecordBehaviour : uint8
{
	None UMETA(Tooltip = "Do not record attachments and instead record in world space."),
	Attachments UMETA(Tooltip = "Do not re-record parents, instead record only an attach track to the parent if it is controlled by a sequence or being recorded."),
	Hierarchy UMETA(Tooltip = "Record the full parent hierarchy including attachments."),
	ProjectDefault UMETA(Tooltip = "Use the project settings default value.")
};

USTRUCT()
struct FTakeRecorderPropertyTrackSettings
{
	GENERATED_BODY()

	/** Optional ActorComponent tag (when keying a component property). */
	UPROPERTY(config, EditAnywhere, Category = PropertyTrack)
	FString ComponentPath;

	/** Path to the keyed property within the Actor or ActorComponent. */
	UPROPERTY(config, EditAnywhere, Category = PropertyTrack)
	FString PropertyPath;
};

USTRUCT()
struct FTakeRecorderTrackSettings
{
	GENERATED_BODY()

	/** The Actor class to create movie scene tracks for. */
	UPROPERTY(config, EditAnywhere, Category = TrackSettings, meta = (MetaClass = "/Script/Engine.Actor"))
	FSoftClassPath MatchingActorClass;

	/** List of property names for which movie scene tracks will be created automatically. */
	UPROPERTY(config, EditAnywhere, Category = TrackSettings)
	TArray<FTakeRecorderPropertyTrackSettings> DefaultPropertyTracks;

	/** List of property names for which movie scene tracks will NOT be created automatically. */
	UPROPERTY(config, EditAnywhere, Category = TrackSettings)
	TArray<FTakeRecorderPropertyTrackSettings> ExcludePropertyTracks;
};


/** Generic track recorder settings used by the track recorders */
struct FTrackRecorderSettings
{
	bool bRecordToPossessable;
	bool bRemoveRedundantTracks;
	bool bReduceKeys;
	bool bSaveRecordedAssets;
	ETakeRecorderAttachRecordBehaviour AttachRecordBehaviour;

	float ReduceKeysTolerance;

	TArray<FTakeRecorderTrackSettings> DefaultTracks;
	TArray<FString> IncludeAnimationNames;
	TArray<FString> ExcludeAnimationNames;
	FTransform TransformOrigin;

	static bool IsDefaultPropertyTrack(UObject* InObjectToRecord, const FString& InPropertyPath, const TArray<FTakeRecorderTrackSettings>& DefaultTracks)
	{
		for (const FTakeRecorderTrackSettings& DefaultTrack : DefaultTracks)
		{
			UClass* MatchingActorClass = DefaultTrack.MatchingActorClass.ResolveClass();
			if (!MatchingActorClass || (!InObjectToRecord->IsA(MatchingActorClass) && !InObjectToRecord->GetOuter()->IsA(MatchingActorClass)))
			{
				continue;
			}

			for (const FTakeRecorderPropertyTrackSettings& PropertyTrackSetting : DefaultTrack.DefaultPropertyTracks)
			{
				if (InPropertyPath != PropertyTrackSetting.PropertyPath)
				{
					continue;
				}

				TArray<FString> ComponentNames;
				PropertyTrackSetting.ComponentPath.ParseIntoArray(ComponentNames, TEXT("."));

				if (ComponentNames.Num() == 0)
				{
					return true;
				}

				for (const FString& ComponentName : ComponentNames)
				{
					if (FindObjectFast<UObject>(InObjectToRecord, *ComponentName) != nullptr)
					{
						return true;
					}
					if (FindObjectFast<UObject>(InObjectToRecord->GetOuter(), *ComponentName) != nullptr)
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	static bool IsExcludePropertyTrack(UObject* InObjectToRecord, const FString& InPropertyPath, const TArray<FTakeRecorderTrackSettings>& DefaultTracks)
	{
		for (const FTakeRecorderTrackSettings& DefaultTrack : DefaultTracks)
		{
			UClass* MatchingActorClass = DefaultTrack.MatchingActorClass.ResolveClass();
			if (!MatchingActorClass || (!InObjectToRecord->IsA(MatchingActorClass) && !InObjectToRecord->GetOuter()->IsA(MatchingActorClass)))
			{
				continue;
			}

			for (const FTakeRecorderPropertyTrackSettings& PropertyTrackSetting : DefaultTrack.ExcludePropertyTracks)
			{
				// If property path is empty, consider it as an exclusion for all properties on this component
				if (PropertyTrackSetting.PropertyPath.IsEmpty())
				{
					return true;
				}

				if (InPropertyPath != PropertyTrackSetting.PropertyPath)
				{
					continue;
				}

				TArray<FString> ComponentNames;
				PropertyTrackSetting.ComponentPath.ParseIntoArray(ComponentNames, TEXT("."));

				for (const FString& ComponentName : ComponentNames)
				{
					if (FindObjectFast<UObject>(InObjectToRecord, *ComponentName) != nullptr)
					{
						return true;
					}
					if (FindObjectFast<UObject>(InObjectToRecord->GetOuter(), *ComponentName) != nullptr)
					{
						return true;
					}
				}
			}
		}

		return false;
	}
};

/** A class that hosts these track recorders and calls their functions. Allows a recorder to gain some limited context about other recorders. */
class IMovieSceneTrackRecorderHost
{
public:
	virtual ~IMovieSceneTrackRecorderHost() = default;
	/** 
	* Is the specified actor part of the current recording? This allows us to do some discovery for attachments and hierarchies.
	*
	* @param OtherActor	Actor to check
	* @return True if the specified actor is being recorded by another source.
	*/
	virtual bool IsOtherActorBeingRecorded(class AActor* OtherActor) const = 0;

	/**
	* Get the object binding for a given actor that is being recorded. An actor can either be a Possessable or a Spawnable but we only have pointers
	* to the original object being recorded. To solve this, we iterate through each actor being recorded and ask it what Guid it ended up with which
	* ends up abstracting away if it's a Spawnable or a Possessable.
	*
	* @param OtherActor Actor to look for.
	* @return A valid guid if the actor is being recorded otherwise an invalid guid.
	*/
	virtual FGuid GetRecordedActorGuid(class AActor* OtherActor) const = 0;

	/** Get the level sequence the other actor is coming from. */
	virtual ULevelSequence* GetLevelSequence(const AActor* OtherActor) { return nullptr; }
	
	/**
	* Get the sequence id of the level sequence the other actor is coming from.
	* Used for setting cross sequence bindings.
	* 
	* @param OtherActor Actor to look for.
	* @return The Sequence ID
	*/
	virtual FMovieSceneSequenceID GetLevelSequenceID(class AActor* OtherActor) = 0;

	/*
	 * Get generic track recorder settings.
	 * 
	 */
	virtual FTrackRecorderSettings GetTrackRecorderSettings() const = 0;

	/** Returns original transform that may get set when recording a skeletal mesh animation. Needed to correctly transform attached children.
	* @param OtherActor Actor to look for.
	* @return The FTransform applied to the actor when recording animation, that the attached actor may need to apply to themselves to orient correctly.
	*/
	virtual FTransform GetRecordedActorAnimationInitialRootTransform(class AActor* OtherActor) const = 0;

	/** Get the root level sequence being recorded into */
	virtual ULevelSequence* GetRootLevelSequence() const = 0;

	/**/
	virtual FMovieSceneSequenceID GetSequenceID() const = 0;
};
