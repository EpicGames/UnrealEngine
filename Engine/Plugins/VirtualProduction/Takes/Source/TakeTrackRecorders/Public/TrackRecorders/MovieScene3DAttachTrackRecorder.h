// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "GameFramework/Actor.h"
#include "MovieScene.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneTrackRecorder.h"
#include "IMovieSceneTrackRecorderFactory.h"
#include "MovieScene3DAttachTrackRecorder.generated.h"

#define UE_API TAKETRACKRECORDERS_API

class UMovieScene3DAttachTrack;
class UMovieScene3DAttachSection;

class FMovieScene3DAttachTrackRecorderFactory : public IMovieSceneTrackRecorderFactory
{
public:
	virtual ~FMovieScene3DAttachTrackRecorderFactory() = default;

	UE_API virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
	UE_API virtual UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const override;

	// Attachment isn't based on any particular property
	virtual bool CanRecordProperty(class UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const override { return false; }
	virtual UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const override { return nullptr; }

	virtual bool CanCreateTrackRecorderForHost(const IMovieSceneTrackRecorderHost* Host, UObject* ObjectToRecord) const override;

	virtual FText GetDisplayName() const override { return NSLOCTEXT("MovieScene3DAttachTrackRecorderFactory", "DisplayName", "Attach Track"); }
};

UCLASS(MinimalAPI, BlueprintType)
class UMovieScene3DAttachTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()
protected:
	// UMovieSceneTrackRecorder Interface
	UE_API virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) override;
	UE_API virtual void FinalizeTrackImpl() override;
	// ~UMovieSceneTrackRecorder Interface

private:
	/** Section to record to */
	TWeakObjectPtr<class UMovieScene3DAttachSection> MovieSceneSection;
	/** Guid in that section.. todo if more than one section put in arrays*/
	FGuid Guid;

	/** Track we are recording to */
	TWeakObjectPtr<class UMovieScene3DAttachTrack> AttachTrack;

	/** Track the actor we are attached to */
	TWeakObjectPtr<class AActor> ActorAttachedTo;

	/** Cached sequence ID for the attach target when it is not being recorded, set during RecordSampleImpl */
	TOptional<FMovieSceneSequenceID> CachedTargetSequenceID;
};

#undef UE_API
