// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieScene.h"
#include "MovieSceneTrackRecorder.h"
#include "IMovieSceneTrackRecorderFactory.h"
#include "Curves/RichCurve.h"
#include "Serializers/MovieSceneTransformSerialization.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "TakesCoreLog.h"
#include "MovieScene3DTransformTrackRecorder.generated.h"

#define UE_API TAKETRACKRECORDERS_API

class FMovieScene3DTransformTrackRecorder;
class UMovieScene3DTransformTrack;
class UMovieScene3DTransformTrack;
class UTakeRecorderActorSource;
struct FActorRecordingSettings;

DECLARE_LOG_CATEGORY_EXTERN(TransformSerialization, Verbose, All);

namespace UE::TakesCore
{
class FActorSequenceInformation;
} // namespace UE::TakesCore


class FMovieScene3DTransformTrackRecorderFactory : public IMovieSceneTrackRecorderFactory
{
public:
	virtual ~FMovieScene3DTransformTrackRecorderFactory() = default;

	UE_API virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
	UE_API virtual UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const override;

	// takerecorder-todo: This should also record "Transform" variable properties, because they can be marked as interp
	virtual bool CanRecordProperty(class UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const override { return false; }
	virtual UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const override { return nullptr; }

	virtual FText GetDisplayName() const override { return NSLOCTEXT("MovieScene3DTransformTrackRecorderFactory", "DisplayName", "Transform Track"); }

	virtual bool IsSerializable() const override { return true; }
	virtual FName GetSerializedType() const override { return FName("Transform"); }
};

/** Structure used to buffer up transform keys. Keys are inserted into tracks in FinalizeTrack() */
namespace UE::TrackRecorders
{
struct FBufferedTransformKeys
{

	TArray<FFrameNumber> Times;
	TArray<double> LocationX, LocationY, LocationZ;
	TArray<double> RotationX, RotationY, RotationZ;
	TArray<double> ScaleX, ScaleY, ScaleZ;
	//Currently used only when needed when collapsing when ProcessingAnimation()
	TArray<FQuat> CachedQuats;

	void Add(const FTransform& InTransform, FFrameNumber InKeyTime)
	{
		Times.Add(InKeyTime);
		Add(InTransform);
	}

	void Add(const FTransform& InTransform)
	{
		LocationX.Add(InTransform.GetTranslation().X);
		LocationY.Add(InTransform.GetTranslation().Y);
		LocationZ.Add(InTransform.GetTranslation().Z);

		FRotator WoundRoation = InTransform.Rotator();
		RotationX.Add(WoundRoation.Roll);
		RotationY.Add(WoundRoation.Pitch);
		RotationZ.Add(WoundRoation.Yaw);

		ScaleX.Add(InTransform.GetScale3D().X);
		ScaleY.Add(InTransform.GetScale3D().Y);
		ScaleZ.Add(InTransform.GetScale3D().Z);
	}


	void Reset()
	{
		Times.Reset();
		LocationX.Reset(); LocationY.Reset(); LocationZ.Reset();
		RotationX.Reset(); RotationY.Reset(); RotationZ.Reset();
		ScaleX.Reset(); ScaleY.Reset(); ScaleZ.Reset();
	}

	void Reserve(int32 Num, bool bReserveTime = false)
	{
		if (bReserveTime)
		{
			Times.Reserve(Num);
		}
		LocationX.Reserve(Num); LocationY.Reserve(Num); LocationZ.Reserve(Num);
		RotationX.Reserve(Num); RotationY.Reserve(Num); RotationZ.Reserve(Num);
		ScaleX.Reserve(Num); ScaleY.Reserve(Num); ScaleZ.Reserve(Num);
	}

	int32 Num() const
	{
		return Times.Num();
	}
	void CreateCachedQuats()
	{
		CachedQuats.SetNum(RotationX.Num());
		for (int32 Index = 0; Index < RotationX.Num(); ++Index)
		{
			CachedQuats[Index] = FQuat(FRotator(RotationY[Index], RotationZ[Index], RotationX[Index]));
		}
	}


	void GetValueFromIndex(int32 CurIndex, FVector& OutLocation, FQuat& OutQuat, FVector& OutScale)
	{
		OutLocation.X = LocationX[CurIndex];
		OutLocation.Y = LocationY[CurIndex];
		OutLocation.Z = LocationZ[CurIndex];

		OutQuat = CachedQuats[CurIndex];

		OutScale.X = ScaleX[CurIndex];
		OutScale.Y = ScaleY[CurIndex];
		OutScale.Z = ScaleZ[CurIndex];
	}

	//normalized 0-1
	float GetU(const FFrameNumber& Time, const FFrameNumber& PrevKeyTime, const FFrameNumber& NextKeyTime)
	{
		return  (float)(Time.Value - PrevKeyTime.Value) / (float)(NextKeyTime.Value - PrevKeyTime.Value);
	}


	//Get Values at the current Time using linear inteprolation. Uses CurIndex to do the linear search for next Time Value
	//as an optimization since we will be calling this function sequentially
	void GetValue(const FFrameNumber& Time, int32& CurIndex, FVector& OutLocation, FQuat& OutQuat, FVector& OutScale)
	{
		if (Time < Times[CurIndex])
		{
			if (CurIndex == 0)
			{
				GetValueFromIndex(CurIndex, OutLocation, OutQuat, OutScale);
			}
			else
			{
				FVector PrevLocation, PrevScale;
				FQuat PrevQuat;
				GetValueFromIndex(CurIndex - 1, PrevLocation, PrevQuat, PrevScale);
				GetValueFromIndex(CurIndex, OutLocation, OutQuat, OutScale);
				float u = GetU(Time, Times[CurIndex - 1], Times[CurIndex]);
				//cbb are there simd versions of linear interp?
				OutLocation.X = PrevLocation.X + (OutLocation.X - PrevLocation.X) * u;
				OutLocation.Y = PrevLocation.Y + (OutLocation.Y - PrevLocation.Y) * u;
				OutLocation.Z = PrevLocation.Z + (OutLocation.Z - PrevLocation.Z) * u;

				OutQuat = FQuat::Slerp(PrevQuat, OutQuat, u);

				OutScale.X = PrevScale.X + (OutScale.X - PrevScale.X) * u;
				OutScale.Y = PrevScale.Y + (OutScale.Y - PrevScale.Y) * u;
				OutScale.Z = PrevScale.Z + (OutScale.Z - PrevScale.Z) * u;
			}
		}
		else if (Time == Times[CurIndex])
		{
			GetValueFromIndex(CurIndex, OutLocation, OutQuat, OutScale);
			if (CurIndex < (Times.Num() - 1))
			{
				++CurIndex;
			}
		}

		else  // Time > Times[CurIndex]
		{
			if (CurIndex == Times.Num() - 1)
			{
				GetValueFromIndex(CurIndex, OutLocation, OutQuat, OutScale);
			}
			else
			{
				//shouldn't ever happen.
				UE_LOGF(LogTakesCore, Log, "Error When Collapsing Animation and Transform");

			}
		}
	}

	void PostMultTransform(const FTransform &Transform)
	{
		CreateCachedQuats(); //do stuff in quat space to avoid flips
		FVector OurLocation, OurScale;
		FQuat OurQuat;
		FTransform OurTransform;
		for (int32 CurIndex = 0; CurIndex < Times.Num(); ++CurIndex)
		{
			GetValueFromIndex(CurIndex, OurLocation, OurQuat, OurScale);
			OurTransform = FTransform(OurQuat, OurLocation, OurScale);

			OurTransform = OurTransform * Transform;
			//OurTransform = OurTransform * Transform.Inverse();
			LocationX[CurIndex] = OurTransform.GetTranslation().X;
			LocationY[CurIndex] = OurTransform.GetTranslation().Y;
			LocationZ[CurIndex] = OurTransform.GetTranslation().Z;

			FRotator WoundRoation = OurTransform.Rotator();
			RotationX[CurIndex] = WoundRoation.Roll;
			RotationY[CurIndex] = WoundRoation.Pitch;
			RotationZ[CurIndex] = WoundRoation.Yaw;

			ScaleX[CurIndex] = OurTransform.GetScale3D().X;
			ScaleY[CurIndex] = OurTransform.GetScale3D().Y;
			ScaleZ[CurIndex] = OurTransform.GetScale3D().Z;
		}
	}
	/**
	Collapse the passed in Additve onto myself, returning
	a new FBufferedTransformKeys
	*/
	FBufferedTransformKeys Collapse(FBufferedTransformKeys& AddLayer)
	{
		if (Num() >= 0 && AddLayer.Num() > 0)
		{
			FBufferedTransformKeys CollapsedTransforms;
			//First merge the two Times Array.
			TArray<FFrameNumber> NewTimes;
			NewTimes.Reserve(Num() + AddLayer.Num());
			int32 OurIndex = 0;
			int32 AddIndex = 0;
			while (OurIndex < Num() && AddIndex < AddLayer.Num())
			{
				if (Times[OurIndex] < AddLayer.Times[AddIndex])
				{
					NewTimes.Add(Times[OurIndex++]);
					if (OurIndex == Num()) //at end move over the AddLayer Times
					{
						while (AddIndex < AddLayer.Num())
						{
							NewTimes.Add(AddLayer.Times[AddIndex++]);
						}
					}
				}
				else
				{
					NewTimes.Add(AddLayer.Times[AddIndex++]);
					if (AddIndex == AddLayer.Num())//at end move our Times
					{
						while (OurIndex < Num())
						{
							NewTimes.Add(Times[OurIndex++]);
						}
					}
				}
			}
			CreateCachedQuats();
			AddLayer.CreateCachedQuats();
			OurIndex = 0;
			AddIndex = 0;
			FVector OurLocation, OurScale, AddLocation, AddScale;
			FQuat OurQuat, AddQuat;
			FTransform OurTransform, AddTransform;
			CollapsedTransforms.Reserve(NewTimes.Num());
			for (FFrameNumber Time : NewTimes)
			{
				GetValue(Time, OurIndex, OurLocation, OurQuat, OurScale);
				OurTransform = FTransform(OurQuat, OurLocation, OurScale);
				AddLayer.GetValue(Time, AddIndex, AddLocation, AddQuat, AddScale);
				AddTransform = FTransform(AddQuat, AddLocation, AddScale);
				OurTransform *= AddTransform;
				CollapsedTransforms.Add(OurTransform);
			}
			CollapsedTransforms.Times = MoveTemp(NewTimes);

			//with list of sorted times we go through and 
			return CollapsedTransforms;// will due NRVO or ::move failing that.
		}
		else
		{
			return *this;
		}
	}
};
}; // namespace UE::TrackRecorders

UCLASS(MinimalAPI, BlueprintType)
class UMovieScene3DTransformTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()
protected:
	UMovieScene3DTransformTrackRecorder(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, InterpolationMode(ERichCurveInterpMode::RCIM_Cubic)
	{
	}

	// UMovieSceneTrackRecorder Interface
	UE_API virtual void CreateTrackImpl() override;
	UE_API virtual void FinalizeTrackImpl() override;
	UE_API virtual void StopRecordingImpl() override;
	UE_API virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) override;
	virtual UMovieSceneSection* GetMovieSceneSection() const override { return Cast<UMovieSceneSection>(MovieSceneSection.Get()); }
	// ~UMovieSceneTrackRecorder Interface

public:
	virtual void SetSavedRecordingDirectory(const FString& InDirectory) override
	{
		TransformSerializer.SetLocalCaptureDir(InDirectory);
	}
	UE_API virtual bool LoadRecordedFile(const FString& InFileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap,  TFunction<void()> InCompletionCallback) override;
	UE_API void PostProcessAnimationData(class UMovieSceneAnimationTrackRecorder* AnimTrackRecorder);
private:
	UE_API bool ResolveTransformToRecord(FTransform& TransformToRecord);
	UE_API bool ShouldAddNewKey(const FTransform& TransformToRecord);
	UE_API void SetUpDefaultTransform();
private:
	/** Track to record to */
	TWeakObjectPtr<class UMovieScene3DTransformTrack> MovieSceneTrack;

	/** Section to record to */
	TWeakObjectPtr<UMovieScene3DTransformSection> MovieSceneSection;

	/** Buffer of transform keys. Keys are inserted into tracks in FinalizeTrack() */
	UE::TrackRecorders::FBufferedTransformKeys BufferedTransforms;

	/** The default transform this recording starts with */
	TOptional<FTransform> DefaultTransform;

	/** Flag indicating that some time while this recorder was active an attachment was also in place */
	bool bWasAttached;

	/** What Interpolation mode does the resulting Transform Track use? */
	ERichCurveInterpMode InterpolationMode;
	
	/** Previous Value And Frame used for checking previous values */
	FTransform PreviousValue;
	bool bSetFirstKey;
	TOptional<FFrameNumber> PreviousFrame;

	/**Serializer */
	FTransformSerializer TransformSerializer;

	/** Active attach parent. Can be nullptr */
	TWeakObjectPtr<USceneComponent> ActiveAttachParent;

	/** Cached sequence information about the attach parent. */
	TSharedPtr<class UE::TakesCore::FActorSequenceInformation> AttachParentSequenceInformation;

	/** An array of UMovieSceneSections that may need removal one finalize. Weak pointers as they may expire earlier. */
	TArray<TWeakObjectPtr<UMovieSceneSection>> SectionsToRemoveOnFinalize;
};

#undef UE_API
