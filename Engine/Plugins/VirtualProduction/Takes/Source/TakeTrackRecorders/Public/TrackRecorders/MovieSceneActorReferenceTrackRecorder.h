// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieSceneTrackRecorderFactory.h"
#include "MovieSceneTrackRecorder.h"
#include "Sections/MovieSceneActorReferenceSection.h"
#include "Sections/MovieSceneBoolSection.h"
#include "TrackInstancePropertyBindings.h"

#include "MovieSceneActorReferenceTrackRecorder.generated.h"

#define UE_API TAKETRACKRECORDERS_API


class FMovieSceneActorReferenceTrackRecorderFactory : public IMovieSceneTrackRecorderFactory
{
public:
	virtual ~FMovieSceneActorReferenceTrackRecorderFactory() = default;

	UE_API virtual bool CanRecordObject(UObject* InObjectToRecord) const override;
	UE_API virtual UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const override;

	UE_API virtual bool CanRecordProperty(UObject* InObjectToRecord, FProperty* InPropertyToRecord) const override;
	UE_API virtual UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const override;

	virtual FText GetDisplayName() const override { return NSLOCTEXT("MovieSceneActorReferenceTrackRecorderFactory", "DisplayName", "Actor Reference Track"); }
	
	virtual bool CanHandlePropertyInstance() const override { return true; }
};

UCLASS(MinimalAPI, BlueprintType)
class UMovieSceneActorReferenceTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()
	
public:
	UE_API void CreateBinding(const FName& InPropertyToRecord);
	
protected:
	// UMovieSceneTrackRecorder Interface
	UE_API virtual void CreateTrackImpl() override;
	UE_API virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) override;
	UE_API virtual void FinalizeTrackImpl() override;
	virtual UMovieSceneSection* GetMovieSceneSection() const override { return Cast<UMovieSceneSection>(MovieSceneSection.Get()); }
	// UMovieSceneTrackRecorder Interface

	UE_API void RemoveRedundantTracks();

private:
	/** Convert an actor to a reference key, locating or creating a binding as necessary, taking into account if the actor is a spawnable or not. */
	FMovieSceneActorReferenceKey ActorToReferenceKey(AActor* InActor) const;
	
	/** 
	 * Evaluate the current binding returning the actor value.
	 * @param OwningObject Optional object to use for binding evaluation, defaults to the owning take recorder source.
	 */
	AActor* EvaluateBinding(const UObject* OwningObject = nullptr) const;
	
private:
	/** Section to record to */
	TWeakObjectPtr<UMovieSceneActorReferenceSection> MovieSceneSection;
	
	/** Binding for this property */
	TSharedPtr<FTrackInstancePropertyBindings> PropertyBinding;
	
	/** Value of the last evaluated binding. */
	TWeakObjectPtr<AActor> LastBindingValue;
	
	/** Flag used to determine whether the first key needs to be set */
	bool bSetFirstKey;
	
	/** True after a recording has started and until it stops. */
	bool bIsRecording = false;
};

#undef UE_API
