// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMovieSceneTrackRecorderFactory.h"
#include "Templates/SharedPointer.h"
#include "MovieScene.h"
#include "MovieSceneTrackRecorder.h"
#include "MovieSceneAnimationTrackRecorderSettings.h"
#include "Animation/AnimSequence.h"
#include "Serializers/MovieSceneAnimationSerialization.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "IAnimRecorderInstance.h"

#include "MovieSceneAnimationTrackRecorder.generated.h"

#define UE_API TAKETRACKRECORDERS_API

class FMovieScene3DTransformTrackRecorder;
class UMovieScene3DTransformTrack;
class UMovieScene3DTransformTrack;
class UTakeRecorderActorSource;
struct FActorRecordingSettings;

class FMovieSceneAnimationTrackRecorderFactory : public IMovieSceneTrackRecorderFactory
{
public:
	virtual ~FMovieSceneAnimationTrackRecorderFactory() = default;

	UE_API virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
	UE_API virtual UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const override;

	virtual bool CanRecordProperty(class UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const override { return false; }
	virtual UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const override { return nullptr; }

	virtual FText GetDisplayName() const override { return NSLOCTEXT("MovieSceneAnimationTrackRecorderFactory", "DisplayName", "Animation Track"); }
	virtual TSubclassOf<UMovieSceneTrackRecorderSettings> GetSettingsClass() const override { return UMovieSceneAnimationTrackRecorderSettings::StaticClass(); }

	virtual bool IsSerializable() const override { return true; }
	virtual FName GetSerializedType() const override { return FName("Animation"); }
};

UCLASS(MinimalAPI, BlueprintType)
class UMovieSceneAnimationTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()
protected:
	// UMovieSceneTrackRecorder Interface
	UE_API virtual void CreateTrackImpl() override;
	UE_API virtual void FinalizeTrackImpl() override;
	UE_API virtual void CancelTrackImpl() override;
	UE_API virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) override;
	UE_API virtual void StopRecordingImpl() override;
	virtual void SetSavedRecordingDirectory(const FString& InDirectory) override
	{
		AnimationSerializer.SetLocalCaptureDir(InDirectory);
	}
	UE_API virtual bool LoadRecordedFile(const FString& InFileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback) override;
	virtual UMovieSceneSection* GetMovieSceneSection() const override { return Cast<UMovieSceneSection>(MovieSceneSection.Get()); }
	// UMovieSceneTrackRecorder Interface

public:
	bool RootWasRemoved() const { return bRootWasRemoved; }
	UE_API void RemoveRootMotion();
	UE_DEPRECATED(5.5, "Use the ProcessRecordedTimes method that takes a FProcessRecordedTimeParams struct.")
	UE_API void ProcessRecordedTimes(const FString& HoursName, const FString& MinutesName, const FString& SecondsName, const FString& FramesName, const FString& SubFramesName, const FString& SlateName, const FString& Slate);

	/** Apply time code data to the animation track. */
	UE_API void ProcessRecordedTimes(const FProcessRecordedTimeParams& InParams);

	UAnimSequence* GetAnimSequence() const { return AnimSequence.Get(); }
	USkeletalMesh* GetSkeletalMesh() const { return SkeletalMesh.Get(); }
	USkeletalMeshComponent* GetSkeletalMeshComponent() const { return SkeletalMeshComponent.Get(); }
	const FTransform& GetComponentTransform() const { return ComponentTransform; }
	const FTransform& GetInitialRootTransform() const { return InitialRootTransform; }

private:
	UE_API bool ResolveTransformToRecord(FTransform& TransformToRecord);
	UE_API void CreateAnimationAssetAndSequence(const AActor* Actor, const FDirectoryPath& AnimationDirectory);
private:
	/** Section to record to */
	TWeakObjectPtr<UMovieSceneSkeletalAnimationSection> MovieSceneSection;

	TWeakObjectPtr<class UAnimSequence> AnimSequence;

	TWeakObjectPtr<class USkeletalMeshComponent> SkeletalMeshComponent;

	TWeakObjectPtr<class USkeletalMesh> SkeletalMesh;

	/** Local transform of the component we are recording */
	FTransform ComponentTransform;

	/** Inverse we are using to zero out root motion */
	FTransform InitialRootTransform;

	bool bAnimationRecorderCreated;

	/** True if RecordSampleImpl tried to initialize the AnimationRecorder on the first sample
	    but ObjectToRecord's weak pointer was already null (typically because the source
	    SkeletalMeshComponent was destroyed between CreateTrack and the first sample). When set,
	    further initialization attempts are skipped and the failure is surfaced on record finish
	    to the user through the editor Message Log. */
	bool bRecordingFailedToStart = false;

	/** Animation Recorder */
	TSharedPtr<UE::AnimationRecording::IAnimRecorderInstance> AnimationRecorder;

	/** Previous Seconds to calc Delta used by Animation Recorder **/
	float  PreviousSeconds;

	/**Serializer */
	FAnimationSerializer AnimationSerializer;

	/** Root Was Removed*/
	bool bRootWasRemoved = true;
};

#undef UE_API
