// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "MovieSceneTrackRecorderSettings.h"
#include "Curves/RichCurve.h"
#include "TimecodeBoneMethod.h"

#include "MovieSceneAnimationTrackRecorderSettings.generated.h"

#define UE_API TAKETRACKRECORDERS_API

UCLASS(MinimalAPI, Abstract, BlueprintType, config=EditorSettings, DisplayName="Animation Recorder")
class UMovieSceneAnimationTrackRecorderEditorSettings : public UMovieSceneTrackRecorderSettings
{
	GENERATED_BODY()
public:
	UE_API UMovieSceneAnimationTrackRecorderEditorSettings(const FObjectInitializer& ObjInit);

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	/** Name of the recorded animation track. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Animation Recorder Settings", meta = (NamingTokens))
	FText AnimationTrackName;

	/** The name of the animation asset. 
	 * Supports any of the following format specifiers that will be substituted when a take is recorded:
	 * {day}       - The day of the timestamp for the start of the recording.
	 * {month}     - The month of the timestamp for the start of the recording.
	 * {year}      - The year of the timestamp for the start of the recording.
	 * {hour}      - The hour of the timestamp for the start of the recording.
	 * {minute}    - The minute of the timestamp for the start of the recording.
	 * {second}    - The second of the timestamp for the start of the recording.
	 * {take}      - The take number.
	 * {slate}     - The slate string.
	 * {actor}     - The name of the actor being recorded.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Animation Recorder Settings", meta = (NamingTokens))
	FString AnimationAssetName;

	/** The name of the subdirectory animations will be placed in. Leave this empty to place into the same directory as the sequence base path. 
	 * Supports any of the following format specifiers that will be substituted when a take is recorded:
	 * {day}       - The day of the timestamp for the start of the recording.
	 * {month}     - The month of the timestamp for the start of the recording.
	 * {year}      - The year of the timestamp for the start of the recording.
	 * {hour}      - The hour of the timestamp for the start of the recording.
	 * {minute}    - The minute of the timestamp for the start of the recording.
	 * {second}    - The second of the timestamp for the start of the recording.
	 * {take}      - The take number.
	 * {slate}     - The slate string.
	 * {actor}     - The name of the actor being recorded.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Animation Recorder Settings", meta = (NamingTokens))
	FString AnimationSubDirectory;
	
	/** Interpolation mode for the recorded keys. */
	UPROPERTY(EditAnywhere, Category = "Animation Recorder Settings", DisplayName = "Interpolation Mode")
	TEnumAsByte<ERichCurveInterpMode> InterpMode;

	/** Tangent mode for the recorded keys. */
	UPROPERTY(EditAnywhere, Category = "Animation Recorder Settings")
	TEnumAsByte<ERichCurveTangentMode> TangentMode;

	/** If true we remove the root animation and move it to a transform track, if false we leave it on the root bone in the anim sequence*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Animation Recorder Settings")
	bool bRemoveRootAnimation;

	/** If true we set the retarget source asset on the saved anim sequence, if not we do not*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Animation Recorder Settings")
	bool bSetRetargetSourceAsset;

	/** The method to record timecode values onto bones */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Animation Recorder Settings", meta = (ShowOnlyInnerProperties))
	FTimecodeBoneMethod TimecodeBoneMethod;
};

UCLASS(MinimalAPI, BlueprintType, config = EditorSettings, DisplayName = "Animation Recorder Settings")
class UMovieSceneAnimationTrackRecorderSettings : public UMovieSceneAnimationTrackRecorderEditorSettings
{
	GENERATED_BODY()
public:
	UMovieSceneAnimationTrackRecorderSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{
	}
};

#undef UE_API
