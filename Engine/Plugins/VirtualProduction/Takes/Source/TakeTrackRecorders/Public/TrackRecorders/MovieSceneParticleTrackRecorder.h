// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "IMovieSceneTrackRecorderFactory.h"
#include "Particles/ParticleSystemComponent.h"
#include "Tracks/MovieSceneParticleTrack.h"
#include "Sections/MovieSceneParticleSection.h"
#include "MovieSceneTrackRecorder.h"
#include "MovieSceneParticleTrackRecorder.generated.h"

#define UE_API TAKETRACKRECORDERS_API

class UMovieSceneParticleTrackRecorder;

class FMovieSceneParticleTrackRecorderFactory : public IMovieSceneTrackRecorderFactory
{
public:
	virtual ~FMovieSceneParticleTrackRecorderFactory() = default;

	UE_API virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
	UE_API virtual UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const override;

	// Particle Systems are entire components and you can't animate them as a property
	virtual bool CanRecordProperty(class UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const override { return false; }
	virtual UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const override { return nullptr; }
	
	virtual FText GetDisplayName() const override { return NSLOCTEXT("MovieSceneParticleTrackRecorderFactory", "DisplayName", "Particle System Track"); }
};

UCLASS(MinimalAPI, BlueprintType)
class UMovieSceneParticleTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()
public:
	UMovieSceneParticleTrackRecorder()
		: bWasTriggered(false)
		, PreviousState(EParticleKey::Activate)
	{
	}

protected:
	// UMovieSceneTrackRecorder Interface
	UE_API virtual void CreateTrackImpl() override;
	UE_API virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) override;
	virtual UMovieSceneSection* GetMovieSceneSection() const override { return MovieSceneSection.Get(); }
	// UMovieSceneTrackRecorder Interface

private:
	UFUNCTION()
	UE_API void OnTriggered(UParticleSystemComponent* Component, bool bActivating);
private:
	/** Object to record from */
	TWeakObjectPtr<class UParticleSystemComponent> SystemToRecord;

	/** Section to record to */
	TWeakObjectPtr<class UMovieSceneParticleSection> MovieSceneSection;

	bool bWasTriggered;

	EParticleKey PreviousState;
};

#undef UE_API
