// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/LazyObjectPtr.h"
#include "IMovieSceneTrackRecorderFactory.h"
#include "MovieSceneTrackRecorder.h"
#include "MovieSceneTrackPropertyRecorder.h"
#include "Serializers/MovieScenePropertySerialization.h"
#include "MovieScenePropertyTrackRecorder.generated.h"

#define UE_API TAKETRACKRECORDERS_API

// Forward Declare
class UObject;

class FMovieScenePropertyTrackRecorderFactory : public IMovieSceneTrackRecorderFactory
{
public:
	virtual ~FMovieScenePropertyTrackRecorderFactory() = default;

	// Property Track only records individual UProperties on an object.
	virtual bool CanRecordObject(UObject* InObjectToRecord) const override { return false; }
	virtual UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const override { return nullptr; }

	UE_API virtual bool CanRecordProperty(UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const override;
	UE_API virtual class UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const override;
	

	virtual FText GetDisplayName() const override { return NSLOCTEXT("MovieScenePropertyTrackTrackRecorderFactory", "DisplayName", "Property Track"); }

	virtual bool IsSerializable() const override { return true; }
	virtual FName GetSerializedType() const override { return FName("Property"); }

	UE_API UMovieSceneTrackRecorder* CreateTrackRecorderForPropertyEnum(ESerializedPropertyType ePropertyType, const FName& InPropertyToRecord) const;

};

UCLASS(MinimalAPI, BlueprintType)
class UMovieScenePropertyTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()
protected:
	// UMovieSceneTrackRecorder Interface
	UE_API virtual void CreateTrackImpl() override;
	UE_API virtual void SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame) override;
	UE_API virtual void FinalizeTrackImpl() override;
	UE_API virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) override;
	// ~UMovieSceneTrackRecorder Interface

	virtual void SetSavedRecordingDirectory(const FString& InDirectory) override
	{
		Directory = InDirectory;
	}
	UE_API virtual bool LoadRecordedFile(const FString& InFileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback) override;


public:
	/** Name of the specific property that we want to record. */
	FName PropertyToRecord;

	/** The property recorder for the specific property that we are recording. */
	TSharedPtr<class IMovieSceneTrackPropertyRecorder> PropertyRecorder;

	/** Cached Directory Name for serialization, used later when we create the PropertyRecorder*/
	FString Directory;
};

#undef UE_API
