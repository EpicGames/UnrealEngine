// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/LazyObjectPtr.h"
#include "IMovieSceneTrackRecorderFactory.h"
#include "MovieSceneTrackRecorder.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Serializers/MovieSceneSpawnSerialization.h"
#include "MovieSceneSpawnTrackRecorder.generated.h"

#define UE_API TAKETRACKRECORDERS_API

// Forward Declares
class UMovieSceneBoolTrack;

DECLARE_LOG_CATEGORY_EXTERN(SpawnSerialization, Verbose, All);

class FMovieSceneSpawnTrackRecorderFactory : public IMovieSceneTrackRecorderFactory
{
public:
	virtual ~FMovieSceneSpawnTrackRecorderFactory() = default;

	UE_API virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
	UE_API virtual UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const override;

	// Spawn Track is based on whether or not the recorded object still exists
	virtual bool CanRecordProperty(class UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const override { return false; }
	virtual UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const override { return nullptr; }

	virtual FText GetDisplayName() const override { return NSLOCTEXT("MovieSceneSpawnTrackRecorderFactory", "DisplayName", "Spawn Track"); }

	virtual bool IsSerializable() const override { return true; }
	virtual FName GetSerializedType() const override { return FName("Spawn"); }
};

UCLASS(MinimalAPI, BlueprintType)
class UMovieSceneSpawnTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()
protected:
	// UMovieSceneTrackRecorder Interface
	UE_API virtual void CreateTrackImpl() override;
	UE_API virtual void FinalizeTrackImpl() override;
	UE_API virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) override;
	virtual void SetSavedRecordingDirectory(const FString& InDirectory) override
	{
		SpawnSerializer.SetLocalCaptureDir(InDirectory);
	}
	UE_API virtual bool LoadRecordedFile(const FString& InFileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback) override;
	virtual UMovieSceneSection* GetMovieSceneSection() const override { return Cast<UMovieSceneSection>(MovieSceneSection.Get()); }
	// UMovieSceneTrackRecorder Interface

private:
	/** Section to record to */
	TWeakObjectPtr<class UMovieSceneBoolSection> MovieSceneSection;

	bool bWasSpawned;

	/**Serializer */
	FSpawnSerializer SpawnSerializer;
	bool bSetFirstKey;
};

#undef UE_API
