// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "MovieScene.h"
#include "TakesCoreLog.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "IMovieSceneTrackRecorderHost.h"
#include "Serializers/MovieScenePropertySerialization.h"

DECLARE_LOG_CATEGORY_EXTERN(PropertySerialization, Verbose, All);

/** Interface for a generic property recorder */
class IMovieSceneTrackPropertyRecorder
{
public:

	virtual ~IMovieSceneTrackPropertyRecorder() = default;
	virtual void Create(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer = false) = 0;
	virtual void SetSectionStartTimecode(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame) = 0;
	virtual void Record(UObject* InObjectToRecord, const FQualifiedFrameTime& CurrentTime) = 0;

	virtual void Finalize(UObject* InObjectToRecord) = 0;

	virtual void SetSavedRecordingDirectory(const FString& InDirectory) = 0;

	virtual bool LoadRecordedFile(const FString& InFileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap,  TFunction<void()> InCompletionCallback) { return false; }

protected:

	/** The Actor Source that owns us */
	IMovieSceneTrackRecorderHost * OwningTakeRecorderSource;
};

/** Helper struct for recording properties */
template <typename PropertyType>
struct FPropertyKey
{
	PropertyType Value;
	FFrameNumber Time;
};

/** Recorder for a simple property of type PropertyType */
template <typename PropertyType>
class FMovieSceneTrackPropertyRecorder : public IMovieSceneTrackPropertyRecorder
{
public:
	FMovieSceneTrackPropertyRecorder(const FTrackInstancePropertyBindings& InBinding)
		: Binding(InBinding)
	{
	}
	virtual ~FMovieSceneTrackPropertyRecorder() = default;

	virtual void Create(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer) override
	{
		OwningTakeRecorderSource = InRecordingHost;
		bSetFirstKey = true;
		if (InObjectToRecord)
		{
			PreviousValue = Binding.GetCurrentValue<PropertyType>(*InObjectToRecord);
		}

		if (!InObjectToRecord)
		{
			MovieSceneSection = nullptr;
		}
		else
		{
		#if WITH_EDITOR
			const FString TrackDisplayName = *Binding.GetProperty(*InObjectToRecord)->GetDisplayNameText().ToString();
		#else // WITH_EDITOR
			const FString TrackDisplayName = Binding.GetProperty(*InObjectToRecord)->GetName();
		#endif // WITH_EDITOR
			MovieSceneSection = AddSection(TrackDisplayName, InMovieScene, InGuid, bOpenSerializer);
			
			// Disable the section after creation so that the track can't be evaluated by Sequencer while recording.
			MovieSceneSection->SetIsActive(false);

			if (bOpenSerializer)
			{	
				OpenSerializer(InObjectToRecord->GetName(), Binding.GetPropertyName(), TrackDisplayName, InGuid);
			}

			PostCreate(InRecordingHost, InObjectToRecord, InMovieScene, InGuid, bOpenSerializer);
		}
	}

	virtual void SetSectionStartTimecode(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame) override
	{
		if (!MovieSceneSection.IsValid())
		{
			return;
		}

		MovieSceneSection->TimecodeSource = FMovieSceneTimecodeSource(InSectionStartTimecode);
		MovieSceneSection->ExpandToFrame(InSectionFirstFrame + FFrameNumber(1));
		MovieSceneSection->SetStartFrame(TRangeBound<FFrameNumber>::Inclusive(InSectionFirstFrame));
	}

	virtual void Record(UObject* InObjectToRecord, const FQualifiedFrameTime& CurrentTime) override
	{
		if (!MovieSceneSection.IsValid())
		{
			return;
		}

		if (InObjectToRecord != nullptr)
		{
			FFrameRate   TickResolution  = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
			FFrameNumber CurrentFrame    = CurrentTime.ConvertTo(TickResolution).FloorToFrame();
			MovieSceneSection->SetEndFrame(CurrentFrame);

			PropertyType NewValue = Binding.GetCurrentValue<PropertyType>(*InObjectToRecord);
			if (ShouldAddNewKey(NewValue))
			{
				if (PreviousFrame.IsSet())
				{
					FPropertyKey<PropertyType> Key;
					Key.Time = PreviousFrame.GetValue();
					Key.Value = PreviousValue;
					Keys.Add(Key);
					FSerializedProperty<PropertyType> Frame;
					Frame.Time = Key.Time;
					Frame.Value = Key.Value;
					Serializer.WriteFrameData(Serializer.FramesWritten, Frame);
				}

				FPropertyKey<PropertyType> Key;
				Key.Time = CurrentFrame;
				Key.Value = NewValue;
				Keys.Add(Key);
				FSerializedProperty<PropertyType> Frame;
				Frame.Time = Key.Time;
				Frame.Value = Key.Value;
				Serializer.WriteFrameData(Serializer.FramesWritten, Frame);

				PreviousValue = NewValue;
				PreviousFrame.Reset();
			}
			else
			{
				if (bSetFirstKey)
				{
					bSetFirstKey = false;
					FSerializedProperty<PropertyType> Frame;
					Frame.Time = CurrentFrame;
					Frame.Value = PreviousValue;
					Serializer.WriteFrameData(Serializer.FramesWritten, Frame);
				}
				PreviousFrame = CurrentFrame;
			}
		}
	}

	virtual void Finalize(UObject* InObjectToRecord) override
	{
		if (!MovieSceneSection.IsValid())
		{
			return;
		}

		// Enable when we finish recording the section
		MovieSceneSection->SetIsActive(true);

		AddKeysToSection(MovieSceneSection.Get(), Keys);

		FTrackRecorderSettings TrackRecorderSettings = OwningTakeRecorderSource->GetTrackRecorderSettings();

		if (TrackRecorderSettings.bReduceKeys)
		{
			ReduceKeys(MovieSceneSection.Get(), TrackRecorderSettings.ReduceKeysTolerance);
		}

		if (TrackRecorderSettings.bRemoveRedundantTracks)
		{
			RemoveRedundantTracks(MovieSceneSection.Get(), InObjectToRecord);
		}

		Serializer.Close();
	}

	virtual void RemoveRedundantTracks(UMovieSceneSection* InSection, UObject* InObjectToRecord)
	{
		if (!InObjectToRecord || !InSection)
		{
			return;
		}
		
		FTrackRecorderSettings TrackRecorderSettings = OwningTakeRecorderSource->GetTrackRecorderSettings();
			
		// The section can be removed if this is a spawnable since the spawnable template should have the same default values
		bool bRemoveSection = true;

		bool bExclude = FTrackRecorderSettings::IsExcludePropertyTrack(InObjectToRecord, Binding.GetPropertyPath(), TrackRecorderSettings.DefaultTracks);
		if (!bExclude)
		{
			// If any channel has more than 1 key, the track cannot be removed
			FMovieSceneChannelProxy& ChannelProxy = InSection->GetChannelProxy();
			for (const FMovieSceneChannelEntry& Entry : InSection->GetChannelProxy().GetAllEntries())
			{
				TArrayView<FMovieSceneChannel* const> Channels = Entry.GetChannels();

				for (int32 Index = 0; Index < Channels.Num(); ++Index)
				{
					if (Channels[Index]->GetNumKeys() > 1)
					{
						return;
					}
				}
			}

			// Assumes each channel is left with 1 or no keys, so the keys can be removed and the default value set
			PropertyType DefaultValue = GetDefaultValue(InSection);

			// Reset channels
			for (const FMovieSceneChannelEntry& Entry : InSection->GetChannelProxy().GetAllEntries())
			{
				TArrayView<FMovieSceneChannel* const> Channels = Entry.GetChannels();

				for (int32 Index = 0; Index < Channels.Num(); ++Index)
				{
					Channels[Index]->Reset();
				}
			}

			SetDefaultValue(InSection, DefaultValue);

			// If recording to a possessable, this section can only be removed if the CDO value is the same and it's not on the allow list of default property tracks
			if (TrackRecorderSettings.bRecordToPossessable)
			{
				bRemoveSection = false;

				UObject* DefaultObject = InObjectToRecord->GetClass()->GetDefaultObject();
				if (DefaultObject && Binding.GetCurrentValue<PropertyType>(*DefaultObject) == DefaultValue)
				{
					bRemoveSection = true;
				}

				if (bRemoveSection && FTrackRecorderSettings::IsDefaultPropertyTrack(InObjectToRecord, Binding.GetPropertyPath(), TrackRecorderSettings.DefaultTracks))
				{
					bRemoveSection = false;
				}
			}
		}

		if (bRemoveSection || bExclude)
		{
			UMovieSceneTrack* MovieSceneTrack = CastChecked<UMovieSceneTrack>(InSection->GetOuter());
			UMovieScene* MovieScene = CastChecked<UMovieScene>(MovieSceneTrack->GetOuter());

			UE_LOGF(LogTakesCore, Log, "Removed unused track (%ls) for (%ls)", *MovieSceneTrack->GetTrackName().ToString(), *InObjectToRecord->GetName());

			MovieSceneTrack->RemoveSection(*InSection);
			MovieScene->RemoveTrack(*MovieSceneTrack);
		}
	}


	virtual void SetSavedRecordingDirectory(const FString& InDirectory) override
	{
		Serializer.SetLocalCaptureDir(InDirectory);
	}
	virtual bool LoadRecordedFile(const FString& InFileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback) override;

private:
	/** 
	 * Helper function, specialized by type, used to check if we do capture-time filtering of keys based
	 * on previous values
	 */
	bool ShouldAddNewKey(const PropertyType& InNewValue) const;

	/** Helper function, specialized by type, used to add an appropriate section to the movie scene */
	class UMovieSceneSection* AddSection(const FString& TrackDisplayName, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bSetDefault = true);

	/** Helper function, specialized by type, after create */
	void PostCreate(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, bool bOpenSerializer = false);

	/** Helper function, specialized by type, used to add keys to the movie scene section at Finalize() time */
	void AddKeysToSection(UMovieSceneSection* InSection, const TArray<FPropertyKey<PropertyType>>& InKeys);

	/** Helper function, specialized by type, used to reduce keys */
	void ReduceKeys(UMovieSceneSection* InSection, float ReduceKeysTolerance);

	/** Get the default value of the track - if there's one key, the value of that key. Otherwise, the default value of the track. */
	PropertyType GetDefaultValue(UMovieSceneSection* InSection);

	/** Set the default value of the track */
	void SetDefaultValue(UMovieSceneSection* InSection, const PropertyType& InNewValue);

	/** Open the Serializer of the right PropertyType*/
	bool OpenSerializer(const FString& InObjectName, const FName& InPropertyName, const FString& InTrackDisplayName, const FGuid& InGuid);

private:
	/** Binding for this property */
	FTrackInstancePropertyBindings Binding;

	/** The keys that are being recorded */
	TArray<FPropertyKey<PropertyType>> Keys;

	/** Section we are recording */
	TWeakObjectPtr<class UMovieSceneSection> MovieSceneSection;

	/** Previous value we use to establish whether we should key */
	PropertyType PreviousValue;
	TOptional<FFrameNumber> PreviousFrame;
	bool bSetFirstKey;


	/** Serializer*/
	TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<PropertyType>> Serializer;
};
