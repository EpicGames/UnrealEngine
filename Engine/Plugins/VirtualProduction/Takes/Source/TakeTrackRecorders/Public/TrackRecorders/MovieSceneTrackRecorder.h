// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "IMovieSceneTrackRecorderHost.h"
#include "MovieSceneTrackRecorderSettings.h"
#include "MovieSceneTrackRecorder.generated.h"

// Forward Declare
class UMovieSceneTrackRecorderSettings;

UCLASS(MinimalAPI, BlueprintType, Transient, Abstract)
class UMovieSceneTrackRecorder : public UObject
{
	GENERATED_BODY()
public:
	/**
	* CreateTrack is called during PreRecording and should perform any clean up of old data, allocate new tracks if needed and create sections that start at a time of zero.
	* The call to PreRecording can take a significant amount of time when many sources are being recorded (due to the possibly large number of allocations) so the creation time
	* is not passed in. Instead Track recorders should implement SetSectionStartTimecodeImpl which will be called when StartRecording is called. StartRecording is called one after 
	* another and should be a non-blocking operation which will ensure all sources will start on the most up to date and in-sync timecode data.
	*/
	void CreateTrack(IMovieSceneTrackRecorderHost* InRecordingHost, UObject* InObjectToRecord, UMovieScene* InMovieScene, UMovieSceneTrackRecorderSettings* InSettingsObject, const FGuid& InObjectGuid)
	{
		OwningTakeRecorderSource = InRecordingHost;
		ObjectToRecord = InObjectToRecord;
		MovieScene = InMovieScene;
		Settings = InSettingsObject;
		ObjectGuid = InObjectGuid;

		CreateTrackImpl();

		// We'll also mark the section as inactive (so that it doesn't get evaluated)
		// This is done after the call to Impl because we need to give the track recorders a chance to create
		// sections in the first place. If this behavior is not desired for some edge case you can implement
		// SetSectionStartTimecodeImpl and revert this change.
		if (UMovieSceneSection* Section = GetMovieSceneSection())
		{
			Section->SetIsActive(false);
		}
	}

	/** 
	* This is called when Recording actually starts happening. Tracks and Sections should have already been created during CreateTrack so this call simply informs you of:
	*	- What the first frame should be for the Section you have created 
	*	- What Timecode you should embed in your Movie Scene section for syncing via the UI later.
	* This is implemented as a separate call from CreateTrack partially for blocking/sync reasons (in case the Timecode Source is pulled live and not the one cached for a given frame)
	* and partially so that this operation is explicit which will make it easier to follow the timecode logic as the implementations become more integrated with each other.
	* @param InSectionStartTimecode - The externally provided timecode at the time of this recording.
	* @param InSectionFirstFrame - The first frame of the section in sequence tick resolution.
	*/
	void SetSectionStartTimecode(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame)
	{
		// Cache our start timecode on the recorder for any track that uses it later.
		StartTimecode = FMovieSceneTimecodeSource(InSectionStartTimecode);

		// If the track recorder knows about it's section at this point in time (it should!) then we'll just set the start frames and timecode source
		// for them.
		if (UMovieSceneSection* Section = GetMovieSceneSection())
		{
			Section->TimecodeSource = FMovieSceneTimecodeSource(InSectionStartTimecode);

			// Ensure we're expanded to at least the next frame so that we don't set the start past the end
			// when we set the first frame.
			Section->ExpandToFrame(InSectionFirstFrame + FFrameNumber(1));
			Section->SetStartFrame(TRangeBound<FFrameNumber>::Inclusive(InSectionFirstFrame));
		}

		SetSectionStartTimecodeImpl(InSectionStartTimecode, InSectionFirstFrame);
	}

	/**
	* This is called after recording has finished for each track. This allows a track recorder to do any post-processing it may
	* require such as removing any sections that did not have any changes in them.
	*/
	void FinalizeTrack()
	{
		// If the section is valid for the Timecode we're going to re-enable it now that we've finished recording. A Track can still remove
		// this section in the Finalize implementation but consolidating a section being active/inactive consolidates a large number of repeat
		// implementations.
		if (UMovieSceneSection* Section = GetMovieSceneSection())
		{
			Section->SetIsActive(true);
		}

		FinalizeTrackImpl();
	}

	/**
	* This is called after recording has finished for each track. This allows a track recorder to do any post-processing it may
	* require such as removing any sections that did not have any changes in them.
	*/
	void CancelTrack()
	{
		CancelTrackImpl();

		if (UMovieSceneSection* Section = GetMovieSceneSection())
		{
			if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(Section->GetOuter()))
			{
				Track->RemoveSection(*Section);
			}
		}
	}

	/**
	* This is called each frame and specifies the qualified time that the sampled data should be recorded at. This is passed as 
	* a FQualifiedFrameTime for better handling of mixed resolution sequences as a user may have modified a sub-sequence to be
	* a different resolution than the parent sequence.
	*/
	void RecordSample(const FQualifiedFrameTime& CurrentTime)
	{
		RecordSampleImpl(CurrentTime);
	}

	/**
	* This is called when the user presses StopRecording. This should be a non-blocking operation as it is called on all sources 
	* one after another as quickly as possible so they all stop at the same time. This can be important for legacy systems that 
	* still rely on floating point time.
	*/
	void StopRecording()
	{
		StopRecordingImpl();
	}

	/**
	* Returns false if the track for this recorder has finished recording data
	*/
	virtual bool ShouldContinueRecording(const FQualifiedFrameTime& FrameTime) const
	{
		return true;
	}

	void InvalidateObjectToRecord()
	{
		ObjectToRecord = nullptr;
	}

	UObject* GetSourceObject() const
	{
		return ObjectToRecord.Get();
	}

	UMovieSceneTrackRecorderSettings* GetTrackRecorderSettings()
	{
		return  Settings.Get();
	}


	/**
	* Set the directory where the recorded values are saved
	* @param InDirectory the directory to save the data
	**/
	virtual void SetSavedRecordingDirectory(const FString& InDirectory) {};

	/**
	* Load the recorded file and create a section
	* @param InFileName The file to load
	* @param InMovieScene The Movie Scene to put the loaded track and section.
	* @return  Returns true if it was loaded and a section was created, false otherwise.
	*/
	virtual bool LoadRecordedFile(const FString& InFileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback) { return false; }

protected:
	virtual void CreateTrackImpl() {}
	virtual void SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame) {}
	virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) {}
	virtual void StopRecordingImpl() {}
	virtual void FinalizeTrackImpl() {}
	virtual void CancelTrackImpl() {}
	virtual UMovieSceneSection* GetMovieSceneSection() const { return nullptr; }

protected:
	/** Object to record from */
	TWeakObjectPtr<UObject> ObjectToRecord;

	/** Owning Object GUID in the Level Sequence */
	FGuid ObjectGuid;

	/** Movie Scene we're supposed to record to */
	TWeakObjectPtr<UMovieScene> MovieScene;

	/** The Actor Source that owns us */
	IMovieSceneTrackRecorderHost* OwningTakeRecorderSource;

	/** Settings object for the factory that created us. Can be nullptr if the factory has no settings object. */
	TWeakObjectPtr<UMovieSceneTrackRecorderSettings> Settings;

	/** The timecode source at the beginning of recording */
	FMovieSceneTimecodeSource StartTimecode;
};
