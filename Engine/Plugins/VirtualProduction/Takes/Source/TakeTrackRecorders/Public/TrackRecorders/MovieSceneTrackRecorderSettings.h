// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieSceneTrackRecorderSettings.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UMovieSceneTrackRecorderSettings : public UObject
{
	GENERATED_BODY()
};
