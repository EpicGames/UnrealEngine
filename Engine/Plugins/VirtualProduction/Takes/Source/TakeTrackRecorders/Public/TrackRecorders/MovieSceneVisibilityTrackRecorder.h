// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/LazyObjectPtr.h"
#include "IMovieSceneTrackRecorderFactory.h"
#include "MovieSceneTrackRecorder.h"
#include "Sections/MovieSceneBoolSection.h"
#include "MovieSceneVisibilityTrackRecorder.generated.h"

#define UE_API TAKETRACKRECORDERS_API

class UMovieSceneBoolTrack;

class FMovieSceneVisibilityTrackRecorderFactory : public IMovieSceneTrackRecorderFactory
{
public:
	virtual ~FMovieSceneVisibilityTrackRecorderFactory() = default;

	UE_API virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
	UE_API virtual UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const override;

	// Visibility is based on a different property for components and actors, and they're not marked as interp
	UE_API virtual bool CanRecordProperty(class UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const override;
	virtual UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const override { return nullptr; }

	virtual FText GetDisplayName() const override { return NSLOCTEXT("MovieSceneVisibilityTrackRecorderFactory", "DisplayName", "Visibility Track"); }
};

UCLASS(MinimalAPI, BlueprintType)
class UMovieSceneVisibilityTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()
protected:
	// UMovieSceneTrackRecorder Interface
	UE_API virtual void CreateTrackImpl() override;
	UE_API virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) override;
	UE_API virtual void FinalizeTrackImpl() override;
	virtual UMovieSceneSection* GetMovieSceneSection() const override { return Cast<UMovieSceneSection>(MovieSceneSection.Get()); }
	// UMovieSceneTrackRecorder Interface

	UE_API void RemoveRedundantTracks();
	UE_API bool IsObjectVisible() const;

private:
	/** Section to record to */
	TWeakObjectPtr<class UMovieSceneBoolSection> MovieSceneSection;

	/** Flag used to track visibility state and add keys when this changes */
	bool bWasVisible;

	/** Flag used to determine whether the first key needs to be set */
	bool bSetFirstKey;
};

#undef UE_API
