// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TakeTrackRecorders : ModuleRules
{
	public TakeTrackRecorders(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
                "Engine",
                "MovieScene",
                "MovieSceneTracks",
				"TakesCore",
				"TimeManagement",
				"SequenceRuntimeRecorder", // For FTimecodeBoneMethod and ETimecodeBoneMode
				"SerializedRecorderInterface",
				"LevelSequence"
            }
        );

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"SequenceRuntimeRecorder"
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"SequenceRecorder",
					"UnrealEd",
					"Sequencer"
				}
			);
		}
	}
}
