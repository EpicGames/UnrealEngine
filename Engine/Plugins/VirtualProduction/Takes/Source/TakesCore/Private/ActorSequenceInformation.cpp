// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorSequenceInformation.h"
#include "TakesUtils.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "MovieSceneCommonHelpers.h"
#include "Evaluation/MovieSceneEvaluationState.h"

namespace UE::TakesCore
{

FActorSequenceInformation::FActorSequenceInformation(ULevelSequence* InLevelSequence, AActor* InActor)
	: WeakTargetActor(InActor)
	, WeakTargetLevelSequence(InLevelSequence)
	, SharedPlaybackState(nullptr)
{
	if (InActor != nullptr && InLevelSequence != nullptr)
	{
	#if WITH_EDITOR
		if (TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SequencerSharedState = TakesUtils::FindSequencerSharedPlaybackState(InLevelSequence))
		{
			SharedPlaybackState = SequencerSharedState;
		}
		else
	#endif // WITH_EDITOR
		if (UWorld* WorldContext = InActor->GetTypedOuter<UWorld>())
		{
			SharedPlaybackState =
				MovieSceneHelpers::CreateTransientSharedPlaybackState(WorldContext,
					const_cast<ULevelSequence*>(InLevelSequence));
		}
	}

	Init();
}

FActorSequenceInformation::FActorSequenceInformation(ULevelSequence* InLevelSequence, AActor* InActor, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> InSharedPlaybackState)
	: WeakTargetActor(InActor)
	, WeakTargetLevelSequence(InLevelSequence)
	, SharedPlaybackState(InSharedPlaybackState)
{
	Init();
}

void FActorSequenceInformation::Init()
{
	TStrongObjectPtr<AActor> StrongTargetActor = WeakTargetActor.Pin();
	TStrongObjectPtr<ULevelSequence> StrongTargetLevelSequence = WeakTargetLevelSequence.Pin();

	if (!StrongTargetActor.IsValid() || !StrongTargetLevelSequence.IsValid())
	{
		return;
	}

	if (ResolveBindingFromSequence(StrongTargetActor.Get(), StrongTargetLevelSequence.Get()))
	{
		if (!SequenceID.IsValid() && !TryFindSequenceID(StrongTargetLevelSequence.Get()))
		{
			// Fallback to root.
			SequenceID = MovieSceneSequenceID::Root;
		}
	}

	if (!CachedObjectBindingGuid.IsValid())
	{
		InitFromCompiledData(StrongTargetLevelSequence.Get(), StrongTargetActor.Get());
	}
}

void FActorSequenceInformation::InitFromCompiledData(ULevelSequence* InLevelSequence, AActor* InActor)
{
	// The actor wasn't found in the root sequence — search all sub-sequences using
	// the pre-compiled hierarchy, which gives us the FMovieSceneSequenceID directly.
	UMovieSceneCompiledDataManager* CompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData();
	if (!CompiledDataManager)
	{
		return;
	}

	FMovieSceneCompiledDataID DataID = CompiledDataManager->FindDataID(InLevelSequence);
	const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(DataID);
	if (!Hierarchy)
	{
		return;
	}

	for (const TPair<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
	{
		if (ULevelSequence* SubSequence = Cast<ULevelSequence>(Pair.Value.GetSequence()))
		{
			if (ResolveBindingFromSequence(InActor, SubSequence))
			{
				if (!SequenceID.IsValid())
				{
					SequenceID = Pair.Key;
				}
				break;
			}
		}
	}
}

bool FActorSequenceInformation::ResolveBindingFromSequence(AActor* InActor, ULevelSequence* InLevelSequence)
{
	if (TOptional<FMovieSceneSpawnableAnnotation> SpawnableAnnotation = FMovieSceneSpawnableAnnotation::Find(InActor))
	{
		bIsControlledBySequence = true;
		CachedObjectBindingGuid = SpawnableAnnotation->ObjectBindingID;
		SequenceID = SpawnableAnnotation->SequenceID;
		WeakControllingSequence = SpawnableAnnotation->OriginatingSequence;
	}
	else
	{
		CachedObjectBindingGuid = TakesUtils::ResolveActorFromSequence(InActor, InLevelSequence, SharedPlaybackState.Pin());
		if (CachedObjectBindingGuid.IsValid())
		{
			WeakControllingSequence = InLevelSequence;
			bIsPossessable = IsPossessableBinding(InLevelSequence, CachedObjectBindingGuid);
			bIsControlledBySequence = bIsPossessable;
		}
	}

	return CachedObjectBindingGuid.IsValid();
}

bool FActorSequenceInformation::IsPossessableBinding(ULevelSequence* InLevelSequence, const FGuid& InBindingGuid) const
{
	if (InLevelSequence == nullptr || !InBindingGuid.IsValid())
	{
		return false;
	}

	UMovieScene* MovieScene = InLevelSequence->GetMovieScene();

	/**
	 * If this actor has an existing binding in the recording sequence but is not yet
	 * flagged as a Sequencer-managed actor, flag it now. This covers actors that were keyframed
	 * via the Sequencer panel in the source sequence (possessables without the SequencerActor tag).
	 * Flagging here ensures CleanExistingDataFromSequence is skipped and section recorders are
	 * suppressed for actors that already have authoritative Sequencer data.
	 *
	 * Spawnable bindings (Take Recorder or Sequencer-created) are excluded from being flagged.
	 * In the modern binding system, spawnables are stored as possessables with a custom binding
	 * of type UMovieSceneSpawnableBindingBase. Without this check, re-recording into a previous
	 * Take Recorder recording would incorrectly flag the source.
	 */
	const FMovieSceneBinding* ExistingBinding = MovieScene->FindBinding(InBindingGuid);
	if (!ExistingBinding)
	{
		return false;
	}

	bool bIsSpawnableBinding = false;

	// Modern path: check for spawnable custom binding
	if (const FMovieSceneBindingReferences* BindingRefs = InLevelSequence->GetBindingReferences())
	{
		if (const UMovieSceneCustomBinding* CustomBinding = BindingRefs->GetCustomBinding(InBindingGuid, 0))
		{
			bIsSpawnableBinding = CustomBinding->IsA<UMovieSceneSpawnableBindingBase>();
		}
	}

	// Legacy path: check old spawnable storage
	if (!bIsSpawnableBinding)
	{
		bIsSpawnableBinding = (MovieScene->FindSpawnable(InBindingGuid) != nullptr);
	}

	return !bIsSpawnableBinding;
}

bool FActorSequenceInformation::TryFindSequenceID(ULevelSequence* ForSequence)
{
	if (TSharedPtr<const UE::MovieScene::FSharedPlaybackState> PinnedPlaybackState = SharedPlaybackState.Pin())
	{
		if (FMovieSceneEvaluationState* EvalState = PinnedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
		{
			if (FMovieSceneSequenceID EvalSequenceID = EvalState->FindSequenceId(ForSequence); EvalSequenceID.IsValid())
			{
				SequenceID = EvalSequenceID;
				return true;
			}
		}
	}

	// Fallback to the compiled data manager.
	// Search the root sequence's hierarchy for ForSequence to find the FMovieSceneSequenceID it
	// was assigned when compiled into the root. If ForSequence IS the root, its ID is always Root.
	TStrongObjectPtr<ULevelSequence> StrongRoot = WeakTargetLevelSequence.Pin();
	if (!StrongRoot.IsValid())
	{
		return false;
	}

	if (ForSequence == StrongRoot.Get())
	{
		SequenceID = MovieSceneSequenceID::Root;
		return true;
	}

	if (UMovieSceneCompiledDataManager* CompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData())
	{
		FMovieSceneCompiledDataID DataID = CompiledDataManager->FindDataID(StrongRoot.Get());
		if (const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(DataID))
		{
			for (const TPair<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
			{
				if (Pair.Value.GetSequence() == ForSequence)
				{
					SequenceID = Pair.Key;
					return true;
				}
			}
		}
	}

	return false;
}

} // namespace UE::TakesCore
