// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITakeRecorderSourcesManager.h"
#include "TakeRecorderSourcesManagerImpl.h"
#include "Engine/Engine.h"

ITakeRecorderSourcesManager* ITakeRecorderSourcesManager::Get()
{
	return GEngine->GetEngineSubsystem<UTakeRecorderSourcesManagerImpl>();
}

ITakeRecorderSourcesManager& ITakeRecorderSourcesManager::GetChecked()
{
	ITakeRecorderSourcesManager* Ptr = Get();
	check(Ptr);
	return *Ptr;
}
