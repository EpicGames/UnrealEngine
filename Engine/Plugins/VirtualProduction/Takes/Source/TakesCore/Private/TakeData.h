// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This whole file is only active in an editor context, and inherits from an editor type.
// It is currently not needed at runtime, but if it is, then this will need revisiting to
// add that same functionality.
#if WITH_EDITOR

#include "TakesCoreLog.h"
#include "TakeMetaData.h"
#include "TakesCoreBlueprintLibrary.h"

#include "LevelSequence.h"
#include "MovieSceneSequence.h"
#include "MovieSceneToolsModule.h"
#include "Sections/MovieSceneSubSection.h"

#include "AssetRegistry/AssetRegistryModule.h"

class FTakesCoreTakeData : public IMovieSceneToolsTakeData
{
public:
	virtual ~FTakesCoreTakeData() = default;

	virtual bool GatherTakes(const UMovieSceneSection* Section, TArray<FAssetData>& AssetData, uint32& OutCurrentTakeNumber)
	{
		const UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
		if (!SubSection)
		{
			return false;
		}

		ULevelSequence* Sequence = Cast<ULevelSequence>(SubSection->GetSequence());
		if (!Sequence)
		{
			return false;
		}

		UTakeMetaData* TakeMetaData = Sequence->FindMetaData<UTakeMetaData>();
		if (!TakeMetaData)
		{
			return false;
		}

		TArray<FAssetData> TakeDatas = UTakesCoreBlueprintLibrary::FindTakes(TakeMetaData->GetSlate());
		for (FAssetData TakeData : TakeDatas)
		{
			FAssetDataTagMapSharedView::FFindTagResult TakeNumberTag = TakeData.TagsAndValues.FindTag(UTakeMetaData::AssetRegistryTag_TakeNumber);

			int32 ThisTakeNumber = 0;
			if (TakeNumberTag.IsSet() && LexTryParseString(ThisTakeNumber, *TakeNumberTag.GetValue()))
			{
				AssetData.Add(TakeData);
			}
		}

		OutCurrentTakeNumber = TakeMetaData->GetTakeNumber();

		return true;
	}

	virtual bool GetTakeNumber(const UMovieSceneSection* Section, FAssetData AssetData, uint32& OutTakeNumber)
	{
		const UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
		if (!SubSection)
		{
			return false;
		}

		ULevelSequence* Sequence = Cast<ULevelSequence>(SubSection->GetSequence());
		if (!Sequence)
		{
			return false;
		}

		UTakeMetaData* TakeMetaData = Sequence->FindMetaData<UTakeMetaData>();
		if (!TakeMetaData)
		{
			return false;
		}

		TArray<FAssetData> TakeDatas = UTakesCoreBlueprintLibrary::FindTakes(TakeMetaData->GetSlate());
		for (FAssetData TakeData : TakeDatas)
		{
			if (TakeData == AssetData)
			{
				FAssetDataTagMapSharedView::FFindTagResult TakeNumberTag = TakeData.TagsAndValues.FindTag(UTakeMetaData::AssetRegistryTag_TakeNumber);

				int32 ThisTakeNumber = 0;
				if (TakeNumberTag.IsSet() && LexTryParseString(ThisTakeNumber, *TakeNumberTag.GetValue()))
				{
					OutTakeNumber = ThisTakeNumber;
					return true;
				}
			}
		}

		return false;
	}

	virtual bool SetTakeNumber(const UMovieSceneSection* Section, uint32 InTakeNumber)
	{
		const UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
		if (!SubSection)
		{
			return false;
		}

		ULevelSequence* Sequence = Cast<ULevelSequence>(SubSection->GetSequence());
		if (!Sequence)
		{
			return false;
		}

		UTakeMetaData* TakeMetaData = Sequence->FindMetaData<UTakeMetaData>();
		if (!TakeMetaData)
		{
			return false;
		}

		bool bWasLocked = TakeMetaData->IsLocked();
		TakeMetaData->Unlock();
		TakeMetaData->SetTakeNumber(InTakeNumber);
		if (bWasLocked)
		{
			TakeMetaData->Lock();
		}

		return true;
	}
};

#endif // WITH_EDITOR
