// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeMetaData.h"

#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "ITakeRecorderNamingTokensModule.h"
#include "NamingTokenData.h"
#include "NamingTokens/TakeRecorderNamingTokensContext.h"
#include "NamingTokensEngineSubsystem.h"
#include "TakePreset.h"
#include "TakesCoreBlueprintLibrary.h"
#include "TakesCoreLog.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "Utils/NamingTokenUtils.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeMetaData)

const FName UTakeMetaData::AssetRegistryTag_Slate       = "TakeMetaData_Slate";
const FName UTakeMetaData::AssetRegistryTag_TakeNumber  = "TakeMetaData_TakeNumber";
const FName UTakeMetaData::AssetRegistryTag_Timestamp = "TakeMetaData_Timestamp";
const FName UTakeMetaData::AssetRegistryTag_TimecodeIn = "TakeMetaData_TimecodeIn";
const FName UTakeMetaData::AssetRegistryTag_TimecodeOut = "TakeMetaData_TimecodeOut";
const FName UTakeMetaData::AssetRegistryTag_Description = "TakeMetaData_Description";
const FName UTakeMetaData::AssetRegistryTag_LevelPath   = "TakeMetaData_LevelPath";

UTakeMetaData::UTakeMetaData(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, Timestamp(0)
{
	TakeNumber = 1;
	bIsLocked = false;
	bFrameRateFromTimecode = true;
	if (FApp::GetCurrentFrameTime())
	{
		FrameRate = FApp::GetTimecodeFrameRate();
	}
}

UTakeMetaData* UTakeMetaData::GetConfigInstance()
{
	static UTakeMetaData* ConfigInstance = NewObject<UTakeMetaData>(GetTransientPackage(), "DefaultTakeMetaData", RF_MarkAsRootSet);
	return ConfigInstance;
}

UTakeMetaData* UTakeMetaData::CreateFromDefaults(UObject* Outer, FName Name)
{
	if (Name != NAME_None)
	{
		check(!FindObject<UObject>(Outer, *Name.ToString()));
	}

	return CastChecked<UTakeMetaData>(StaticDuplicateObject(GetConfigInstance(), Outer, Name, RF_NoFlags));
}

namespace UE::TakeRecorder::MetaDataHelpers::Private
{
	/** A copy of the last metadata. */
	static TStrongObjectPtr<UTakeMetaData>& GetLastMetaData()
	{
		static TStrongObjectPtr<UTakeMetaData> CachedMetaData;
		return CachedMetaData;
	}

	/** Weak reference to the current metadata. */
	static TWeakObjectPtr<UTakeMetaData>& GetCurrentWeakMetaData()
	{
		static TWeakObjectPtr<UTakeMetaData> CachedWeakMetaData;
		return CachedWeakMetaData;
	}

	/** Updates the cached metadata. */
	static void UpdateCachedMetaData(UTakeMetaData* NewMetaData)
	{
		if (NewMetaData)
		{
			GetCurrentWeakMetaData() = NewMetaData;
			GetLastMetaData().Reset();
		}
		else
		{
			// We want to copy the metadata object so the original can be cleaned up if the level sequence needs to be removed.
			UTakeMetaData* MetaDataCopy = GetCurrentWeakMetaData().IsValid() ?
				DuplicateObject(GetCurrentWeakMetaData().Get(), GetTransientPackage()) : nullptr;
			GetLastMetaData().Reset(MetaDataCopy);
		}
	}
}

UTakeMetaData* UTakeMetaData::GetMostRecentMetaData()
{
	using namespace UE::TakeRecorder::MetaDataHelpers::Private;
	if (UTakeMetaData* CurrentMetaData = GetCurrentWeakMetaData().Get())
	{
		return CurrentMetaData;
	}

	return GetLastMetaData().Get();
}

void UTakeMetaData::SetMostRecentMetaData(UTakeMetaData* InMetaData)
{
	using namespace UE::TakeRecorder::MetaDataHelpers::Private;
	UpdateCachedMetaData(InMetaData);
}

bool UTakeMetaData::Recorded() const
{
	return bool( GetTimestamp() != FDateTime(0) );
}

bool UTakeMetaData::IsLocked() const
{
	return bIsLocked;
}

void UTakeMetaData::Lock()
{
	bIsLocked = true;
}

void UTakeMetaData::Unlock()
{
	bIsLocked = false;
}

FString UTakeMetaData::GenerateAssetPath(const FString& PathFormatString, UTakeRecorderNamingTokensContext* InContext) const
{
	const FNamingTokenResultData Result = ProcessTokens(FText::FromString(PathFormatString), InContext);
	return Result.EvaluatedText.ToString();
}

bool UTakeMetaData::TryGenerateRootAssetPath(const FString& PathFormatString, FString& OutGeneratedAssetPath,
	FText* OutErrorMessage, UTakeRecorderNamingTokensContext* InContext) const
{
	const FString TakeNameToken = TEXT("takeName");
	if (UE::NamingTokens::Utils::IsTokenInString(TakeNameToken, PathFormatString))
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage =
				FText::Format(NSLOCTEXT("TakeMetaData", "ErrorGenerateRootAssetPath_InvalidToken",
					"Token '{0}' cannot be present in this asset path {1}."),
					FText::FromString(TakeNameToken), FText::FromString(PathFormatString));
		}
		return false;
	}

	OutGeneratedAssetPath = GenerateAssetPath(PathFormatString, InContext);
	return true;
}

FNamingTokenResultData UTakeMetaData::ProcessTokens(const FText& InText, UTakeRecorderNamingTokensContext* InContext) const
{
	if (!ensure(GEngine))
	{
		return FNamingTokenResultData();
	}

	if (!InContext)
	{
		InContext = NewObject<UTakeRecorderNamingTokensContext>();
	}
	InContext->TakeMetaData = this;

	FNamingTokenFilterArgs NamingTokenFilters;
	NamingTokenFilters.AdditionalNamespacesToInclude = { ITakeRecorderNamingTokensModule::GetTakeRecorderNamespace() };
	return GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>()->EvaluateTokenText(InText,
		NamingTokenFilters, { InContext });
}

const FString& UTakeMetaData::GetSlate() const
{
	return Slate;
}

int32 UTakeMetaData::GetTakeNumber() const
{
	return TakeNumber;
}

FDateTime UTakeMetaData::GetTimestamp() const
{
	return Timestamp;
}

FTimecode UTakeMetaData::GetTimecodeIn() const
{
	return TimecodeIn;
}

FTimecode UTakeMetaData::GetTimecodeOut() const
{
	return TimecodeOut;
}

FFrameTime UTakeMetaData::GetDuration() const
{
	return Duration;
}

FFrameRate UTakeMetaData::GetFrameRate() 
{
	if (bFrameRateFromTimecode)
	{
		FrameRate = FApp::GetTimecodeFrameRate();
	}
	return FrameRate;
}

FString UTakeMetaData::GetDescription() const
{
	return Description;
}

UTakePreset* UTakeMetaData::GetPresetOrigin() const
{
	return PresetOrigin.Get();
}

ULevel* UTakeMetaData::GetLevelOrigin() const
{
#if WITH_EDITORONLY_DATA
	return LevelOrigin.Get();
#else
	return nullptr;
#endif // WITH_EDITORONLY_DATA
}

FString UTakeMetaData::GetLevelPath() const
{
#if WITH_EDITORONLY_DATA
	return !LevelOrigin.IsNull() ? LevelOrigin.ToString() : FString();
#else
	return FString();
#endif // WITH_EDITORONLY_DATA
}

bool UTakeMetaData::GetFrameRateFromTimecode() const
{
	return bFrameRateFromTimecode;
}

void UTakeMetaData::SetSlate(FString InSlate, bool bEmitChanged)
{
	if (!bIsLocked)
	{
		Slate = MoveTemp(InSlate);

		if (bEmitChanged)
		{
			UTakesCoreBlueprintLibrary::OnTakeRecorderSlateChanged(Slate);
			OnTakeSlateChanged().Broadcast(Slate, this);
		}
	}
}

void UTakeMetaData::SetTakeNumber(int32 InTakeNumber, bool bEmitChanged)
{
	if (!bIsLocked)
	{
		TakeNumber = FMath::Max(1, InTakeNumber);

		if (bEmitChanged)
		{
			UTakesCoreBlueprintLibrary::OnTakeRecorderTakeNumberChanged(TakeNumber);
			OnTakeNumberChanged().Broadcast(TakeNumber, this);
		}
	}
}

void UTakeMetaData::SetTimestamp(FDateTime InTimestamp)
{
	if (!bIsLocked)
	{
		Timestamp = InTimestamp;
	}
}

void UTakeMetaData::SetTimecodeIn(FTimecode InTimecodeIn)
{
	if (!bIsLocked)
	{
		TimecodeIn = InTimecodeIn;
	}
}

void UTakeMetaData::SetTimecodeOut(FTimecode InTimecodeOut)
{
	if (!bIsLocked)
	{
		TimecodeOut = InTimecodeOut;
	}
}

void UTakeMetaData::SetDuration(FFrameTime InDuration)
{
	if (!bIsLocked)
	{
		Duration = InDuration;
	}
}

void UTakeMetaData::SetFrameRate(FFrameRate InFrameRate)
{
	if (!bIsLocked)
	{
		FrameRate = InFrameRate;
	}
}

void UTakeMetaData::SetDescription(FString InDescription)
{
	if (!bIsLocked)
	{
		Description = InDescription;
	}
}

void UTakeMetaData::SetPresetOrigin(UTakePreset* InPresetOrigin)
{
	if (!bIsLocked)
	{
		PresetOrigin = InPresetOrigin;
	}
}

void UTakeMetaData::SetLevelOrigin(ULevel* InLevelOrigin)
{
#if WITH_EDITORONLY_DATA
	if (!bIsLocked)
	{
		LevelOrigin = InLevelOrigin; 
	}
#endif // WITH_EDITORONLY_DATA
}

void UTakeMetaData::SetFrameRateFromTimecode(bool InFromTimecode)
{
	if (!bIsLocked)
	{
		bFrameRateFromTimecode = InFromTimecode;
	}
}

UTakeMetaData::FOnTakeSlateChanged& UTakeMetaData::OnTakeSlateChanged()
{
	static FOnTakeSlateChanged OnSlateChanged;
	return OnSlateChanged;
}

UTakeMetaData::FOnTakeNumberChanged& UTakeMetaData::OnTakeNumberChanged()
{
	static FOnTakeNumberChanged OnTakeNumberChanged;
	return OnTakeNumberChanged;
}

void UTakeMetaData::ExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	IMovieSceneMetaDataInterface::ExtendAssetRegistryTags(Context);

	Context.AddTag(FAssetRegistryTag(	AssetRegistryTag_Slate,							Slate,                 
										FAssetRegistryTag::ETagType::TT_Alphabetical,	FAssetRegistryTag::TD_None));
	Context.AddTag(FAssetRegistryTag(	AssetRegistryTag_TakeNumber,					LexToString(TakeNumber),
										FAssetRegistryTag::ETagType::TT_Numerical,		FAssetRegistryTag::TD_None));
	Context.AddTag(FAssetRegistryTag(	AssetRegistryTag_Timestamp,						Timestamp.ToString(),
										FAssetRegistryTag::ETagType::TT_Chronological,	FAssetRegistryTag::TD_Date | FAssetRegistryTag::TD_Time));
	Context.AddTag(FAssetRegistryTag(	AssetRegistryTag_TimecodeIn,					TimecodeIn.ToString(),
										FAssetRegistryTag::ETagType::TT_Numerical,		FAssetRegistryTag::TD_None));
	Context.AddTag(FAssetRegistryTag(	AssetRegistryTag_TimecodeOut,					TimecodeOut.ToString(),
										FAssetRegistryTag::ETagType::TT_Numerical,		FAssetRegistryTag::TD_None));
	Context.AddTag(FAssetRegistryTag(	AssetRegistryTag_Description,					Description,
										FAssetRegistryTag::ETagType::TT_Alphabetical,	FAssetRegistryTag::TD_None));
}

#if WITH_EDITOR

void UTakeMetaData::ExtendAssetRegistryTagMetaData(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	OutMetadata.Add(AssetRegistryTag_Slate, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("TakeMetaData", "Slate_Label", "Slate"))
		.SetTooltip(    NSLOCTEXT("TakeMetaData", "Slate_Tip",   "The slate that this level sequence was recorded with"))
	);

	OutMetadata.Add(AssetRegistryTag_TakeNumber, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("TakeMetaData", "Take_Label", "Take #"))
		.SetTooltip(    NSLOCTEXT("TakeMetaData", "Take_Tip",   "The take number of this recorded level sequence"))
	);

	OutMetadata.Add(AssetRegistryTag_Timestamp, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("TakeMetaData", "Timestamp_Label", "Timestamp"))
		.SetTooltip(NSLOCTEXT("TakeMetaData", "Timestamp_Tip", "The time that this take was started"))
	);

	OutMetadata.Add(AssetRegistryTag_TimecodeIn, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("TakeMetaData", "TimecodeIn_Label", "Timecode In"))
		.SetTooltip(NSLOCTEXT("TakeMetaData", "TimecodeIn_Tip", "The timecode when this recording was started"))
	);

	OutMetadata.Add(AssetRegistryTag_TimecodeOut, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("TakeMetaData", "TimecodeOut_Label", "Timecode Out"))
		.SetTooltip(NSLOCTEXT("TakeMetaData", "TimecodeOut_Tip", "The timecode when this recording was stopped"))
	);

	OutMetadata.Add(AssetRegistryTag_Description, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("TakeMetaData", "Description_Label", "Description"))
		.SetTooltip(    NSLOCTEXT("TakeMetaData", "Description_Tip",   "User-specified description for this take"))
	);

	OutMetadata.Add(AssetRegistryTag_LevelPath, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("TakeMetaData", "LevelPath_Label", "Map"))
		.SetTooltip(    NSLOCTEXT("TakeMetaData", "LevelPath_Tip",   "Map used for this take"))
	);
}

#endif // WITH_EDITOR

