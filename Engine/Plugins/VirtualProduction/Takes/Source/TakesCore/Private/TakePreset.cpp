// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakePreset.h"
#include "LevelSequence.h"
#include "MovieScene.h"

#if WITH_EDITOR
#include "MovieSceneToolsProjectSettings.h"
#endif // WITH_EDITOR

#include "TakePresetSettings.h"
#include "Misc/TransactionObjectEvent.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakePreset)

FSimpleMulticastDelegate UTakePreset::OnTakePresetAllocatedDelegate;

UTakePreset::UTakePreset(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{}

ULevelSequence* UTakePreset::GetOrCreateLevelSequence()
{
	if (!LevelSequence)
	{
		CreateLevelSequence();
	}
	return LevelSequence;
}

FSimpleMulticastDelegate& UTakePreset::OnTakePresetAllocated()
{
	return OnTakePresetAllocatedDelegate;
}

UTakePreset* UTakePreset::AllocateTransientPreset(UTakePreset* TemplatePreset)
{
	static const TCHAR* PackageName = TEXT("/Temp/TakeRecorder/PendingTake");

	UTakePreset* ExistingPreset = FindObject<UTakePreset>(nullptr, TEXT("/Temp/TakeRecorder/PendingTake.PendingTake"));
	if (ExistingPreset)
	{
		return ExistingPreset;
	}

	static FName DesiredName = "PendingTake";

	UPackage* NewPackage = CreatePackage(PackageName);
	NewPackage->SetFlags(RF_Transient);
	NewPackage->AddToRoot();

	UTakePreset* NewPreset = nullptr;

	if (TemplatePreset)
	{
		NewPreset = DuplicateObject<UTakePreset>(TemplatePreset, NewPackage, DesiredName);
		NewPreset->SetFlags(RF_Transient | RF_Transactional | RF_Standalone);
	}
	else
	{
		NewPreset = NewObject<UTakePreset>(NewPackage, DesiredName, RF_Transient | RF_Transactional | RF_Standalone);
	}

	NewPreset->GetOrCreateLevelSequence();

	OnTakePresetAllocatedDelegate.Broadcast();

	return NewPreset;
}

void UTakePreset::CreateLevelSequence()
{
	if (LevelSequence)
	{
		LevelSequence->Modify();

		FName UniqueName = MakeUniqueObjectName(GetTransientPackage(), ULevelSequence::StaticClass(), "DEAD_TakePreset_LevelSequence");
		LevelSequence->Rename(*UniqueName.ToString(), GetTransientPackage());
		LevelSequence = nullptr;
	}

	// Copy the transient and transactional flags from the parent
	EObjectFlags SequenceFlags = GetFlags() & (RF_Transient | RF_Transactional);
	LevelSequence = NewObject<ULevelSequence>(this, UTakePresetSettings::Get()->GetTargetRecordClass(), GetFName(), SequenceFlags);
	LevelSequence->Initialize();

#if WITH_EDITOR
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();
	const float DefaultStartTime = ProjectSettings->DefaultStartTime;
	const float DefaultDuration = ProjectSettings->DefaultDuration;
#else // WITH_EDITOR
	// These values are the default values from the Project Settings.
	const float DefaultStartTime = 0.0f;
	const float DefaultDuration = 5.0f;
#endif // WITH_EDITOR

	UMovieScene* MovieScene = LevelSequence->GetMovieScene();
	FFrameNumber StartFrame = (DefaultStartTime * MovieScene->GetTickResolution()).RoundToFrame();
	int32        Duration = (DefaultDuration * MovieScene->GetTickResolution()).RoundToFrame().Value;

	LevelSequence->GetMovieScene()->SetPlaybackRange(StartFrame, Duration);

#if WITH_EDITOR
	FMovieSceneEditorData& EditorData = LevelSequence->GetMovieScene()->GetEditorData();
	EditorData.ViewStart = -1.0;
	EditorData.ViewEnd   =  5.0;
	EditorData.WorkStart = -1.0;
	EditorData.WorkEnd   =  5.0;
#endif // WITH_EDITOR

	OnLevelSequenceChangedEvent.Broadcast();
}

void UTakePreset::CopyFrom(UTakePreset* TemplatePreset)
{
	Modify();

	if (TemplatePreset && TemplatePreset->LevelSequence)
	{
		CopyFrom(TemplatePreset->LevelSequence);
	}
	else
	{
		CreateLevelSequence();
	}
}

void UTakePreset::CopyFrom(ULevelSequence* TemplateLevelSequence)
{
	Modify();

	// Always call the sequence the same as the owning preset
	FName SequenceName = GetFName();

	if (TemplateLevelSequence)
	{
		// Rename the old one
		if (LevelSequence)
		{
			LevelSequence->Modify();

			FName UniqueName = MakeUniqueObjectName(GetTransientPackage(), ULevelSequence::StaticClass(), "DEAD_TakePreset_LevelSequence");
			LevelSequence->Rename(*UniqueName.ToString(), GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		}

		EObjectFlags SequenceFlags = GetFlags() & (RF_Transient | RF_Transactional);

		LevelSequence = Cast<ULevelSequence>(StaticDuplicateObject(TemplateLevelSequence, this, SequenceName, SequenceFlags));
		LevelSequence->SetFlags(SequenceFlags);

		OnLevelSequenceChangedEvent.Broadcast();
	}
	else
	{
		CreateLevelSequence();
	}
}

FDelegateHandle UTakePreset::AddOnLevelSequenceChanged(const FSimpleDelegate& InHandler)
{
	return OnLevelSequenceChangedEvent.Add(InHandler);
}

void UTakePreset::RemoveOnLevelSequenceChanged(FDelegateHandle DelegateHandle)
{
	OnLevelSequenceChangedEvent.Remove(DelegateHandle);
}

#if WITH_EDITOR
void UTakePreset::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetChangedProperties().Contains(GET_MEMBER_NAME_CHECKED(UTakePreset, LevelSequence)))
	{
		OnLevelSequenceChangedEvent.Broadcast();
	}
}
#endif // WITH_EDITOR

