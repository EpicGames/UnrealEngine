// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakePresetSettings.h"

#include "LevelSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakePresetSettings)

UTakePresetSettings::UTakePresetSettings()
	: TargetRecordClass(ULevelSequence::StaticClass())
{}

UTakePresetSettings* UTakePresetSettings::Get()
{
	return GetMutableDefault<UTakePresetSettings>();
}

UClass* UTakePresetSettings::GetTargetRecordClass() const
{
	return TargetRecordClass.TargetRecordClass ? TargetRecordClass.TargetRecordClass.Get() : ULevelSequence::StaticClass();
}

void UTakePresetSettings::SetTargetRecordClass(const TSoftClassPtr<ULevelSequence>& InClass)
{
	TargetRecordClass.TargetRecordClass = InClass;
#if WITH_EDITOR
	if (FProperty* Property = FindFProperty<FProperty>(GetClass(), GET_MEMBER_NAME_CHECKED(UTakePresetSettings, TargetRecordClass)))
	{
		FPropertyChangedEvent ChangedEvent(Property);
		PostEditChangeProperty(ChangedEvent);
	}
#endif
}

#if WITH_EDITOR
void UTakePresetSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UObject::PostEditChangeProperty(PropertyChangedEvent);
	SaveConfig();
	OnSettingsChangedDelegate.Broadcast();
}
#endif
