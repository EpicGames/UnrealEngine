// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderSource.h"
#include "Styling/SlateIconFinder.h"
#include "MovieSceneSequenceID.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderSource)

UTakeRecorderSource::UTakeRecorderSource(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	,bEnabled(true)
	,TakeNumber(0)
{
	TrackTint = FColor(127, 127, 127);
}

#if WITH_EDITOR
void UTakeRecorderSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	UTakeRecorderSources::OnSourceModified().Broadcast(this);
}
#endif

bool UTakeRecorderSource::OverwriteExistingTrackData() const
{
	return true;
}

const FSlateBrush* UTakeRecorderSource::GetDisplayIconImpl() const
{
	return FSlateIconFinder::FindCustomIconBrushForClass(GetClass(), TEXT("ClassThumbnail"));
}

TArray<UTakeRecorderSource*> UTakeRecorderSource::PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer)
{
	return TArray<UTakeRecorderSource*>();
}
