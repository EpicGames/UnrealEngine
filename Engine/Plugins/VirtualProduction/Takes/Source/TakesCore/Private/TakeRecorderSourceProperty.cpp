// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderSourceProperty.h"
#include "Algo/Accumulate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderSourceProperty)

#if WITH_EDITOR
void UActorRecorderPropertyMap::PostEditUndo()
{
	Super::PostEditUndo();
	UpdateCachedValues();
}

void UActorRecorderPropertyMap::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UpdateCachedValues();
}
#endif // WITH_EDITOR

void UActorRecorderPropertyMap::UpdateCachedValues()
{
	for (UActorRecorderPropertyMap* Child: Children)
	{
		Child->Parent = this;
		Child->UpdateCachedValues();
	}

	ChildChanged();
}

int32 UActorRecorderPropertyMap::NumberOfComponentsRecordedOnThis() const
{
	return Children.Num();
}

int32 UActorRecorderPropertyMap::NumberOfPropertiesRecordedOnThis() const
{
	return Algo::Accumulate(Properties, 0, [](int32 Result, const FActorRecordedProperty& Property) -> int32
	{
		return (Property.bEnabled) ? Result+1 : Result;
	});
}


void UActorRecorderPropertyMap::ChildChanged()
{
	RecordingInfo = {NumberOfPropertiesRecordedOnThis(), NumberOfComponentsRecordedOnThis()};
	for (UActorRecorderPropertyMap* Child: Children)
	{
		RecordingInfo += Child->RecordingInfo;
	}

	if (Parent.IsValid())
	{
		Parent->ChildChanged();
	}
}

