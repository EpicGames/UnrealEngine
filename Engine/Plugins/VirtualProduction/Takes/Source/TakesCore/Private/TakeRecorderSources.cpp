// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderSources.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CanvasTypes.h"
#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "Engine/TimecodeProvider.h"
#include "LevelSequence.h"
#include "MetaData/MovieSceneShotMetaData.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneFolder.h"
#include "NamingTokens/TakeRecorderNamingTokensContext.h"
#include "Engine/TimecodeProvider.h"
#include "Misc/App.h"
#include "MetaData/MovieSceneShotMetaData.h"
#include "NamingTokens/TakeRecorderNamingTokensContext.h"
#include "UObject/Package.h"
#include "Sections/MovieSceneSubSection.h"
#include "TakeMetaData.h"
#include "TakeRecorderSource.h"
#include "TakesCoreLog.h"
#include "TakesUtils.h"
#include "TrackRecorders/IMovieSceneTrackRecorderHost.h"
#include "Tracks/MovieSceneSubTrack.h"

#if WITH_EDITOR
#include "ObjectTools.h"
#include "Editor.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "TakeRecorderSources"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderSources)

DEFINE_LOG_CATEGORY(SubSequenceSerialization);

TArray<TPair<FQualifiedFrameTime, FQualifiedFrameTime> > UTakeRecorderSources::RecordedTimes;

UTakeRecorderSources::UTakeRecorderSources(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, SourcesSerialNumber(0)
{
	// Ensure instances are always transactional
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		SetFlags(RF_Transactional);
	}

	if (GEngine)
	{
		constexpr bool bIsRHS = true;
		GEngine->AddEngineStat(TEXT("STAT_TakeRecorderSources"), TEXT("STATCAT_TakeRecorder"),
							   LOCTEXT("DisplayTakeRecorderSources", "Displays current take recorder sources in the stats panel."),
							   UEngine::FEngineStatRender::CreateUObject(this, &UTakeRecorderSources::RenderStatTakeSources),
							   nullptr, bIsRHS);
	}
}

UTakeRecorderSource* UTakeRecorderSources::AddSource(TSubclassOf<UTakeRecorderSource> InSourceType)
{
	UTakeRecorderSource* NewSource = nullptr;

	if (UClass* Class = InSourceType.Get())
	{
		NewSource = NewObject<UTakeRecorderSource>(this, Class, NAME_None, RF_Transactional);
		if (ensure(NewSource))
		{
			NewSource->Initialize();
			Sources.Add(NewSource);
			++SourcesSerialNumber;

			OnSourceAdded().Broadcast(NewSource);
		}
	}

	return NewSource;
}

void UTakeRecorderSources::RemoveSource(UTakeRecorderSource* InSource)
{
	if (InSource != nullptr)
	{
		InSource->PreRemove();
	}

	Sources.Remove(InSource);

	// Remove the entry from the sub-sequence map as we won't be needing it anymore.
	SourceSubSequenceMap.Remove(InSource);

	++SourcesSerialNumber;

	OnSourceRemoved().Broadcast(InSource);
}

FDelegateHandle UTakeRecorderSources::BindSourcesChanged(const FSimpleDelegate& Handler)
{
	return OnSourcesChangedEvent.Add(Handler);
}

void UTakeRecorderSources::UnbindSourcesChanged(FDelegateHandle Handle)
{
	OnSourcesChangedEvent.Remove(Handle);
}

UTakeRecorderSources::FOnSourceAdded& UTakeRecorderSources::OnSourceAdded()
{
	static FOnSourceAdded OnSourceAddedEvent;
	return OnSourceAddedEvent;
}

UTakeRecorderSources::FOnSourceRemoved& UTakeRecorderSources::OnSourceRemoved()
{
	static FOnSourceRemoved OnSourceRemovedEvent;
	return OnSourceRemovedEvent;
}

UTakeRecorderSources::FOnSourceModified& UTakeRecorderSources::OnSourceModified()
{
	static FOnSourceModified OnSourceModifiedEvent;
	return OnSourceModifiedEvent;
}

UTakeRecorderSources::FOnActorBindingAdded& UTakeRecorderSources::OnAddSourceForActorBinding()
{
	static FOnActorBindingAdded OnActorBindingAddedDelegate;
	return OnActorBindingAddedDelegate;
}

ULevelSequence* UTakeRecorderSources::AddSourceForActorBinding(AActor* InActor, ULevelSequence* InLevelSequence)
{
	// Communicate to the actor sources module we're adding a new binding.
	check(OnAddSourceForActorBinding().IsBound());
	UTakeRecorderSource* NewSource = OnAddSourceForActorBinding().Execute(InActor, InLevelSequence);
	if (ensure(NewSource))
	{
		if (const IMovieSceneTrackRecorderHost* AsTrackRecorderHost = NewSource->AsTrackRecorderHost())
		{
			const bool bIsPossessable = AsTrackRecorderHost->GetTrackRecorderSettings().bRecordToPossessable;
			AddedActorPossessableBindings.Add(InActor, bIsPossessable);
		}
		
		// Look for a subsequence if applicable.
		if (TObjectPtr<ULevelSequence>* Subsequence = SourceSubSequenceMap.Find(NewSource))
		{
			return *Subsequence;
		}
	}

	// Fallback to original sequence (probably root).
	return InLevelSequence;
}

bool UTakeRecorderSources::IsActorPossessable(const AActor* InActor) const
{
	// Check bindings first -- we can't just check the sources, because reference bindings create temporary sources that may be
	// cleared out by this point.
	if (const bool *AddedBinding = AddedActorPossessableBindings.Find(InActor))
	{
		return *AddedBinding;
	}
	
	for (const UTakeRecorderSource* Source : GetSources())
	{
		if (Source && Source->IsActorSourceForActor(InActor))
		{
			return TakesUtils::IsSourcePossessable(Source);
		}
	}
				
	return GetSettings().bRecordToPossessable;
}

#if WITH_EDITOR
void UTakeRecorderSources::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (!PropertyChangedEvent.Property || PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTakeRecorderSources, Sources))
	{
		++SourcesSerialNumber;
	}

	// There have been cases where users are transacted sources to remote nodes with Multi-user and those sources cannot be assigned due to missing objects.
	// For example, creating a audio source fails on a remote nDisplay node because of missing audio objects.  We want to flag these in the log.
	int32 SlotIndex = 0;
	for (UTakeRecorderSource* Source : Sources)
	{
		if (!Source)
		{
			UE_LOGF(LogTakesCore, Warning, "Transacted source at slot %d is invalid will be ignored during recording.", SlotIndex);
		}
		SlotIndex++;
	}
}
#endif // WITH_EDITOR

void UTakeRecorderSources::StartPreRecordingRecursive(const TArray<UTakeRecorderSource*>& InSources, ULevelSequence* InRootSequence,
	TArray<UTakeRecorderSource*>& NewSourcesOut, FManifestSerializer* InManifestSerializer)
{
	AddedActorPossessableBindings.Empty();
	
	TSet<FString> UsedTrackNames;
	PreRecordingRecursive(InSources, InRootSequence, NewSourcesOut, InManifestSerializer, UsedTrackNames);
	
	TSet<UTakeRecorderSource*> AllSources(InSources);
	AllSources.Append(NewSourcesOut);
	
	for (UTakeRecorderSource* Source : AllSources)
	{
		if (Source)
		{
			Source->OnAllSourcesPreRecordingFinished();
		}
	}
}

void UTakeRecorderSources::PreRecordingRecursive(TArray<UTakeRecorderSource*> InSources, ULevelSequence* InRootSequence,
                                                 TArray<UTakeRecorderSource*>& NewSourcesOut, FManifestSerializer* InManifestSerializer, TSet<FString>& UsedTrackNames)
{

	TArray<UTakeRecorderSource*> NewSources;

	// Optionally create a folder in the Sequencer UI that will contain this source. We don't want sub-sequences to have folders
	// created for their sources as you would end up with a Subscene with one item in it hidden inside of a folder, so instead
	// only the root sequence gets folders created.
	const bool bCreateSequencerFolders = true;
	NewSourcesOut.Append(InSources);

	for (UTakeRecorderSource* Source : InSources)
	{
		if (Source && Source->bEnabled)
		{
			ULevelSequence* TargetSequence = InRootSequence;
			FMovieSceneSequenceID TargetSequenceID = MovieSceneSequenceID::Root;

			// The Sequencer Take system is built around swapping out sub-sequences. If they want to use this system, we create a sub-sequence
			// for the Source and tell it to write into this sub-sequence instead of the root sequence. We then keep track of which Source
			// is using which sub-sequence so that we can push the correct sequence for all points of the Source's recording lifecycle.
			if (Settings.bRecordSourcesIntoSubSequences && Source->SupportsSubscenes())
			{
				const FString BaseSubSequenceTrackName = TakesUtils::SanitizeObjectName(Source->GetSubsceneTrackName(InRootSequence));
				FString SubSequenceTrackName = BaseSubSequenceTrackName;
				
				// Simple duplicate name handling to prevent collisions. Normally the user should have a unique token in the title,
				// such as {actor} which will keep the name unique.
				{
					int32 DuplicateCount = 1;
					bool bTrackNameAlreadyUsed = false;
					do
					{
						UsedTrackNames.Add(SubSequenceTrackName, &bTrackNameAlreadyUsed);
						if (bTrackNameAlreadyUsed)
						{
							SubSequenceTrackName = FString::Printf(TEXT("%s_%s"), *BaseSubSequenceTrackName, *FString::FromInt(DuplicateCount++));
						}
					}
					while (bTrackNameAlreadyUsed);
				}

				const FString SubSequenceAssetName = TakesUtils::SanitizeObjectName(Source->GetSubsceneAssetName(InRootSequence));
				TargetSequence = CreateSubSequenceForSource(InRootSequence, SubSequenceTrackName, SubSequenceAssetName, Settings);

				// If there's already a Subscene Track for our sub-sequence we need to remove that track before create a new one. No data is lost in this process as the
				// sequence that the subscene points to has been copied by CreateSubSequenceForSource so a new track pointed to the new subsequence includes all the old data.
				const FString SequenceName = FPaths::GetBaseFilename(TargetSequence->GetPathName());
				UMovieSceneSubTrack* SubsceneTrack = nullptr;

				/*
				 * Note that as part of the TakeRecorder in runtime work and GetDisplayName() being an editor only function, the logic below is currently only run in editor.
				 * When adding runtime support, with a non editor alternative to GetDisplayName, the below for loop in the if branch will need updating.
				 * As Take Recorder at runtime is not yet support, it is ok for this not to function or work at runtime, but will eventually be updated.
				 */

			#if WITH_EDITOR
				if (Source->OverwriteExistingTrackData())
				{
					for (UMovieSceneTrack* Track : InRootSequence->GetMovieScene()->GetTracks())
					{
						if (Track->IsA<UMovieSceneSubTrack>())
						{
							// TODO: Need something other than DisplayName to track this.
							// Do we want to increment the name count when not overwriting?
							if (Track->GetDisplayName().ToString() == SubSequenceTrackName)
							{
								SubsceneTrack = CastChecked<UMovieSceneSubTrack>(Track);
								SubsceneTrack->RemoveAllAnimationData();
								// Currently, we only expect 1 copy at most. If we want to overwrite all copies
								// there needs to be additional logic to clean out other tracks.
								break;
							}
						}
					}
				}
			#endif // WITH_EDITOR

				// We need to add the new subsequence to the root sequence immediately so that it shows up in the UI and you can tell that things
				// are being recorded, otherwise they don't show up until recording stops and then it magically pops in.
				if (!SubsceneTrack)
				{
					SubsceneTrack = CastChecked<UMovieSceneSubTrack>(InRootSequence->GetMovieScene()->AddTrack(UMovieSceneSubTrack::StaticClass()));
				}

				// Track should not be transactional during the recording process
				SubsceneTrack->ClearFlags(RF_Transactional);

			#if WITH_EDITOR
				// We create a new sub track for every Source so that we can name the Subtrack after the Source instead of just the sections within it.
				SubsceneTrack->SetDisplayName(FText::FromString(Source->GetSubsceneTrackName(InRootSequence)));
				SubsceneTrack->SetColorTint(Source->TrackTint);
			#endif // WITH_EDITOR

				// When we create the Subscene Track we'll make sure a folder is created for it to sort into and add the new Subscene Track as a child of it.
				if (bCreateSequencerFolders)
				{
					UMovieSceneFolder* Folder = AddFolderForSource(Source, InRootSequence->GetMovieScene());
					Folder->AddChildTrack(SubsceneTrack);
				}

				// We initialize the sequence to start at zero and be a 0 frame length section as there is no data in the sections yet.
				// We'll have to update these sections each frame as the recording progresses so they appear to get longer like normal
				// tracks do as we record into them.
				FFrameNumber RecordStartFrame = Settings.bStartAtCurrentTimecode ? FFrameRate::TransformTime(FFrameTime(FApp::GetTimecode().ToFrameNumber(TargetLevelSequenceDisplayRate)), TargetLevelSequenceDisplayRate, TargetLevelSequenceTickResolution).FloorToFrame() : InRootSequence->GetMovieScene()->GetPlaybackRange().GetLowerBoundValue();
				UMovieSceneSubSection* NewSubSection = SubsceneTrack->AddSequence(TargetSequence, RecordStartFrame, 0);

				// Section should not be transactional during the recording process
				NewSubSection->ClearFlags(RF_Transactional);

				NewSubSection->SetRowIndex(SubsceneTrack->GetMaxRowIndex() + 1);
				SubsceneTrack->FixRowIndices();

				TargetSequenceID = NewSubSection->GetSequenceID();

				ActiveSubSections.Add(NewSubSection);
				if (InManifestSerializer)
				{
					FName SerializedType("SubSequence");
					FManifestProperty  ManifestProperty(SubSequenceAssetName, SerializedType, FGuid());
					InManifestSerializer->WriteFrameData(InManifestSerializer->FramesWritten, ManifestProperty);

					FString AssetPath = InManifestSerializer->GetLocalCaptureDir();

					IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
					if (!PlatformFile.DirectoryExists(*AssetPath))
					{
						PlatformFile.CreateDirectory(*AssetPath);
					}

					AssetPath = AssetPath / SubSequenceAssetName;
					if (!PlatformFile.DirectoryExists(*AssetPath))
					{
						PlatformFile.CreateDirectory(*AssetPath);
					}

					TSharedPtr<FManifestSerializer> NewManifestSerializer = MakeShared<FManifestSerializer>();
					CreatedManifestSerializers.Add(NewManifestSerializer);
					InManifestSerializer = NewManifestSerializer.Get();

					InManifestSerializer->SetLocalCaptureDir(AssetPath);

					FManifestFileHeader Header(SubSequenceAssetName, SerializedType, FGuid());
					FText Error;
					FString FileName = FString::Printf(TEXT("%s_%s"), *(SerializedType.ToString()), *(SubSequenceAssetName));

					if (!InManifestSerializer->OpenForWrite(FileName, Header, Error))
					{
						UE_LOGF(SubSequenceSerialization, Warning, "Error Opening Sequence Sequencer File: Subject '%ls' Error '%ls'", *(SubSequenceAssetName), *(Error.ToString()));
					}
				}
			}

			// Update our mappings of which sources use which sub-sequence.
			SourceSubSequenceMap.FindOrAdd(Source) = TargetSequence;

			for (UTakeRecorderSource* NewlyAddedSource : Source->PreRecording(TargetSequence, TargetSequenceID, InRootSequence, InManifestSerializer))
			{
				// Add it to our classes list of sources 
				Sources.Add(NewlyAddedSource);

				// And then track it separately so we can recursively call PreRecording 
				NewSources.Add(NewlyAddedSource);
			}

			// We need to wait until PreRecording is called on a source before asking it to place itself in a folder
			// so that the Source has had a chance to create any required sections that will go in the folder.
			if (!Settings.bRecordSourcesIntoSubSequences && bCreateSequencerFolders)
			{
				if (UMovieSceneFolder* Folder = AddFolderForSource(Source, InRootSequence->GetMovieScene()))
				{
					// Different sources can create different kinds of tracks so we allow each source to decide how it gets
					// represented inside the folder.
					Source->AddContentsToFolder(Folder);
				}
			}
		}
	}

	if (NewSources.Num())
	{
		// We don't want to nestle sub-sequences recursively so we always pass the Root Sequence and not the sequence
		// created for a new source.
		PreRecordingRecursive(NewSources, InRootSequence, NewSourcesOut, InManifestSerializer, UsedTrackNames);
		SourcesSerialNumber++;
	}
}

void UTakeRecorderSources::StartRecordingPreRecordedSources(const FQualifiedFrameTime& CurrentFrameTime)
{
	StartRecordingTheseSources(PreRecordedSources, CurrentFrameTime);
	PreRecordedSources.Reset(0);
}

void UTakeRecorderSources::PreRecordSources(TArray<UTakeRecorderSource*> InSources)
{
	PreRecordedSources.Reset(0);
	StartPreRecordingRecursive(InSources, CachedLevelSequence, PreRecordedSources, CachedManifestSerializer);
}


void UTakeRecorderSources::StartRecordingSource(TArray<UTakeRecorderSource*> InSources, const FQualifiedFrameTime& CurrentFrameTime)
{
	// This calls PreRecording recursively on every source so that all sources that get added by another source
	// have had PreRecording called.
	TArray<UTakeRecorderSource*> NewSources;
	StartPreRecordingRecursive(InSources, CachedLevelSequence, NewSources, CachedManifestSerializer);
	
	// Only start recording these sources if a recording is already in progress. It's possible this method is called during PreRecord, and we don't
	// want to end up calling Record twice. This condition helps when starting a recording with an actor reference property where the value points
	// to another actor that has not been added as a source, and we need to add it automatically before formally starting the recording.
	if (bIsRecording)
	{
		// Ensures uniqueness. Recursive method will add old as new sources.
		TSet<UTakeRecorderSource*> UniqueSources(InSources);
		UniqueSources.Append(NewSources);
		InSources = UniqueSources.Array();
		StartRecordingTheseSources(InSources, CurrentFrameTime);
	}
}


void UTakeRecorderSources::StartRecordingTheseSources(const TArray<UTakeRecorderSource*>& InSources, const FQualifiedFrameTime& CurrentFrameTime)
{
	const FTimecode CurrentTimecode = FTimecode::FromFrameNumber(FFrameRate::TransformTime(CurrentFrameTime.Time, TargetLevelSequenceTickResolution, TargetLevelSequenceDisplayRate).FloorToFrame(), TargetLevelSequenceDisplayRate);

	for (UTakeRecorderSource* Source : InSources)
	{
		if (Source && Source->bEnabled)
		{
			if (Settings.bRecordSourcesIntoSubSequences && Source->SupportsSubscenes()) //Set Timcode on MovieScene if we created a sub scene for it
			{
				for (const TObjectPtr<UMovieSceneSubSection>& ActiveSubSection : ActiveSubSections)
				{
					// Set timecode source and start time if it hasn't been set
					if (ActiveSubSection->TimecodeSource.Timecode == FTimecode())
					{
						SetSectionStartTimecode(ActiveSubSection, CurrentTimecode, CurrentFrameTime);
					}
				}
			}

			Source->StartRecording(CurrentTimecode, CurrentFrameTime.Time.FloorToFrame(), SourceSubSequenceMap[Source]);
		}
	}
}

void UTakeRecorderSources::SetSectionStartTimecode(UMovieSceneSubSection* SubSection, const FTimecode& Timecode, const FQualifiedFrameTime& CurrentFrameTime)
{
	SubSection->TimecodeSource = FMovieSceneTimecodeSource(Timecode);

	// Ensure we're expanded to at least the next frame so that we don't set the start past the end
	// when we set the first frame.
	FFrameNumber RecordStartFrame = CurrentFrameTime.Time.FloorToFrame(); 		
	SubSection->ExpandToFrame(RecordStartFrame + FFrameNumber(1));
	SubSection->SetStartFrame(TRangeBound<FFrameNumber>::Inclusive(RecordStartFrame));

	// If the section does not have a valid sequence, do not attempt to access it.
	// It may not be valid if it was never created properly, such as when recording into a locked sequence.
	if (UMovieSceneSequence* SubSectionSequence = SubSection->GetSequence())
	{
		if (UMovieScene* SubSectionMovieScene = SubSectionSequence->GetMovieScene())
		{
			SubSectionMovieScene->SetPlaybackRange(TRange<FFrameNumber>(RecordStartFrame, RecordStartFrame + 1));
		}
	}
}


void UTakeRecorderSources::SetCachedAssets(class ULevelSequence* InSequence, FManifestSerializer* InManifestSerializer)
{
	// We want to cache the Serializer and Level Sequence in case more objects start recording mid-recording.
	// We want them to use the same logic flow as if initialized from scratch so that they properly sort into
	// sub-sequences, etc.
	CachedManifestSerializer = InManifestSerializer;
	CachedLevelSequence = InSequence;
	RecordedTimes.Empty();
}

void UTakeRecorderSources::PreRecording(class ULevelSequence* InSequence, const FQualifiedFrameTime& InCurrentFrameTime, FManifestSerializer* InManifestSerializer)
{
	CachedFrameTime = InCurrentFrameTime;
	SetCachedAssets(InSequence, InManifestSerializer);
	PreRecordSources(Sources);
}

void UTakeRecorderSources::StartRecording(class ULevelSequence* InSequence, const FQualifiedFrameTime& InCurrentFrameTime, FManifestSerializer* InManifestSerializer)
{
	CachedFrameTime = InCurrentFrameTime;
	bIsRecording = true;
	TargetLevelSequenceTickResolution = InSequence->GetMovieScene()->GetTickResolution();
	TargetLevelSequenceDisplayRate = InSequence->GetMovieScene()->GetDisplayRate();

	FTimecode CurrentTimecode = FTimecode::FromFrameNumber(FFrameRate::TransformTime(InCurrentFrameTime.Time, TargetLevelSequenceTickResolution, TargetLevelSequenceDisplayRate).FloorToFrame(), TargetLevelSequenceDisplayRate);

	UE_LOGF(LogTakesCore, Log, "StartRecording: %ls", *CurrentTimecode.ToString());

	StartRecordingPreRecordedSources(InCurrentFrameTime);
}

FFrameTime UTakeRecorderSources::AdvanceTime(const FQualifiedFrameTime& CurrentFrameTime, float DeltaTime)
{
	CachedFrameTime = CurrentFrameTime;
	return CurrentFrameTime.ConvertTo(TargetLevelSequenceTickResolution);
}

FFrameTime UTakeRecorderSources::TickRecording(class ULevelSequence* InSequence, const FQualifiedFrameTime& CurrentFrameTime, float DeltaTime)
{
	CachedFrameTime = CurrentFrameTime;

	FTimecode Timecode = FTimecode::FromFrameNumber(FFrameRate::TransformTime(CurrentFrameTime.Time, TargetLevelSequenceTickResolution, TargetLevelSequenceDisplayRate).FloorToFrame(), TargetLevelSequenceDisplayRate);

#if !NO_LOGGING
	UE_LOGF(LogTakesCore, VeryVerbose, "TickRecording: %ls", *Timecode.ToString());
#endif

	bool bTimeIncremented = DeltaTime > 0.0f;

	if (bTimeIncremented) //only record if time incremented, may not with timecode providers with low frame rates
	{
		for (int32 SourceIndex = 0; SourceIndex < Sources.Num(); ++SourceIndex)
		{
			if (Sources[SourceIndex] && Sources[SourceIndex]->bEnabled)
			{
				Sources[SourceIndex]->TickRecording(CurrentFrameTime);
			}
		}
	}

	if (FApp::GetCurrentFrameTime().IsSet())
	{
		RecordedTimes.Add(TPair<FQualifiedFrameTime, FQualifiedFrameTime>(CurrentFrameTime, FApp::GetCurrentFrameTime().GetValue() ));
	}

	// If we're recording into sub-sections we want to update their range every frame so they appear to
	// animate as their contents are filled. We can't check against the size of all sections (not all
	// source types have data in their sections until the end) and if you're partially re-recording
	// a track it would size to the existing content which would skip the animation as well.

	FFrameNumber EndFrame = CurrentFrameTime.Time.CeilToFrame();
	for (UMovieSceneSubSection* SubSection : ActiveSubSections)
	{
		// Subsections will have been created to start at the time that they appeared, so we just need to expand their range to this recording time
		SubSection->ExpandToFrame(EndFrame);
	}
	return CurrentFrameTime.ConvertTo(TargetLevelSequenceTickResolution);
}

void UTakeRecorderSources::StopRecording(class ULevelSequence* InSequence, const bool bCancelled)
{
	UE_LOGF(LogTakesCore, Log, "StopRecording");

	bIsRecording = false;

	for (auto Source : Sources)
	{
		if (Source && Source->bEnabled)
		{
			Source->StopRecording(SourceSubSequenceMap[Source]);
		}
	}

	TArray<UTakeRecorderSource*> SourcesToRemove;
	for (auto Source : Sources)
	{
		if (Source && Source->bEnabled)
		{
			for (auto SourceToRemove : Source->PostRecording(SourceSubSequenceMap[Source], InSequence, bCancelled))
			{
				SourcesToRemove.Add(SourceToRemove);
			}
		}
	}

	if (SourcesToRemove.Num())
	{
		for (auto SourceToRemove : SourcesToRemove)
		{
			Sources.Remove(SourceToRemove);
		}
		++SourcesSerialNumber;
	}

	for (UTakeRecorderSource* Source : Sources)
	{
		if (Source)
		{
			Source->FinalizeRecording();
		}
	}

	// Re-enable transactional after recording
	InSequence->GetMovieScene()->SetFlags(RF_Transactional);
	for (UMovieSceneSection* Section : InSequence->GetMovieScene()->GetAllSections())
	{
		Section->MarkAsChanged();
	}

	// Ensure each sub-section is as long as it should be. If we're recording into subsections and a user is doing a partial
	// re-record of the data within the sub section we can end up with the case where the new section is shorter than the original
	// data. We don't want to trim the data unnecessarily, and we've been updating the length of the section every frame of the recording
	// as we go (to show the 'animation' of it recording), but we need to restore it to the full length.
	for (UMovieSceneSubSection* SubSection : ActiveSubSections)
	{
		if (bCancelled)
		{
			if (UMovieSceneTrack* SubTrack = Cast<UMovieSceneTrack>(SubSection->GetOuter()))
			{
				SubTrack->RemoveSection(*SubSection);
			}
		}
		else
		{
			UMovieSceneSequence* SubSequence = SubSection->GetSequence();
			if (SubSequence)
			{
				const bool bUpperBoundOnly = false; // Expand the Play Range of the sub-section to encompass all sections within it.
				TakesUtils::ClampPlaybackRangeToEncompassAllSections(SubSequence->GetMovieScene(), bUpperBoundOnly);
				TakesUtils::ResetViewAndWorkRange(SubSequence->GetMovieScene());

			#if WITH_EDITOR
				// SetReadOnly is editor only.
				// Lock the sequence so that it can't be changed without implicitly unlocking it now
				if (Settings.bAutoLock)
				{
					SubSequence->GetMovieScene()->SetReadOnly(true);
				}
			#endif // WITH_EDITOR

				ULevelSequence* SequenceAsset = CastChecked<ULevelSequence>(SubSequence);

			#if WITH_EDITOR
				// Mark that this subsequence has been recorded and is a subsequence.
				UMovieSceneShotMetaData* MovieSequenceMetaData = SequenceAsset->FindOrAddMetaData<UMovieSceneShotMetaData>();
				MovieSequenceMetaData->SetIsRecorded(true);
				MovieSequenceMetaData->SetIsSubSequence(true);

				// Lock the meta data so it can't be changed without implicitly unlocking it now
				UTakeMetaData* TakeMetaData = SequenceAsset->FindMetaData<UTakeMetaData>();
				check(TakeMetaData);
				TakeMetaData->Lock();
			#endif // WITH_EDITOR

				SubSection->SetRange(SubSequence->GetMovieScene()->GetPlaybackRange());

				for (UMovieSceneSection* Section : SubSequence->GetMovieScene()->GetAllSections())
				{
					Section->MarkAsChanged();
				}

				// Re-enable transactional after recording
				SubSequence->GetMovieScene()->SetFlags(RF_Transactional);
			}

			// Re-enable transactional after recording
			SubSection->SetFlags(RF_Transactional);

			if (UMovieSceneTrack* SubTrack = Cast<UMovieSceneTrack>(SubSection->GetOuter()))
			{
				SubTrack->SetFlags(RF_Transactional);
			}
		}
	}

	if (Settings.bRemoveRedundantTracks)
	{
		RemoveRedundantTracks();
	}

	if (CreatedManifestSerializers.Num())
	{
		for (auto Serializer : CreatedManifestSerializers)
		{
			Serializer->Close();
		}
	}

	TArray<UObject*> AssetsToCleanUp;
	for (const TTuple<TObjectPtr<UTakeRecorderSource>, TObjectPtr<ULevelSequence>>& SourceSubSequence : SourceSubSequenceMap)
	{
		// Only save subsequences but not the root sequence, it is already saved in UTakeRecorder::StopInternal
		const bool bIsValidSubSequence = SourceSubSequence.Value && SourceSubSequence.Value.Get() != InSequence;
		if (bIsValidSubSequence)
		{
			if (bCancelled)
			{
				AssetsToCleanUp.Add(SourceSubSequence.Value.Get());
			}
			else if (Settings.bSaveRecordedAssets)
			{
				const bool bIsPossessable = TakesUtils::IsSourcePossessable(SourceSubSequence.Key);
				TakesUtils::SaveAsset(SourceSubSequence.Value, bIsPossessable);
			}
		}
	}

	SourceSubSequenceMap.Empty();
	ActiveSubSections.Empty();
	CreatedManifestSerializers.Empty();
	CachedManifestSerializer = nullptr;
	CachedLevelSequence = nullptr;

#if WITH_EDITOR
	if (GEditor && AssetsToCleanUp.Num() > 0)
	{
		ObjectTools::ForceDeleteObjects(AssetsToCleanUp, false);
	}
#endif // WITH_EDITOR
}

ULevelSequence* UTakeRecorderSources::FindExistingSubSequence(const ULevelSequence* InRootSequence, const FString& InSubSequenceAssetName)
{
	if (const UMovieSceneSubTrack* SubTrack = InRootSequence->GetMovieScene()->FindTrack<UMovieSceneSubTrack>())
	{
		// Look at each section in the track to see if it has the same name as our new SubSequence name.
		for (UMovieSceneSection* Section : SubTrack->GetAllSections())
		{
			const UMovieSceneSubSection* SubSection = CastChecked<UMovieSceneSubSection>(Section);
			if (FPaths::GetBaseFilename(SubSection->GetSequence()->GetPathName()) == InSubSequenceAssetName)
			{
				return CastChecked<ULevelSequence>(SubSection->GetSequence());
			}
		}
	}

	return nullptr;
}

ULevelSequence* UTakeRecorderSources::CreateSubSequenceForSource(ULevelSequence* InRootSequence, const FString& SubSequenceTrackName,
	const FString& SubSequenceAssetName, const FTakeRecorderSourcesSettings& InSettings)
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	// We want to sanitize the object names because sometimes they come from names with spaces and other invalid characters in them.
	const FString& SequenceDirectory = FPaths::GetPath(InRootSequence->GetPathName());
	FString SequenceName = FPaths::GetBaseFilename(InRootSequence->GetPathName());

	// We need to check the Root Sequence to see if they already have a sub-sequence with this name so that we duplicate the right
	// sequence and re-use that, instead of just making a new blank sequence every time. This will help in cases where they've done a recording, 
	// modified a sub-sequence and want to record from that setup. Each source will individually remove any old data inside the Sub Sequence
	// so we don't have to worry about any data the user added via Sequencer unrelated to what they recorded.
	ULevelSequence* ExistingSubSequence = FindExistingSubSequence(InRootSequence, SubSequenceAssetName);
	if (ExistingSubSequence)
	{
		UE_LOGF(LogTakesCore, Log, "Found existing sub-section for source %ls, duplicating sub-section for recording into.", *SubSequenceAssetName);
	}
	
	FString SubSequenceDirectory;

#if WITH_EDITOR
	if (const UTakeMetaData* AssetMetaData = InRootSequence->FindMetaData<UTakeMetaData>())
	{
		// User specified subsequence directory.
		UTakeRecorderNamingTokensContext* Context = NewObject<UTakeRecorderNamingTokensContext>();
		Context->TakeMetaData = AssetMetaData;
		SubSequenceDirectory = InSettings.SubSequenceDirectory;
		SubSequenceDirectory = AssetMetaData->GenerateAssetPath(SubSequenceDirectory, Context);
	}
#endif // WITH_EDITOR
	
	if (SubSequenceDirectory.IsEmpty())
	{
		// Fallback to original behavior.
		SubSequenceDirectory = FString::Printf(TEXT("%s_Subscenes"), *SequenceName);
	}
	
	FString NewPath = FString::Printf(TEXT("%s/%s/%s"), *SequenceDirectory, *SubSequenceDirectory, *SubSequenceAssetName);

	ULevelSequence* OutAsset = nullptr;
	TakesUtils::CreateNewAssetPackage<ULevelSequence>(NewPath, OutAsset, nullptr, ExistingSubSequence, InRootSequence->GetClass());
	if (OutAsset)
	{

		OutAsset->Initialize();

		// We only set their tick resolution/display rate if we're creating the sub-scene from scratch. If we created it in the
		// past it will have the right resolution, but if the user modified it then we will preserve their desired resolution.
		if (!ExistingSubSequence)
		{
			// Movie scene should not be transactional during the recording process
			OutAsset->GetMovieScene()->ClearFlags(RF_Transactional);

			OutAsset->GetMovieScene()->SetTickResolutionDirectly(InRootSequence->GetMovieScene()->GetTickResolution());
			OutAsset->GetMovieScene()->SetDisplayRate(InRootSequence->GetMovieScene()->GetDisplayRate());
		}

#if WITH_EDITOR
		if (UTakeMetaData* TakeMetaData = InRootSequence->FindMetaData<UTakeMetaData>())
		{
			UTakeMetaData* OutTakeMetaData = OutAsset->CopyMetaData(TakeMetaData);

			// Tack on the sub sequence name so that it's unique from the root sequence
			OutTakeMetaData->SetSlate(TakeMetaData->GetSlate() + TEXT("_") + SubSequenceTrackName, false);
		}
#endif // WITH_EDITOR

		OutAsset->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(OutAsset);
	}

	return OutAsset;
}

UMovieSceneFolder* UTakeRecorderSources::AddFolderForSource(const UTakeRecorderSource* InSource, UMovieScene* InMovieScene)
{
	check(InSource);
	check(InMovieScene);

	if (!InSource->ShouldCreateFolderForContents())
	{
		return nullptr;
	}

	// The TakeRecorderSources needs to create Sequencer UI folders to put each Source into so that Sources are not creating
	// their own folder structures inside of sub-sequences. This folder structure is designed to match the structure in
	// the Take Recorder UI, which is currently not customizable. If that becomes customizable this code should be updated
	// to ensure the created folder structure matches the one visible in the Take Recorder UI.

	// Search the Movie Scene for a folder with this name
	UMovieSceneFolder* FolderToUse = nullptr;

	FName FolderName = *InSource->GetName();

#if WITH_EDITOR
	// Currently we use the category that the Source is filed under as this is what the UI currently sorts by.
	FolderName = FName(*InSource->GetClass()->GetMetaData(FName("Category")));
	for (UMovieSceneFolder* Folder : InMovieScene->GetRootFolders())
	{
		if (Folder->GetFolderName() == FolderName)
		{
			FolderToUse = Folder;
			break;
		}
	}
#endif // WITH_EDITOR

	// If we didn't find a folder with this name we're going to go ahead and create a new folder
	if (FolderToUse == nullptr)
	{
		FolderToUse = NewObject<UMovieSceneFolder>(InMovieScene, NAME_None, RF_Transactional);
		FolderToUse->SetFolderName(FolderName);
	#if WITH_EDITOR
		InMovieScene->AddRootFolder(FolderToUse);
	#endif // WITH_EDITOR
	}

#if WITH_EDITOR
	// We want to expand these folders in the Sequencer UI (since these are visible as they record).
	InMovieScene->GetEditorData().ExpansionStates.FindOrAdd(FolderName.ToString()) = FMovieSceneExpansionState(true);
#endif // WITH_EDITOR

	return FolderToUse;
}

void UTakeRecorderSources::RemoveRedundantTracks()
{
	TArray<FGuid> ReferencedBindings;
	for (auto SourceSubSequence : SourceSubSequenceMap)
	{
		ULevelSequence* LevelSequence = SourceSubSequence.Value;
		if (!LevelSequence)
		{
			continue;
		}

		UMovieScene* MovieScene = LevelSequence->GetMovieScene();
		if (!MovieScene)
		{
			continue;
		}

		for (UMovieSceneSection* Section : MovieScene->GetAllSections())
		{
			Section->GetReferencedBindings(ReferencedBindings);
		}
	}


	for (auto SourceSubSequence : SourceSubSequenceMap)
	{
		ULevelSequence* LevelSequence = SourceSubSequence.Value;
		if (!LevelSequence)
		{
			continue;
		}

		UMovieScene* MovieScene = LevelSequence->GetMovieScene();
		if (!MovieScene)
		{
			continue;
		}

		TArray<FGuid> ParentBindings;
		for (const FMovieSceneBinding& Binding : ((const UMovieScene*)MovieScene)->GetBindings())
		{
			FMovieScenePossessable* Possessable = MovieScene->FindPossessable(Binding.GetObjectGuid());
			if (Possessable)
			{
				ParentBindings.Add(Possessable->GetParent());
			}
		}

		TArray<FGuid> BindingsToRemove;
		for (const FMovieSceneBinding& Binding : ((const UMovieScene*)MovieScene)->GetBindings())
		{
			if (Binding.GetTracks().Num() == 0 && !ReferencedBindings.Contains(Binding.GetObjectGuid()) && !ParentBindings.Contains(Binding.GetObjectGuid()))
			{
				BindingsToRemove.Add(Binding.GetObjectGuid());
			}
		}

		if (BindingsToRemove.Num() == 0)
		{
			continue;
		}

		for (FGuid BindingToRemove : BindingsToRemove)
		{
			MovieScene->RemovePossessable(BindingToRemove);
		}

		UE_LOGF(LogTakesCore, Log, "Removed %d unused object bindings in (%ls)", BindingsToRemove.Num(), *LevelSequence->GetName());
	}
}


int32 UTakeRecorderSources::RenderStatTakeSources(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	check(GEngine);
	UFont* Font = FPlatformProperties::SupportsWindowedMode() ? GEngine->GetSmallFont() : GEngine->GetMediumFont();
	const int32 RowHeight = FMath::TruncToInt(Font->GetMaxCharHeight());

	float CharWidth, CharHeight;
	Font->GetCharSize(TEXT(' '), CharWidth, CharHeight);
	auto RenderStatForSource = [RowHeight, World, Viewport, Font, Canvas, CharWidth, X, &Y](UTakeRecorderSource* Source) mutable
	{
		FText DisplayText = Source->GetDisplayText();
		FString DrawString = FString::Printf(TEXT("TakeRecorderSource: %s"), *DisplayText.ToString());
		X = X - Font->GetStringSize(*DrawString);
		Canvas->DrawShadowedString(X, Y, *DrawString, Font, FColor::Green);
		Y += RowHeight;
	};

	int32 NumSources = 0;
	for (UTakeRecorderSource* Source : Sources)
	{
		if (Source && Source->bEnabled)
		{
			RenderStatForSource(Source);
			NumSources ++;
		}
	}

	if (NumSources == 0)
	{
		FString DisplayText = TEXT("No Take Recorder sources assigned");
		X = X - Font->GetStringSize(*DisplayText);

		Canvas->DrawShadowedString(X, Y, *DisplayText, Font, FColor::Red);
		Y += RowHeight;
	}
	return Y;
}

#undef LOCTEXT_NAMESPACE
