// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderSourcesManagerImpl.h"

#include "LevelSequence.h"
#include "TakeRecorderSource.h"
#include "TakeRecorderSources.h"

void UTakeRecorderSourcesManagerImpl::Initialize(FSubsystemCollectionBase& Collection)
{
}

void UTakeRecorderSourcesManagerImpl::Deinitialize()
{
	RuntimeSources.Empty();
}

UTakeRecorderSources* UTakeRecorderSourcesManagerImpl::CopySources(ULevelSequence* FromSequence, ULevelSequence* ToSequence)
{
	if (!IsValid(FromSequence) || !IsValid(ToSequence))
	{
		return nullptr;
	}

	if (UTakeRecorderSources* FoundSources = FindSources(FromSequence))
	{
	#if WITH_EDITOR
		// when in Editor, ensure they are added to the metadata, overwriting anything thats already there.
		return ToSequence->CopyMetaData<UTakeRecorderSources>(FoundSources);
	#else // WITH_EDITOR
		UTakeRecorderSources* CopiedSources = DuplicateObject(FoundSources, ToSequence);
		RuntimeSources.Add(ToSequence, CopiedSources);
		return CopiedSources;
	#endif // WITH_EDITOR
	}

	return nullptr;
}

UTakeRecorderSources* UTakeRecorderSourcesManagerImpl::CopySources(UTakeRecorderSources* InSources, ULevelSequence* ToSequence)
{
	if (!IsValid(ToSequence))
	{
		return nullptr;
	}

#if WITH_EDITOR
	// when in Editor, ensure they are added to the metadata, overwriting anything thats already there.
	return ToSequence->CopyMetaData<UTakeRecorderSources>(InSources);
#else // WITH_EDITOR
	UTakeRecorderSources* CopiedSources = DuplicateObject(InSources, ToSequence);
	RuntimeSources.Add(ToSequence, CopiedSources);
	return CopiedSources;
#endif // WITH_EDITOR
}

UTakeRecorderSources* UTakeRecorderSourcesManagerImpl::FindSources(const ULevelSequence* InSequence) const
{
	if (!IsValid(InSequence))
	{
		return nullptr;
	}

#if WITH_EDITOR
	// When in editor, we check the sequence metadata.
	if (UTakeRecorderSources* MetaDataSources = InSequence->FindMetaData<UTakeRecorderSources>())
	{
		return MetaDataSources;
	}
#else // WITH_EDITOR
	// check if the sources are stored for the sequence
	if (TObjectPtr<UTakeRecorderSources> const* SourcesPtr = RuntimeSources.Find(InSequence))
	{
		return *SourcesPtr;
	}
#endif // WITH_EDITOR

	return nullptr;
}

UTakeRecorderSources* UTakeRecorderSourcesManagerImpl::FindOrAddSources(ULevelSequence* InSequence)
{
	if (!IsValid(InSequence))
	{
		return nullptr;
	}

#if WITH_EDITOR
	return InSequence->FindOrAddMetaData<UTakeRecorderSources>();
#else // WITH_EDITOR
	if (UTakeRecorderSources* FoundSources = FindSources(InSequence))
	{
		return FoundSources;
	}

	UTakeRecorderSources* NewSources = NewObject<UTakeRecorderSources>(InSequence);
	RuntimeSources.Add(InSequence, NewSources);
	return NewSources;
#endif // WITH_EDITOR
}

bool UTakeRecorderSourcesManagerImpl::HasSources(const ULevelSequence* InSequence) const
{
	return FindSources(InSequence) != nullptr;
}

UTakeRecorderSource* UTakeRecorderSourcesManagerImpl::AddSource(ULevelSequence* InSequence, const TSubclassOf<UTakeRecorderSource> InSourceClass)
{
	if (UTakeRecorderSources* FoundSources = FindOrAddSources(InSequence))
	{
		return FoundSources->AddSource(InSourceClass);
	}
	return nullptr;
}

void UTakeRecorderSourcesManagerImpl::RemoveSources(ULevelSequence* InSequence)
{
	if (!IsValid(InSequence))
	{
		return;
	}

#if WITH_EDITOR
	InSequence->RemoveMetaData<UTakeRecorderSources>();
#else // WITH_EDITOR
	RuntimeSources.Remove(InSequence);
#endif // WITH_EDITOR
}

void UTakeRecorderSourcesManagerImpl::RemoveSource(ULevelSequence* InSequence, UTakeRecorderSource* InSource)
{
	if (!IsValid(InSource))
	{
		return;
	}
	
	if (UTakeRecorderSources* FoundSources = FindSources(InSequence))
	{
		FoundSources->RemoveSource(InSource);
	}
}
