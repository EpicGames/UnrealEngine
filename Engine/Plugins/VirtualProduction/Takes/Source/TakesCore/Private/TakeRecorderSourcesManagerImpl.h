// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Subsystems/EngineSubsystem.h"
#include "ITakeRecorderSourcesManager.h"

#include "TakeRecorderSourcesManagerImpl.generated.h"

/**
 * Implementation of the ITakeRecorderSourcesManager interface, implemented as an Engine subsystem.
 * Handles Take Recorder Sources in both editor and runtime contexts.
 * At editor time, the LevelSequence metadata is used to track and store the Take Recorder sources.
 * At runtime, a transient map is used to store the sources per level sequence. This is designed to be transient and unserialized due to
 * requirements of recording at runtime, and setups most likely be game based.
 */
UCLASS(MinimalAPI)
class UTakeRecorderSourcesManagerImpl final : 
	public UEngineSubsystem, 
	public ITakeRecorderSourcesManager
{
	GENERATED_BODY()

public:
	//~ Begin UEngineSubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End UEngineSubsystem

	//~ Begin ITakeRecorderSourcesManager
	virtual UTakeRecorderSources* CopySources(ULevelSequence* FromSequence, ULevelSequence* ToSequence) override;
	virtual UTakeRecorderSources* CopySources(UTakeRecorderSources* InSources, ULevelSequence* ToSequence) override;
	virtual UTakeRecorderSources* FindSources(const ULevelSequence* InSequence) const override;
	virtual UTakeRecorderSources* FindOrAddSources(ULevelSequence* InSequence) override;
	virtual bool HasSources(const ULevelSequence* InSequence) const override;
	virtual UTakeRecorderSource* AddSource(ULevelSequence* InSequence, const TSubclassOf<UTakeRecorderSource> InSourceClass) override;
	virtual void RemoveSource(ULevelSequence* InSequence, UTakeRecorderSource* InSource) override;
	virtual void RemoveSources(ULevelSequence* InSequence) override;
	//~ End ITakeRecorderSourcesManager

private:
	/** 
	 * Sources map used at runtime to store the TakeRecorderSources per LevelSequence.
	 * This map can be transient as we have no intention of ever serializing it.
	 * It is used only at runtime for adding sources to record, and does not need to be persisted out.
	 */
	UPROPERTY(Transient)
	TMap<TObjectPtr<ULevelSequence>, TObjectPtr<UTakeRecorderSources>> RuntimeSources;
};
