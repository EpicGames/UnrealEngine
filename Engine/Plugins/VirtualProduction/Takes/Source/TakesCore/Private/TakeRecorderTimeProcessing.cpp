// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderTimeProcessing.h"

#include "LevelSequence.h"
#include "MovieScene.h"
#include "MovieSceneTakeSection.h"
#include "MovieSceneTakeTrack.h"
#include "TakeMetaData.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Curves/RealCurve.h"

namespace UE::TakesCore
{
void ProcessRecordedTimes(ULevelSequence* InSequence, UMovieSceneTakeTrack* TakeTrack, const TOptional<TRange<FFrameNumber>>& FrameRange,
	const FArrayOfRecordedTimePairs& RecordedTimes)
{
	check(TakeTrack);
	UMovieScene* MovieScene = InSequence->GetMovieScene();

	// In case we need it later, get the earliest timecode source *before* we
	// add the take section, since its timecode source will be default
	// constructed as all zeros and might accidentally compare as earliest.
	const FMovieSceneTimecodeSource EarliestTimecodeSource = MovieScene->GetEarliestTimecodeSource();

	TakeTrack->RemoveAllAnimationData();

	UMovieSceneTakeSection* TakeSection = Cast<UMovieSceneTakeSection>(TakeTrack->CreateNewSection());
	TakeTrack->AddSection(*TakeSection);

	if (FrameRange.IsSet())
	{
		FTimecodeExtractionData Data = ExtractIntoTimecodeArrays(MovieScene, *FrameRange, RecordedTimes);
		TakeSection->HoursCurve.Set(Data.Times, MoveTemp(Data.Hours));
		TakeSection->MinutesCurve.Set(Data.Times, MoveTemp(Data.Minutes));
		TakeSection->SecondsCurve.Set(Data.Times, MoveTemp(Data.Seconds));
		TakeSection->FramesCurve.Set(Data.Times, MoveTemp(Data.Frames));
		// On the last use of Data.Times, we can move.
		TakeSection->SubFramesCurve.Set(MoveTemp(Data.Times), MoveTemp(Data.SubFrames));
		TakeSection->RateCurve.SetDefault(Data.TCRate.AsDecimal());
	}

	// Since the take section was created post recording here in this
	// function, it wasn't available at the start of recording to have
	// its timecode source set with the other sections, so we set it here.
	if (TakeSection->HoursCurve.GetNumKeys() > 0)
	{
		// We populated the take section's timecode curves with data, so
		// use the first values as the timecode source.
		const int32 Hours = TakeSection->HoursCurve.GetValues()[0];
		const int32 Minutes = TakeSection->MinutesCurve.GetValues()[0];
		const int32 Seconds = TakeSection->SecondsCurve.GetValues()[0];
		const int32 Frames = TakeSection->FramesCurve.GetValues()[0];
		const bool bIsDropFrame = false;
		const FTimecode Timecode(Hours, Minutes, Seconds, Frames, bIsDropFrame);
		TakeSection->TimecodeSource = FMovieSceneTimecodeSource(Timecode);
	}
	else
	{
		// Otherwise, adopt the earliest timecode source from one of the movie
		// scene's other sections as the timecode source for the take section.
		// This case is unlikely.
		TakeSection->TimecodeSource = EarliestTimecodeSource;
	}

#if WITH_EDITORONLY_DATA
	if (UTakeMetaData* TakeMetaData = InSequence->FindMetaData<UTakeMetaData>())
	{
		TakeSection->Slate.SetDefault(FString::Printf(TEXT("%s_%d"), *TakeMetaData->GetSlate(), TakeMetaData->GetTakeNumber()));
	}
#endif // WITH_EDITORONLY_DATA

	if (TakeSection->GetAutoSizeRange().IsSet())
	{
		TakeSection->SetRange(TakeSection->GetAutoSizeRange().GetValue());
	}
}

FTimecodeExtractionData ExtractIntoTimecodeArrays(
	UMovieScene* InMovieScene, const TRange<FFrameNumber>& InRange, const FArrayOfRecordedTimePairs& InRecordedTimes,
	ETimecodeExtractionFlags InFlags
	)
{
	const int32 Num = InRecordedTimes.Num();
	const auto Reserve = [InFlags, Num](auto& Array, ETimecodeExtractionFlags RequiredFlag)
	{
		if (EnumHasAnyFlags(InFlags, RequiredFlag))
		{
			Array.Reserve(Num);
		}
	};
	const auto AddItem = [InFlags](auto& Array, const auto& Value, ETimecodeExtractionFlags RequiredFlag)
	{
		if (EnumHasAnyFlags(InFlags, RequiredFlag))
		{
			Array.Add(Value);
		}
	};
	
	TArray<int32> Hours, Minutes, Seconds, Frames;
	TArray<FMovieSceneFloatValue> SubFrames;
	TArray<FFrameNumber> Times;
	Reserve(Hours, ETimecodeExtractionFlags::Hours);
	Reserve(Minutes, ETimecodeExtractionFlags::Minutes);
	Reserve(Seconds, ETimecodeExtractionFlags::Seconds);
	Reserve(Frames, ETimecodeExtractionFlags::Frames);
	Reserve(SubFrames, ETimecodeExtractionFlags::Subframes);
	Reserve(Times, ETimecodeExtractionFlags::Times);
	
	const FFrameRate TickResolution = InMovieScene->GetTickResolution();
	const FFrameRate DisplayRate = InMovieScene->GetDisplayRate();
	FFrameRate TCRate = TickResolution;
	for (const TPair<FQualifiedFrameTime, FQualifiedFrameTime>& RecordedTimePair : InRecordedTimes)
	{
		const FFrameNumber FrameNumber = RecordedTimePair.Key.Time.FrameNumber;
		if (!InRange.Contains(FrameNumber))
		{
			continue;
		}

		const FTimecode Timecode = RecordedTimePair.Value.ToTimecode();
		TCRate = RecordedTimePair.Value.Rate;
		AddItem(Hours, Timecode.Hours, ETimecodeExtractionFlags::Hours);
		AddItem(Minutes, Timecode.Minutes, ETimecodeExtractionFlags::Minutes);
		AddItem(Seconds, Timecode.Seconds, ETimecodeExtractionFlags::Seconds);
		AddItem(Frames, Timecode.Frames, ETimecodeExtractionFlags::Frames);

		FMovieSceneFloatValue SubFrame;
		if (RecordedTimePair.Value.Time.GetSubFrame() > 0)
		{
			// If the Timecode provided gave us a subframe value then we should use that value.  Otherwise, we should compute
			// the most appropriate value based on the timecode rate.
			SubFrame.Value = RecordedTimePair.Value.Time.GetSubFrame();
		}
		else
		{
			const FFrameTime FrameTime = FFrameRate::TransformTime(RecordedTimePair.Key.Time, TickResolution, DisplayRate);
			const FQualifiedFrameTime FrameTimeAsTimeCodeRate(FrameTime, TCRate);
			SubFrame.Value = FrameTimeAsTimeCodeRate.Time.GetSubFrame();
		}

		SubFrame.InterpMode = ERichCurveInterpMode::RCIM_Linear;
		AddItem(SubFrames, SubFrame, ETimecodeExtractionFlags::Subframes);
		AddItem(Times, FrameNumber, ETimecodeExtractionFlags::Times);
	}

	Hours.Shrink();
	Minutes.Shrink();
	Seconds.Shrink();
	Frames.Shrink();
	SubFrames.Shrink();
	Times.Shrink();
	return FTimecodeExtractionData
	{
		MoveTemp(Hours), MoveTemp(Minutes), MoveTemp(Seconds), MoveTemp(Frames), MoveTemp(SubFrames), MoveTemp(Times), TCRate
	};
}
}
