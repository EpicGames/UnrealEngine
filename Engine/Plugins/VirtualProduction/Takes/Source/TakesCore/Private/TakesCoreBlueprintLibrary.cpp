// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakesCoreBlueprintLibrary.h"
#include "TakeMetaData.h"
#include "LevelSequence.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakesCoreBlueprintLibrary)

namespace
{
	static UTakesCoreBlueprintLibrary::FOnTakeRecorderSlateChanged TakeRecorderSlateChanged;
	static UTakesCoreBlueprintLibrary::FOnTakeRecorderTakeNumberChanged TakeRecorderTakeNumberChanged;
}

int32 UTakesCoreBlueprintLibrary::ComputeNextTakeNumber(const FString& Slate)
{
	int32 MaxTake = 0;

	for (const FAssetData& Asset : FindTakes(Slate))
	{
		FAssetDataTagMapSharedView::FFindTagResult TakeNumberTag = Asset.TagsAndValues.FindTag(UTakeMetaData::AssetRegistryTag_TakeNumber);

		int32 ThisTakeNumber = 0;
		if (TakeNumberTag.IsSet() && LexTryParseString(ThisTakeNumber, *TakeNumberTag.GetValue()))
		{
			MaxTake = FMath::Max(MaxTake, ThisTakeNumber);
		}
	}

	return MaxTake + 1;
}

TArray<FAssetData> UTakesCoreBlueprintLibrary::FindTakes(const FString& Slate, int32 TakeNumber)
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();

	FARFilter Filter;
	Filter.ClassPaths.Add(ULevelSequence::StaticClass()->GetClassPathName());
	Filter.TagsAndValues.Add(UTakeMetaData::AssetRegistryTag_Slate, Slate);
	Filter.bRecursiveClasses = true; // Needed for other types of LevelSequences, like CineAssembly

	TArray<FAssetData> AllAssets;
	AssetRegistry.GetAssets(Filter, AllAssets);

	// Filter.TagsAndValues matches *any* tag/value pair, rather than *all*, so we have to run the filter twice
	if (TakeNumber > 0 && AllAssets.Num() > 0)
	{
		FARFilter TakeFilter;
		TakeFilter.TagsAndValues.Add(UTakeMetaData::AssetRegistryTag_TakeNumber, LexToString(TakeNumber));
		AssetRegistry.RunAssetsThroughFilter(AllAssets, TakeFilter);
	}
	
	return AllAssets;
}

void UTakesCoreBlueprintLibrary::SetOnTakeRecorderSlateChanged(FOnTakeRecorderSlateChanged OnTakeRecorderSlateChanged)
{
	TakeRecorderSlateChanged = OnTakeRecorderSlateChanged;
};

void UTakesCoreBlueprintLibrary::SetOnTakeRecorderTakeNumberChanged(FOnTakeRecorderTakeNumberChanged OnTakeRecorderTakeNumberChanged)
{
	TakeRecorderTakeNumberChanged = OnTakeRecorderTakeNumberChanged;
};

void UTakesCoreBlueprintLibrary::OnTakeRecorderSlateChanged(const FString& InSlate)
{
	TakeRecorderSlateChanged.ExecuteIfBound(InSlate);
}

void UTakesCoreBlueprintLibrary::OnTakeRecorderTakeNumberChanged(int32 InTakeNumber)
{
	TakeRecorderTakeNumberChanged.ExecuteIfBound(InTakeNumber);
}



