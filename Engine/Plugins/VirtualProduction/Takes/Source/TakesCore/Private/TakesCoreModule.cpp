// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakesCoreLog.h"
#include "TakeMetaData.h"

#include "LevelSequence.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "TakeData.h"
#include "MovieSceneToolsModule.h"
#endif // WITH_EDITOR

LLM_DEFINE_TAG(Takes_TakesCore);

class FTakesCoreModule : public IModuleInterface
{
	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(Takes_TakesCore);

		// Ensure the level sequence module is loaded
		FModuleManager::Get().LoadModuleChecked("LevelSequence");

		LevelSequenceCDO = GetMutableDefault<ULevelSequence>();

	
	#if WITH_EDITOR
		TakeData = MakeShareable(new FTakesCoreTakeData);
		// Add empty take meta data to the ULevelSequence CDO to ensure that
		// asset registry tooltips show up in the editor
		UTakeMetaData* MetaData = GetMutableDefault<ULevelSequence>()->FindOrAddMetaData<UTakeMetaData>();
		MetaData->SetFlags(RF_Transient);

		// Register take data with movie scene tools so sequencer knows how to switch takes
		FMovieSceneToolsModule::Get().RegisterTakeData(TakeData.Get());
	#endif // WITH_EDITOR
	}

	virtual void ShutdownModule() override
	{
		LLM_SCOPE_BYTAG(Takes_TakesCore);

	#if WITH_EDITOR
		if (ULevelSequence* CDO = LevelSequenceCDO.Get())
		{
			CDO->RemoveMetaData<UTakeMetaData>();
		}

		FMovieSceneToolsModule::Get().UnregisterTakeData(TakeData.Get());
	#endif // WITH_EDITOR
	}

	// Weak ptr to the level sequence CDO so we can gracefully remove the take meta-data on shutdown module
	// without crashing when ShutdownModule is called after the CDO has been destroyed.
	TWeakObjectPtr<ULevelSequence> LevelSequenceCDO;

#if WITH_EDITORONLY_DATA
	TSharedPtr<IMovieSceneToolsTakeData> TakeData;
#endif // WITH_EDITORONLY_DATA
};

IMPLEMENT_MODULE(FTakesCoreModule, TakesCore);
DEFINE_LOG_CATEGORY(LogTakesCore);
