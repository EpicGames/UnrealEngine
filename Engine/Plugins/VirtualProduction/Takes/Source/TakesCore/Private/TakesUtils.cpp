// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakesUtils.h"
#include "AssetRegistry/AssetData.h"

#include "Engine/Engine.h"
#include "Engine/World.h"
#include "LevelSequence.h"
#include "Math/Range.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneSection.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneTrack.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "TakeRecorderSource.h"
#include "TrackRecorders/IMovieSceneTrackRecorderHost.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "UObject/SavePackage.h"
#include "ITakeRecorderSourcesManager.h"
#include "MovieSceneCommonHelpers.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "MovieSceneBindingReferences.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "ActorSequenceInformation.h"

#include "Serialization/ArchiveReplaceOrClearExternalReferences.h"

#if WITH_EDITOR
#include "ILevelSequenceEditorToolkit.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "TakesUtils"

namespace TakesUtils
{

static bool bSaveTakesWithNoErrorFlag = false;
static FAutoConsoleVariableRef  CVarSaveTakesWithNoError(TEXT("Takes.SaveTakesWithNoErrorFlag"), bSaveTakesWithNoErrorFlag, TEXT("Enable the no error flag for package saves. By default, we always want to report errors back to the user."));

static bool bSaveTakesAsync = false;
static FAutoConsoleVariableRef  CVarSaveTakesAsyncSave(TEXT("Takes.SaveTakesWithAsyncSave"), bSaveTakesAsync, TEXT("Enable the async save flag for take recorder."));
	
class FArchiveClearExternalReferences : public FArchiveReplaceOrClearExternalReferences<UObject>
{
	using Super = FArchiveReplaceOrClearExternalReferences<UObject>;

	TArray<UObject*> RemovedObjects;
public:
	
	FArchiveClearExternalReferences(UObject* InSearchObject, const TMap<UObject*, UObject*>& InReplacementMap, UPackage* InDestPackage)
		: Super(InSearchObject, InReplacementMap, InDestPackage, EArchiveReplaceObjectFlags::NullPrivateRefs | EArchiveReplaceObjectFlags::DelayStart )
	{
		SerializeSearchObject();
	}

	const TArray<UObject*>& GetRemovedObjects() const { return RemovedObjects; }

	virtual FArchive& operator<<(UObject*& Obj) override
	{
		UObject* Original = Obj;
		// We don't want to clear references if we're dealing with a spawnable.
		if (!IsObjectPossessable(Obj))
		{
			Super::operator<<(Obj);
		}
		if (Original && !Obj)
		{
			RemovedObjects.AddUnique(Original);
		}
		
		return *this;
	}
	
	/** Checks if the given object is considered possessable. */
	bool IsObjectPossessable(UObject* Obj) const
	{
		if (const ULevelSequence* LevelSequence = Cast<ULevelSequence>(SearchObject))
		{
			if (const AActor* Actor = Cast<AActor>(Obj))
			{
				// Actors that already have a possessable binding anywhere in the sequence hierarchy
				// (e.g. non-recorded actors driven by the source sequence whose bindings were
				// duplicated into the take) must always have their binding references preserved.
				// Using the global bRecordToPossessable fallback in IsActorPossessable would clear
				// these bindings when recording as spawnable, leaving them red/unresolvable.
				UE::TakesCore::FActorSequenceInformation SeqInfo(
					const_cast<ULevelSequence*>(LevelSequence), const_cast<AActor*>(Actor));
				if (SeqInfo.IsPossessable())
				{
					return true;
				}

				if (const UTakeRecorderSources* Sources = ITakeRecorderSourcesManager::GetChecked().FindSources(LevelSequence))
				{
					return Sources->IsActorPossessable(Actor);
				}
			}
		}
		
		return false;
	}
};

static void LogRemovedExternalObjects(UObject* ReferencingObject, const TArray<UObject*>& InRemovedObjects)
{
	if (InRemovedObjects.IsEmpty())
	{
		return;
	}

	const FString List = FString::JoinBy(InRemovedObjects, TEXT(", "), [](UObject* Object)
	{
		return ensure(Object) ? Object->GetPathName() : TEXT("null");
	});
	UE_LOGF(LogTakesCore, Warning, "While saving %ls, the following references were cleared because they were private and external: %ls",
		*ReferencingObject->GetPathName(), *List
		);
}


FString SanitizeInvalidChars(const FString& InText, const TCHAR* InvalidChars)
{
	FString SanitizedText = InText;

	const TCHAR* InvalidChar = InvalidChars ? InvalidChars : TEXT("");
	while (*InvalidChar)
	{
		SanitizedText.ReplaceCharInline(*InvalidChar, TCHAR('_'), ESearchCase::CaseSensitive);
		++InvalidChar;
	}

	return SanitizedText;
}

FString SanitizeObjectName(const FString& InObjectName)
{
	return SanitizeInvalidChars(InObjectName, INVALID_OBJECTNAME_CHARACTERS);
}

FString SanitizePackageName(const FString& InPackageName)
{
	FString SanitizedName = SanitizeInvalidChars(InPackageName, INVALID_LONGPACKAGE_CHARACTERS);

	// Coalesce multiple contiguous slashes into a single slash
	int32 CharIndex = 0;
	while (CharIndex < SanitizedName.Len())
	{
		if (SanitizedName[CharIndex] == TEXT('/'))
		{
			int32 SlashCount = 1;
			while (CharIndex + SlashCount < SanitizedName.Len() &&
				SanitizedName[CharIndex + SlashCount] == TEXT('/'))
			{
				SlashCount++;
			}

			if (SlashCount > 1)
			{
				SanitizedName.RemoveAt(CharIndex + 1, SlashCount - 1, EAllowShrinking::No);
			}
		}

		CharIndex++;
	}

	return SanitizedName;
}

/** Helper function - get the first PIE world (or first PIE client world if there is more than one) */
UWorld* GetFirstPIEWorld()
{
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.World()->IsPlayInEditor())
		{
			if (Context.World()->GetNetMode() == ENetMode::NM_Standalone ||
				(Context.World()->GetNetMode() == ENetMode::NM_Client && Context.PIEInstance == 2))
			{
				return Context.World();
			}
		}
	}

	return nullptr;
}

void ClampPlaybackRangeToEncompassAllSections(UMovieScene* InMovieScene, bool bUpperBoundOnly)
{
	check(InMovieScene);

	TOptional < TRange<FFrameNumber> > PlayRange;

	TArray<UMovieSceneSection*> MovieSceneSections = InMovieScene->GetAllSections();
	for (UMovieSceneSection* Section : MovieSceneSections)
	{
		TRange<FFrameNumber> SectionRange = Section->GetRange();
		if (SectionRange.GetLowerBound().IsClosed() && SectionRange.GetUpperBound().IsClosed())
		{
			if (!PlayRange.IsSet())
			{
				PlayRange = SectionRange;
			}
			else
			{
				PlayRange = TRange<FFrameNumber>::Hull(PlayRange.GetValue(), SectionRange);
			}
		}
	}

	if (!PlayRange.IsSet())
	{
		return;
	}

	// Extend only the upper bound because the start was set at the beginning of recording
	if (bUpperBoundOnly)
	{
		PlayRange.GetValue().SetLowerBoundValue(InMovieScene->GetPlaybackRange().GetLowerBoundValue());
	}

	InMovieScene->SetPlaybackRange(PlayRange.GetValue());
}

void ResetViewAndWorkRange(UMovieScene* InMovieScene)
{
	TRange<FFrameNumber> PlayRange = InMovieScene->GetPlaybackRange();

	// Initialize the working and view range with a little bit more space
	FFrameRate  TickResolution = InMovieScene->GetTickResolution();
	const double OutputViewSize = PlayRange.Size<FFrameNumber>() / TickResolution;
	const double OutputChange = OutputViewSize * 0.1;

#if WITH_EDITOR
	TRange<double> NewRange = UE::MovieScene::ExpandRange(PlayRange / TickResolution, OutputChange);
	FMovieSceneEditorData& EditorData = InMovieScene->GetEditorData();
	EditorData.ViewStart = EditorData.WorkStart = NewRange.GetLowerBoundValue();
	EditorData.ViewEnd = EditorData.WorkEnd = NewRange.GetUpperBoundValue();
#endif // WITH_EDITOR
}

void CleanupExternalReferences(UObject* InObject, UPackage* InPackage)
{
	// We want to remove any external references before calling package save because those references will
	// prevent users from saving because of checks in the package save system. 
	// FArchiveReplaceOrClearExternalReferences handles recursively serializing subobjects, such as components.
	const FArchiveClearExternalReferences ReplaceActorInvalidReferences(InObject, {}, InPackage);
	LogRemovedExternalObjects(InObject, ReplaceActorInvalidReferences.GetRemovedObjects());
}

void CleanupExternalReferences(UObject* InObject)
{
	UPackage* const Package = InObject->GetOutermost();
	CleanupExternalReferences(InObject, Package);
}
	
void SaveAsset(UObject* InObject, bool bAlwaysKeepExternalReferences)
{
	if (!InObject)
	{
		return;
	}

	// auto-save asset outside of the editor
	UPackage* const Package = InObject->GetOutermost();
	FString const PackageName = Package->GetName();
	FString const PackageFileName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());

	if (IFileManager::Get().IsReadOnly(*PackageFileName))
	{
		UE_LOGF(LogTakesCore, Error, "Could not save read only file: %ls", *PackageFileName);
		return;
	}

	double StartTime = FPlatformTime::Seconds();

	if (!bAlwaysKeepExternalReferences)
	{
		CleanupExternalReferences(InObject, Package);		
	}

	FSavePackageArgs SaveArgs;
	SaveArgs.TopLevelFlags = RF_Standalone;
	if (bSaveTakesAsync)
	{
		SaveArgs.SaveFlags = SaveArgs.SaveFlags | SAVE_Async;		
	}

	if (bSaveTakesWithNoErrorFlag)
	{
		SaveArgs.SaveFlags = SaveArgs.SaveFlags | SAVE_NoError;
	}

	if (!Package->IsFullyLoaded())
	{
		Package->FullyLoad();
	}

	if (UPackage::SavePackage(Package, NULL, *PackageFileName, SaveArgs))
	{
		if (bSaveTakesAsync)
		{
			UPackage::WaitForAsyncFileWrites();
		}
		double ElapsedTime = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogTakesCore, Log, "Saved %ls in %0.2f seconds", *PackageName, ElapsedTime);
	}
	else
	{
		UE_LOGF(LogTakesCore, Log, "Failed to save %ls.", *PackageName);
	}
}

void CreateCameraCutTrack(ULevelSequence* LevelSequence, const FGuid& RecordedCameraGuid, const FMovieSceneSequenceID& SequenceID, const TRange<FFrameNumber>& InRange)
{
	if (!RecordedCameraGuid.IsValid() || !LevelSequence)
	{
		return;
	}

	UMovieSceneTrack* CameraCutTrack = LevelSequence->GetMovieScene()->GetCameraCutTrack();
	if (CameraCutTrack && CameraCutTrack->GetAllSections().Num() > 1)
	{
		return;
	}


	if (!CameraCutTrack)
	{
		CameraCutTrack = LevelSequence->GetMovieScene()->AddCameraCutTrack(UMovieSceneCameraCutTrack::StaticClass());
	}
	else
	{
		CameraCutTrack->RemoveAllAnimationData();
	}

	UMovieSceneCameraCutSection* CameraCutSection = Cast<UMovieSceneCameraCutSection>(CameraCutTrack->CreateNewSection());
	CameraCutSection->SetCameraBindingID(UE::MovieScene::FRelativeObjectBindingID(RecordedCameraGuid, SequenceID));
	CameraCutSection->SetRange(InRange);
	CameraCutTrack->AddSection(*CameraCutSection);
}

UWorld* DiscoverSourceWorld()
{
	UWorld* SourceWorld = nullptr;

	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		if (WorldContext.WorldType == EWorldType::PIE || WorldContext.WorldType == EWorldType::Game )
		{
			SourceWorld = WorldContext.World();
			break;
		}
		if (WorldContext.WorldType == EWorldType::Editor)
		{
			SourceWorld = WorldContext.World();
		}
	}

	check(SourceWorld);
	return SourceWorld;
}

#if WITH_EDITOR
TSharedPtr<ISequencer> OpenSequencer(ULevelSequence* LevelSequence, FText* OutError)
{
	TSharedPtr<ISequencer> Sequencer;
	if ( GEditor != nullptr )
	{
		// Open the sequence and set the sequencer ptr
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(LevelSequence);

		IAssetEditorInstance*        AssetEditor         = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(LevelSequence, false);
		ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);

		Sequencer = LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;

		if (!Sequencer.IsValid() && OutError)
		{
			*OutError = FText::Format(LOCTEXT("FailedToOpenSequencerError", "Failed to open Sequencer for asset '{0}."), FText::FromString(LevelSequence->GetPathName()));
		}
	}
		
	return Sequencer;
}

FQualifiedFrameTime GetRecordTime(TSharedPtr<ISequencer> Sequencer, ULevelSequence* SequenceAsset,
	const FTimecode& TimecodeAtStart, bool bStartAtCurrentTimecode, float TimeDilation)
{
	FQualifiedFrameTime RecordTime;

	// Sequencer handles time dilation.
	RecordTime = Sequencer.IsValid()
		? Sequencer->GetGlobalTime()
		: GetRecordTime(SequenceAsset, TimecodeAtStart, bStartAtCurrentTimecode, TimeDilation);

	return RecordTime;
}

TSharedPtr<const UE::MovieScene::FSharedPlaybackState> FindSequencerSharedPlaybackState(ULevelSequence* LevelSequence)
{
	if (GEditor != nullptr)
	{
		if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			IAssetEditorInstance* AssetEditor = AssetEditorSubsystem->FindEditorForAsset(LevelSequence, false);
			if (ILevelSequenceEditorToolkit* Toolkit = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor))
			{
				if (TSharedPtr<ISequencer> Sequencer = Toolkit->GetSequencer())
				{
					return Sequencer->FindSharedPlaybackState();
				}
			}
		}
	}
	return nullptr;
}
#endif // WITH_EDITOR

FQualifiedFrameTime GetRecordTime(ULevelSequence* SequenceAsset,
	const FTimecode& TimecodeAtStart, bool bStartAtCurrentTimecode, float TimeDilation)
{
	FQualifiedFrameTime RecordTime;

	if (SequenceAsset)
	{
		if (UMovieScene* MovieScene = SequenceAsset->GetMovieScene())
		{
			const FFrameRate FrameRate = MovieScene->GetDisplayRate();
			const FFrameRate TickResolution = MovieScene->GetTickResolution();

			const FTimecode CurrentTimecode = FApp::GetTimecode();

			const FFrameTime CurrentFrameTime = FFrameRate::TransformTime(FFrameTime(CurrentTimecode.ToFrameNumber(FrameRate)), FrameRate, TickResolution);
			const FFrameTime StartFrameTime = FFrameRate::TransformTime(FFrameTime(TimecodeAtStart.ToFrameNumber(FrameRate)), FrameRate, TickResolution);

			// If we are managing time dilation, then we need to apply it to the delta only.
			const FFrameTime DeltaFrameTime = CurrentFrameTime - StartFrameTime;
			const FFrameTime DilatedFrameTime = DeltaFrameTime * TimeDilation;

			// Offset with the start time after dilation has been applied, if applicable.
			const FFrameTime FinalFrameTime = bStartAtCurrentTimecode
				? StartFrameTime + DilatedFrameTime
				: DilatedFrameTime;
			
			RecordTime = FQualifiedFrameTime(FinalFrameTime, TickResolution);
		}
	}

	return RecordTime;
}

FGuid ResolveActorFromSequence(const AActor* InActor, const ULevelSequence* CurrentSequence)
{
	UMovieScene* MovieScene = CurrentSequence->GetMovieScene();

	// Look through all Possessables in the sequence to see if there's one with the same name as the actor. We purposely do not look at 
	// Spawnables so that recording as spawnable will always create a new spawnable.
	for (int32 PossessableCount = 0; PossessableCount < MovieScene->GetPossessableCount(); ++PossessableCount)
	{
		const FMovieScenePossessable& Possessable = MovieScene->GetPossessable(PossessableCount);
	#if WITH_EDITOR
		// GetActorLabel is an editor only function, so for runtime, use just the name directly.
		// When we have a better way of identifying actors without their labels, this will need updating.
		if (Possessable.GetName() == InActor->GetActorLabel())
	#else // WITH_EDITOR
		if (Possessable.GetName() == InActor->GetName())
	#endif // WITH_EDITOR

		{
			return Possessable.GetGuid();
		}
	}

	// There's no Possessable with the same name as the actor, so this actor hasn't been added
	// to the sequence yet.
	return FGuid();
}

FGuid ResolveActorFromSequence(const AActor* InActor, const ULevelSequence* CurrentSequence, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	if (InActor == nullptr || CurrentSequence == nullptr)
	{
		return FGuid();
	}

	UMovieScene* MovieScene = CurrentSequence->GetMovieScene();
	const FMovieSceneBindingReferences* BindingRefs = CurrentSequence->GetBindingReferences();
	if (!BindingRefs)
	{
		return FGuid();
	}

	FMovieSceneBindingResolveParams BindingResolveParams;
	BindingResolveParams.Sequence = const_cast<ULevelSequence*>(CurrentSequence);
	BindingResolveParams.SequenceID = MovieSceneSequenceID::Root;

	UE::UniversalObjectLocator::FResolveParams LocatorResolveParams(InActor->GetWorld());

	for (int32 i = 0; i < MovieScene->GetPossessableCount(); ++i)
	{
		const FGuid Guid = MovieScene->GetPossessable(i).GetGuid();

		// Skip spawnables stored as possessables with a custom binding (modern style)
		if (const UMovieSceneCustomBinding* CustomBinding = BindingRefs->GetCustomBinding(Guid, 0))
		{
			if (CustomBinding->IsA<UMovieSceneSpawnableBindingBase>())
			{
				continue;
			}
		}

		BindingResolveParams.ObjectBindingID = Guid;

		TArray<UObject*, TInlineAllocator<1>> BoundObjects;
		BindingRefs->ResolveBinding(BindingResolveParams, LocatorResolveParams, SharedPlaybackState, BoundObjects);

		if (BoundObjects.Contains(const_cast<AActor*>(InActor)))
		{
			return Guid;
		}
	}

	return FGuid();
}

bool IsSourcePossessable(const UTakeRecorderSource* InSource)
{
	check(InSource);
	if (const IMovieSceneTrackRecorderHost* RecorderHost = InSource->AsTrackRecorderHost())
	{
		return RecorderHost->GetTrackRecorderSettings().bRecordToPossessable;
	}
	
	const UTakeRecorderSources* Sources = InSource->GetTypedOuter<UTakeRecorderSources>();
	check(Sources);
	return Sources->GetSettings().bRecordToPossessable;
}

bool DoesTrackHaveAnyKeys(UMovieSceneTrack& Track)
{
	for (UMovieSceneSection* Section : Track.GetAllSections())
	{
		if (!Section)
		{
			continue;
		}

		for (const FMovieSceneChannelEntry& Entry : Section->GetChannelProxy().GetAllEntries())
		{
			for (const FMovieSceneChannel* Channel : Entry.GetChannels())
			{
				if (Channel && Channel->GetNumKeys() > 0)
				{
					return true;
				}
			}
		}
	}

	return false;
}

}

#undef LOCTEXT_NAMESPACE
