// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSequence.h"
#include "GameFramework/Actor.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"

#define UE_API TAKESCORE_API

namespace UE::TakesCore
{

/**
 * Resolves and caches an actor's relationship to a level sequence hierarchy at construction time.
 * Determines whether the actor is bound as a possessable or spawnable, and which sequence and
 * sequence ID owns that binding.
 */
class FActorSequenceInformation
{
public:
	FActorSequenceInformation() = delete;

	/** Resolves binding information for InActor within InLevelSequence, acquiring a shared playback state automatically. */
	UE_API explicit FActorSequenceInformation(ULevelSequence* InLevelSequence, AActor* InActor);

	/** Resolves binding information for InActor within InLevelSequence using the provided shared playback state. */
	UE_API explicit FActorSequenceInformation(ULevelSequence* InLevelSequence, AActor* InActor, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> InSharedPlaybackState);

	/** Returns the binding GUID resolved at construction. Invalid if no binding was found. */
	const FGuid& GetCachedObjectBindingGuid() const { return CachedObjectBindingGuid; }

	/** Returns true if the actor has an active binding in the sequence hierarchy (spawnable or tracked possessable). */
	bool IsControlledBySequence() const { return bIsControlledBySequence; }

	/** Returns the FMovieSceneSequenceID of the sequence that owns the binding. */
	const FMovieSceneSequenceID& GetControllingSequenceID() const { return SequenceID; }

	/** Returns the sequence that directly owns the actor's binding. */
	TWeakObjectPtr<UMovieSceneSequence> GetControllingSequence() const { return WeakControllingSequence; }

	/** Returns true if the actor is bound as a possessable with existing track data. */
	bool IsPossessable() const { return IsControlledBySequence() && bIsPossessable; }

	/** Returns true if the actor is bound as a Sequencer-managed spawnable. */
	bool IsSpawnable() const { return IsControlledBySequence() && !bIsPossessable; }

	/** Returns the actor passed at construction. */
	AActor* GetTargetActor() const { return WeakTargetActor.Get(); }

	/** Returns the root level sequence passed at construction. */
	ULevelSequence* GetTargetLevelSequence() const { return WeakTargetLevelSequence.Get(); }

private:
	/** Drives the resolution pipeline: tries the root sequence, then compiled sub-sequence data. */
	void Init();

	/** Searches sub-sequences in the compiled hierarchy when the root sequence yields no binding. */
	void InitFromCompiledData(ULevelSequence* InLevelSequence, AActor* InActor);

	/** Attempts to resolve the actor's binding GUID from InLevelSequence. Returns true on success. */
	bool ResolveBindingFromSequence(AActor* InActor, ULevelSequence* InLevelSequence);

	/** Returns true if InBindingGuid is a possessable with existing track data (not a spawnable custom binding). */
	bool IsPossessableBinding(ULevelSequence* InLevelSequence, const FGuid& InBindingGuid) const;

	/** Resolves SequenceID for ForSequence via playback state or compiled hierarchy. Returns true on success. */
	bool TryFindSequenceID(ULevelSequence* ForSequence);

	TWeakObjectPtr<AActor> WeakTargetActor;
	TWeakObjectPtr<ULevelSequence> WeakTargetLevelSequence;

	bool bIsControlledBySequence = false;
	bool bIsPossessable = false;
	FGuid CachedObjectBindingGuid;
	FMovieSceneSequenceID SequenceID;
	TWeakObjectPtr<UMovieSceneSequence> WeakControllingSequence;
	TWeakPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState;
};

} // namespace UE::TakesCore

#undef UE_API
