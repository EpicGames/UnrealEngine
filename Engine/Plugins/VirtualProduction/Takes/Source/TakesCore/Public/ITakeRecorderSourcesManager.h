// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "ITakeRecorderSourcesManager.generated.h"

class UTakeRecorderSource;
class UTakeRecorderSources;
class ULevelSequence;

UINTERFACE(MinimalAPI, NotBlueprintable)
class UTakeRecorderSourcesManager : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface for the public Take Recorder Sources manager interface.
 * This interface allows callers to work with Take Recorder Sources uniformly at editor and runtime,
 * with the implementation handling the differences between the two environments.
 * 
 * At editor time, the LevelSequence metadata is used to store and track sources as is the traditional behaviour.
 * At runtime, the sources should be stored per level sequence editor separate from the sequence, as metadata is unavailable in runtime.
 */
class ITakeRecorderSourcesManager
{
	GENERATED_BODY()

public:
	/** Returns a pointer to the global ITakeRecorderSourcesManager implementation, or nullptr if not available. */
	static TAKESCORE_API ITakeRecorderSourcesManager* Get();

	/**
	 * Returns a reference to the global ITakeRecorderSourcesManager implementation.
	 * Asserts if no implementation is available.
	 */
	static TAKESCORE_API ITakeRecorderSourcesManager& GetChecked();

	/**
	 * Copies all recorder sources from one level sequence to another.
	 *
	 * @param FromSequence  The sequence to copy sources from.
	 * @param ToSequence    The sequence to copy sources to.
	 * @return Pointer to the newly copied Sources object, or nullptr if sources were not found in FromSequence, or nothing was copied.
	 */
	virtual UTakeRecorderSources* CopySources(ULevelSequence* FromSequence, ULevelSequence* ToSequence) = 0;

	/**
	 * Copies the given sources onto the given sequence. This will replace any sources already on ToSequence.
	 *
	 * @param InSources     The sources to copy.
	 * @param ToSequence    The sequence to copy sources to.
	 * @return Pointer to the newly copied Sources object, or nullptr if nothing was copied.
	 */
	virtual UTakeRecorderSources* CopySources(UTakeRecorderSources* InSources, ULevelSequence* ToSequence) = 0;

	/**
	 * Returns the UTakeRecorderSources associated with the given level sequence, or nullptr if none exist.
	 *
	 * @param InSequence  The sequence to look up.
	 */
	virtual UTakeRecorderSources* FindSources(const ULevelSequence* InSequence) const = 0;

	/**
	 * Returns the UTakeRecorderSources associated with the given level sequence, creating and associating
	 * a new instance if one does not already exist.
	 *
	 * @param InSequence  The sequence to look up or create sources for.
	 */
	virtual UTakeRecorderSources* FindOrAddSources(ULevelSequence* InSequence) = 0;

	/**
	 * Returns true if a UTakeRecorderSources object is currently associated with the given level sequence.
	 *
	 * @param InSequence  The sequence to check.
	 */
	virtual bool HasSources(const ULevelSequence* InSequence) const = 0;

	/**
	 * Convenience wrapper that adds a new recorder source of type TSource to the given level sequence.
	 *
	 * @param InSequence  The sequence to add the source to.
	 * @return The newly created UTakeRecorderSource, or nullptr on failure.
	 */
	template <typename TSource>
	UTakeRecorderSource* AddSource(ULevelSequence* InSequence)
	{
		return AddSource(TSource::StaticClass(), InSequence);
	}

	/**
	 * Adds a new recorder source of the specified class to the given level sequence.
	 *
	 * @param InSequence     The sequence to add the source to.
	 * @param InSourceClass  The class of source to create and add.
	 * @return The newly created UTakeRecorderSource, or nullptr on failure.
	 */
	virtual UTakeRecorderSource* AddSource(ULevelSequence* InSequence, const TSubclassOf<UTakeRecorderSource> InSourceClass) = 0;

	/**
	 * Removes a recorder source from the given level sequence's source list.
	 *
	 * @param InSequence  The sequence whose source list to modify.
	 * @param InSource    The source to remove.
	 */
	virtual void RemoveSource(ULevelSequence* InSequence, UTakeRecorderSource* InSource) = 0;

	/**
	 * Removes the UTakeRecorderSources association for the given level sequence.
	 * Callers are responsible for invoking this when a sequence is no longer needed.
	 *
	 * @param InSequence  The sequence whose sources entry should be removed.
	 */
	virtual void RemoveSources(ULevelSequence* InSequence) = 0;
};
