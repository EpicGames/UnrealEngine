// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TakeRecorderNamingTokensContext.generated.h"

class AActor;
class UTakeMetaData;

/**
 * Context object which may be passed to NamingTokens evaluations from within TakeRecorder.
 * This is stored under TakesCore rather than the TakeRecorderNamingTokensModule for dependency management. Multiple
 * Take modules need access to the context but don't need access to the NamingTokens themselves and would result
 * in circular referencing.
 */
UCLASS(MinimalAPI)
class UTakeRecorderNamingTokensContext : public UObject
{
	GENERATED_BODY()
	
public:
	/** MetaData specifically for this context. Setting this prevents having to perform a global lookup. */
	UPROPERTY(Transient)
	TWeakObjectPtr<const UTakeMetaData> TakeMetaData;
	
	/** The specific actor for this context. @todo NamingTokens - Determine if we should be retrieving the {actor} a different way. */
	UPROPERTY(Transient)
	TWeakObjectPtr<const AActor> Actor;
	
	/** The audio device channel for this context. @todo NamingTokens - Determine if we should be retrieving the {channel} a different way. */
	UPROPERTY(Transient)
	int32 AudioInputDeviceChannel = 0;
};