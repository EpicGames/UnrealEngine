// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "IMovieSceneMetaData.h"
#include "TakePreset.h"
#include "Misc/DateTime.h"
#include "Misc/QualifiedFrameTime.h"
#include "TakeMetaData.generated.h"

#define UE_API TAKESCORE_API

class UTakePreset;
class UTakeRecorderNamingTokensContext;
struct FNamingTokenResultData;

/**
 * Take meta-data that is stored on ULevelSequence assets that are recorded through the Take Recorder.
 * Meta-data is retrieved through ULevelSequence::FindMetaData<UTakeMetaData>()
 */
UCLASS(MinimalAPI, config=EditorSettings, PerObjectConfig, BlueprintType)
class UTakeMetaData : public UObject, public IMovieSceneMetaDataInterface
{
public:
	GENERATED_BODY()

	UE_API UTakeMetaData(const FObjectInitializer& ObjInit);

public:

	/** The asset registry tag that contains the slate for this meta-data */
	static UE_API const FName AssetRegistryTag_Slate;

	/** The asset registry tag that contains the take number for this meta-data */
	static UE_API const FName AssetRegistryTag_TakeNumber;

	/** The asset registry tag that contains the timestamp for this meta-data */
	static UE_API const FName AssetRegistryTag_Timestamp;

	/** The asset registry tag that contains the timecode in for this meta-data */
	static UE_API const FName AssetRegistryTag_TimecodeIn;

	/** The asset registry tag that contains the timecode out for this meta-data */
	static UE_API const FName AssetRegistryTag_TimecodeOut;

	/** The asset registry tag that contains the user-description for this meta-data */
	static UE_API const FName AssetRegistryTag_Description;

	/** The asset registry tag that contains the level-path for this meta-data */
	static UE_API const FName AssetRegistryTag_LevelPath;
	
	/**
	 * Access the global config instance that houses default settings for take meta data for a given project
	 */
	static UE_API UTakeMetaData* GetConfigInstance();

	/**
	 * Create a new meta-data object from the project defaults
	 *
	 * @param Outer    The object to allocate the new meta-data within
	 * @param Name     The name for the new object. Must not already exist
	 */
	static UE_API UTakeMetaData* CreateFromDefaults(UObject* Outer, FName Name);
	
	/**
	 * Retrieve the most recent or active take metadata.
	 */
	static UE_API UTakeMetaData* GetMostRecentMetaData();

	/**
	 * Update the most recent accessed metadata.
	 */
	static UE_API void SetMostRecentMetaData(UTakeMetaData* InMetaData);
	
public:

	/**
	 * Extend the default ULevelSequence asset registry tags
	 */
	UE_API virtual void ExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void ExtendAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override {}

#if WITH_EDITOR
	/**
	 * Extend the default ULevelSequence asset registry tag meta-data
	 */
	UE_API virtual void ExtendAssetRegistryTagMetaData(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
#endif // WITH_EDITOR

public:

	/**
	 * Check whether this take is locked
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API bool IsLocked() const;

	/**
	 * Check if this take was recorded (as opposed
	 * to being setup for recording)
	 */ 
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API bool Recorded() const;

	/**
	 * @return The slate for this take
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API const FString& GetSlate() const;

	/**
	 * @return The take number for this take
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API int32 GetTakeNumber() const;

	/**
	 * @return The timestamp for this take
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API FDateTime GetTimestamp() const;

	/**
	 * @return The timecode in for this take
	 */
	UFUNCTION(BlueprintCallable, Category = "Take")
	UE_API FTimecode GetTimecodeIn() const;

	/**
	 * @return The timecode out for this take
	 */
	UFUNCTION(BlueprintCallable, Category = "Take")
	UE_API FTimecode GetTimecodeOut() const;

	/**
	 * @return The duration for this take
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API FFrameTime GetDuration() const;

	/**
	 * @return The frame-rate for this take
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API FFrameRate GetFrameRate();

	/**
	 * @return The user-provided description for this take
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API FString GetDescription() const;

	/**
	 * @return The preset on which the take was originally based
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API UTakePreset* GetPresetOrigin() const;

	/**
	 * @return The AssetPath of the Level used to create a Recorded Level Sequence
	 */ 	
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API FString GetLevelPath() const;

	/**
	 * @return The Map used to create this recording
	 */ 
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API ULevel* GetLevelOrigin() const;

	/**
	*  @return Get if we get frame rate from time code
	*/
	UFUNCTION(BlueprintCallable, Category = "Take")
	UE_API bool GetFrameRateFromTimecode() const;

public:

	/**
	 * Lock this take, causing it to become read-only
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API void Lock();

	/**
	 * Unlock this take if it is read-only, allowing it to be modified once again
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API void Unlock();

	/**
	 * Generate the desired asset path for this take meta-data
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API FString GenerateAssetPath(const FString& PathFormatString, UTakeRecorderNamingTokensContext* InContext = nullptr) const;

	/**
	 * Attempt to generate the root asset path.
	 * @return true if successful, false if the path couldn't be generated.
	 */
	UE_API bool TryGenerateRootAssetPath(const FString& PathFormatString, FString& OutGeneratedAssetPath, FText* OutErrorMessage = nullptr, UTakeRecorderNamingTokensContext* InContext = nullptr) const;
	
	/** Process TakeRecorder Naming Tokens. */
	UE_API FNamingTokenResultData ProcessTokens(const FText& InText, UTakeRecorderNamingTokensContext* InContext = nullptr) const;
	
	/**
	 * Set the slate for this take and reset its take number to 1
	 * @param bEmitChanged Whether or not to send a slate changed event
	 * @note: Only valid for takes that have not been locked
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API void SetSlate(FString InSlate, bool bEmitChanged = true);

	/**
	 * Set this take's take number. Take numbers are always clamped to be >= 1.
	 * @param bEmitChanged Whether or not to send a take number changed event
	 * @note: Only valid for takes that have not been locked
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API void SetTakeNumber(int32 InTakeNumber, bool bEmitChanged = true);

	/**
	 * Set this take's timestamp
	 * @note: Only valid for takes that have not been locked
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API void SetTimestamp(FDateTime InTimestamp);

	/**
	 * Set this take's timecode in
	 * @note: Only valid for takes that have not been locked
	 */
	UFUNCTION(BlueprintCallable, Category = "Take")
	UE_API void SetTimecodeIn(FTimecode InTimecodeIn);

	/**
	 * Set this take's timecode out
	 * @note: Only valid for takes that have not been locked
	 */
	UFUNCTION(BlueprintCallable, Category = "Take")
	UE_API void SetTimecodeOut(FTimecode InTimecodeOut);

	/**
	 * Set this take's duration
	 * @note: Only valid for takes that have not been locked
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API void SetDuration(FFrameTime InDuration);

	/**
	 * Set this take's frame-rate
	 * @note: Only valid for takes that have not been locked
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API void SetFrameRate(FFrameRate InFrameRate);

	/**
	 * Set this take's user-provided description
	 * @note: Only valid for takes that have not been locked
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API void SetDescription(FString InDescription);

	/**
	 * Set the preset on which the take is based
	 * @note: Only valid for takes that have not been locked
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API void SetPresetOrigin(UTakePreset* InPresetOrigin);

	/**
	 *  Set the map used to create this recording
	 */
	UFUNCTION(BlueprintCallable, Category="Take")
	UE_API void SetLevelOrigin(ULevel* InLevelOrigin);

	/**
	*  Set if we get frame rate from time code
	*/
	UFUNCTION(BlueprintCallable, Category = "Take")
	UE_API void SetFrameRateFromTimecode(bool InFromTimecode);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTakeSlateChanged, const FString& SlateName, UTakeMetaData* TakeMetaData);
	/** Multicast delegate emitted when the slate is changed. */
	static UE_API FOnTakeSlateChanged& OnTakeSlateChanged();

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTakeNumberChanged, int32 TakeNumber, UTakeMetaData* TakeMetaData);
	/** Multicast delegate emitted when take number is changed. */
	static UE_API FOnTakeNumberChanged& OnTakeNumberChanged();
	
private:

	/** Whether the take is locked */
	UPROPERTY()
	bool bIsLocked;

	/** The user-provided slate information for the take */
	UPROPERTY(config)
	FString Slate;

	/** The take number */
	UPROPERTY()
	int32 TakeNumber;

	/** The timestamp at which the take was initiated */
	UPROPERTY()
	FDateTime Timestamp;

	/** The timecode at the start of recording */
	UPROPERTY()
	FTimecode TimecodeIn;

	/** The timecode at the end of recording */
	UPROPERTY()
	FTimecode TimecodeOut;

	/** The desired duration for the take */
	UPROPERTY(config)
	FFrameTime Duration;

	/** The frame rate the take was recorded at. We default to 60fps. */
	UPROPERTY()
	FFrameRate FrameRate = FFrameRate(60, 1);

	/** A user-provided description for the take */
	UPROPERTY(config)
	FString Description;

	/** The preset that the take was based off */
	UPROPERTY(config)
	TSoftObjectPtr<UTakePreset> PresetOrigin;

#if WITH_EDITORONLY_DATA
	/** The level map used to create this recording */
	UPROPERTY()
	TSoftObjectPtr<ULevel> LevelOrigin;
#endif // WITH_EDITORONLY_DATA

	/** Whether or not we get or frame rate from Timecode, default to true */
	UPROPERTY()
	bool bFrameRateFromTimecode;
};

#undef UE_API
