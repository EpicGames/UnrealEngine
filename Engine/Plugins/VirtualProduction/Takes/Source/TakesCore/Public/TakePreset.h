// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "TakePreset.generated.h"

#define UE_API TAKESCORE_API

class ULevelSequence;

/**
 * Take preset that is stored as an asset comprising a ULevelSequence, and a set of actor recording sources
 */
UCLASS(MinimalAPI, BlueprintType)
class UTakePreset : public UObject
{
public:

	GENERATED_BODY()

	UE_API UTakePreset(const FObjectInitializer& ObjInit);

	/**
	 * Get this preset's level sequence that is used as a template for a new take recording
	 */
	ULevelSequence* GetLevelSequence() const
	{
		return LevelSequence;
	}

	/**
	 * Retrieve this preset's level sequence template, creating one if necessary
	 */
	UE_API ULevelSequence* GetOrCreateLevelSequence();

	/**
	 * Forcibly re-create this preset's level sequence template, even if one already exists
	 */
	UE_API void CreateLevelSequence();

	/**
	 * Copy the specified template preset into this instance. Copies the level sequence and all its recording meta-data.
	 */
	UE_API void CopyFrom(UTakePreset* TemplatePreset);

	/**
	 * Copy the specified level-sequence into this instance. Copies the level sequence and all its recording meta-data.
	 */
	UE_API void CopyFrom(ULevelSequence* TemplateLevelSequence);

	/**
	 * Bind onto an event that is triggered when this preset's level sequence has been changed
	 */
	UE_API FDelegateHandle AddOnLevelSequenceChanged(const FSimpleDelegate& InHandler);

	/**
	 * Remove a previously bound handler for the  event that is triggered when this preset's level sequence has been changed
	 */
	UE_API void RemoveOnLevelSequenceChanged(FDelegateHandle DelegateHandle);

	/**
	 * Allocate the transient preset used by the take recorder.
	 */
	static UE_API UTakePreset* AllocateTransientPreset(UTakePreset* TemplatePreset);

	/**
	 * Static multicast delegate fired by AllocateTransientPreset when it creates a new transient pending take.
	 */
	static UE_API FSimpleMulticastDelegate& OnTakePresetAllocated();

private:

#if WITH_EDITOR
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif // WITH_EDITOR

	/** Instanced level sequence template that is used to define a starting point for a new take recording */
	UPROPERTY(Instanced)
	TObjectPtr<ULevelSequence> LevelSequence;

	FSimpleMulticastDelegate OnLevelSequenceChangedEvent;

	static FSimpleMulticastDelegate OnTakePresetAllocatedDelegate;
};

#undef UE_API
