// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"
#include "TakePresetSettings.generated.h"

#define UE_API TAKESCORE_API

class ULevelSequence;

/** Wraps the class so this can be customized by a type layout. */
USTRUCT()
struct FTakeRecorderTargetRecordClassProperty
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Take Recorder")
	TSoftClassPtr<ULevelSequence> TargetRecordClass;
};

/**
 * Settings for how to assemble UTakePreset.
 * 
 * These settings belong into UTakeRecorderSettings but those are in TakeRecorder, which depends on TakesCore. 
 * DisplayName is important so these settings are displayed in the same category as UTakeRecorderSettings.
 */
UCLASS(MinimalAPI, config=EditorSettings, BlueprintType, DisplayName = "Take Recorder")
class UTakePresetSettings : public UObject
{
public:
	GENERATED_BODY()

	UE_API UTakePresetSettings();

	/** @return The settings object */
	static UE_API UTakePresetSettings* Get();

	/** @return The class that recorded sequences should have */
	UE_API UClass* GetTargetRecordClass() const;
	
	/** Sets the target record class */
	UE_API void SetTargetRecordClass(const TSoftClassPtr<ULevelSequence>& InClass);

	DECLARE_MULTICAST_DELEGATE(FOnSettingsChanged);
	/** Invoked when any settings change. */
	FOnSettingsChanged& OnSettingsChanged() { return OnSettingsChangedDelegate; }

	static FName GetTargetRecordClassMemberName() { return GET_MEMBER_NAME_CHECKED(UTakePresetSettings, TargetRecordClass); } 

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

private:
	
	/** The class that recorded sequences should have. */
	UPROPERTY(config, EditAnywhere, Category = "Take Recorder")
	FTakeRecorderTargetRecordClassProperty TargetRecordClass;

	/** Invoked when any settings change. */
	FOnSettingsChanged OnSettingsChangedDelegate;
};

#undef UE_API
