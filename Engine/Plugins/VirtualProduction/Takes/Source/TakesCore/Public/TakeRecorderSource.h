// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Textures/SlateIcon.h"
#include "Serializers/MovieSceneManifestSerialization.h"
#include "TakeRecorderSources.h"
#include "Misc/QualifiedFrameTime.h"
#include "TakeRecorderSource.generated.h"

#define UE_API TAKESCORE_API

struct FMovieSceneSequenceID;
class ULevelSequence;

/**
 * Base class for all sources that can be recorded with the Take Recorder. Custom recording sources can
 * be created by inheriting from this class and implementing the Start/Tick/Stop recording functions. 
 * The level sequence that the recording is being placed into is provided so that the take can decide
 * to store the data directly in the resulting level sequence, but sources are not limited to generating
 * data in the specified Level Sequence. The source should be registered with the ITakeRecorderModule for
 * it to show up in the Take Recorder UI. If creating a recording setup via code you can just add instances
 * of your Source to the UTakeRecorderSources instance you're using to record and skip registering them with
 * the module.
 *
 * Sources should reset their state before recording as there is not a guarantee that the object will be newly
 * created for each recording.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, BlueprintType)
class UTakeRecorderSource : public UObject
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderSource(const FObjectInitializer& ObjInit);
	
	// ~Begin UObject
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// ~End UObject

	/** True if this source is cued for recording or not */
	UPROPERTY(BlueprintReadWrite, Category="Source")
	bool bEnabled;

	UPROPERTY(BlueprintReadWrite, Category="Source")
	int32 TakeNumber;

	UPROPERTY(BlueprintReadWrite, Category="Source")
	FColor TrackTint;

public:
	/**
	 * Gives the Source object an opportunity to perform additional initialization outside of the constructor. This can be necessary
	 * when needing to perform additional initialization that should not occur if/when the object is duplicated (as is the case when 
	 * all sources are duplicated prior to each record pass).
	 */
	virtual void Initialize() {}

	/** 
	* If the source is Valid and ready to be recorded. Tells the Take Recorder that it's really going to record. Note that PreRecord/StartRecord
	* may still get called on the source even if not Valid. This is used mainly so we only Start Recording if we have more than one Valid Source.
	*/
	virtual bool IsValid() const { return true; }

	/**
	* This is called on all sources before recording is started. This allows a source to return a list of new sources 
	* that should be added to the recording. This is useful for abstract sources (such as "Player" or "World Settings")
	* which are convenience wrappers for existing sources (such as a Actor Source). In these cases, these abstract sources
	* simply do their logic to find out which new sources need to be made and then return them. These new sources only need
	* to exist for the lifespan of a single UTakeRecorderSources recording.
	*
	* Do any computationally expensive work in this function (as opposed to StartRecording) so that all sources can have
	* StartRecording called as closely as possible to each other. See StartRecording for more details.
	*
	* Will not be called if this recording source is not enabled.
	*
	* @param InSequence - The Level Sequence the take is being recorded into
	* @param InSequenceID - The sequence ID for InSequence, or MovieSceneSequenceID::Root where InSequence == InRootSequence (ie, not sub-sequence recording)
	* @param InRootSequence - The Root Level Sequence that may contain the InSequence as a child or if no subsequences, is the same as InSequence.
	* @param InManifestSerializer - Manifest Serializer that we may write into.
	* @return An array of newly created take recorder sources. Can be an empty list if no additional sources needed to be
	* created by this source.
	*/
	UE_API virtual TArray<UTakeRecorderSource*> PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer);

	/** Called after all sources have had PreRecording run. */
	virtual void OnAllSourcesPreRecordingFinished() { }
	
	/**
	* This is called when the UTakeRecorderSources starts a recording, after all sources have had PreRecording called on them.
	* Implementations should avoid blocking on this call (instead place that in PreRecording) so that the sources all get
	* StartRecording called on them as close to possible as one another. This is useful for any source that relies on platform
	* time (or other time sources) so that a source does not spend a long time being initialized and causing different sources
	* to record drastically different times.
	*
	* @param InSectionStartTimecode - The externally provided timecode at the time of this recording starting.
	* @param InSectionFirstFrame - The first frame of the section in tick resolution
	* @param InSequence - The Level Sequence the take is being recorded into.
	*
	*/
	virtual void StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence) { }

	/**
	* This is called each frame and allows the source to record any new information from the current frame. Called after all
	* actors in the level Tick.
	* 
	* @param - CurrentSequenceTime - The frame of the Level Sequence for this frame that the data should be stored on. Passed
	*								 as a qualified time to handle any differences in sub-sequence resolutions.
	* 
	* Will not be called if this recording source is not enabled.
	*/
	virtual void TickRecording(const FQualifiedFrameTime& CurrentSequenceTime) {}

	/**
	* This is called when the UTakeRecorderSources stops recording. This is called on all sources after recording has finished.
	*
	* This should avoid being a blocking call (use PostRecording instead) so that all sources can be stopped as soon as possible
	* after the user requests the recording end. See StartRecording for more details about why having all recordings start/stop
	* as close as possible to each other is important.
	*
	* Will not be called if this recording source is not enabled.
	*/
	virtual void StopRecording(class ULevelSequence* InSequence) { }
	
	/**
	* This is called on all sources after recording is stopped. By returning the same list of additional sources as provided in
	* PreRecording the source can clean up any additional temporary sources that were created by this recording. These additional
	* temporary sources will be properly shut down so they have a chance to store their data before being removed from the list.
	*
	* Will not be called if this recording source is not enabled.
	*
	* @return An array of take recorder sources to be removed. Likely to match the same list as PreRecording. Can be an empty list
	* if no additional sources are required by this source.
	*/
	virtual TArray<UTakeRecorderSource*> PostRecording(class ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled) { return TArray<UTakeRecorderSource*>(); }

	/*
	 * This is called on all sources after post recording. There should not be any dependencies between sources at this point.
	 */
	virtual void FinalizeRecording() {}

	/**
	* This allows a Source to return an array of dynamically spawned settings objects for that source.
	* These will be shown in the UI as a separate category when the source is Selected.
	*
	* @return A an array of additional objects to be shown in the UI. Can be an empty list if no additional settings objects are needed.
	*/
	virtual TArray<UObject*> GetAdditionalSettingsObjects() const { return TArray<UObject*>(); }

	/** 
	 * Supports recording into subscenes 
	 */
	virtual bool SupportsSubscenes() const { return true; }

	/**
	* When recorded to a Subscenes track, what should the subscene track name of the Section be?
	*/
	virtual FString GetSubsceneTrackName(ULevelSequence* InSequence) const { return TEXT("Unnamed_Source"); }

	/**
	* When recorded to a Subscenes track, what should the sub level sequence asset name be? By default, returns the name of the track
	*/
	virtual FString GetSubsceneAssetName(ULevelSequence* InSequence) const { return GetSubsceneTrackName(InSequence); }

	/**
	* If you are not recording into a sub-sequence then this will be called after PreRecording is called and will specify the folder
	* that this Source should add itself to. This will not be called if you are recording into a sub-sequence as recorded data
	* should be placed in the root of the sub-sequence in that case.
	*/
	virtual void AddContentsToFolder(class UMovieSceneFolder* InFolder) {}

	/**
	 * Optional function to allow sources to decide if they want to create folders for their contents or not.
	 */
	virtual bool ShouldCreateFolderForContents() const { return true; }


	/**
	 * Whether this source can be added (some sources should only exist once) 
	 */
	virtual bool CanAddSource(UTakeRecorderSources* InSources) const { return true; }

	/**
	 * Whether this source should overwrite existing track data, or leave it, or potentially create new data.
	 * This is primarily used in response to Spawnables, such as we if we want to overwrite subsequence data
	 * or clear existing data in non-subsequence tracks.
	 */
	UE_API virtual bool OverwriteExistingTrackData() const;

	/**
	 * Called by UTakeRecorderSources right before a Source is removed from its internal map.
	 * Allows for any cleanup or logic the Source needs to do when being removed.
	 */
	virtual void PreRemove() {}

public:

	/**
	 * Get the optional category text to display on the Take Recorder source list for this source
	 */
	FText GetCategoryText() const
	{
		return GetCategoryTextImpl();
	}

	/**
	 * Get the text to display on the Take Recorder source list for this source
	 */
	FText GetDisplayText() const
	{
		return GetDisplayTextImpl();
	}

	/**
	 * Get the text to display on the add source menu for this source
	 */
	FText GetAddSourceDisplayText() const
	{
		FText Result = GetAddSourceDisplayTextImpl();
		if (Result.IsEmpty())
		{
			ensureMsgf(IsTemplate(RF_ClassDefaultObject) /* This can be called on the CDO. */, TEXT("You must provide localized source name."));
			return GetDisplayText();
		}
		return Result;
	}
	
	/**
	 * Get the icon to display on the Take Recorder source list for this source
	 */
	const FSlateBrush* GetDisplayIcon() const
	{
		return GetDisplayIconImpl();
	}

	/**
	 * A very brief text summary of what is going to be recorded for this source
	 */
	FText GetDescriptionText() const 
	{
		return GetDescriptionTextImpl();
	}

	/**
	 * Whether or the source can be referenced via take number
	 */
	virtual bool SupportsTakeNumber() const { return true; }
	
	/** If this source implements the IMovieSceneTrackRecorderHost interface. This is used to avoid RTTI. */
	virtual const class IMovieSceneTrackRecorderHost* AsTrackRecorderHost() const { return nullptr; }
	
	/** If this source is an actor source for the given actor. Used for cross-module communication. */
	virtual bool IsActorSourceForActor(const AActor* Actor) const { return false; }

private:

	/** Default implementations for UI functions */
	UE_API virtual const FSlateBrush* GetDisplayIconImpl() const;
	virtual FText GetCategoryTextImpl() const { return FText(); }
	virtual FText GetDisplayTextImpl() const { return FText(); }
	virtual FText GetDescriptionTextImpl() const { return FText(); }
	virtual FText GetAddSourceDisplayTextImpl() const { return FText(); };
};

#undef UE_API
