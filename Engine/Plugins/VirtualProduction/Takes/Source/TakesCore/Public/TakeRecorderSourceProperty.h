// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "TakeRecorderSources.h"
#include "TakeRecorderSourceProperty.generated.h"

#define UE_API TAKESCORE_API

USTRUCT(BlueprintType)
struct FActorRecordedProperty
{
	GENERATED_BODY()

	FActorRecordedProperty()
		: PropertyName(NAME_None)
		, bEnabled(false)
		, RecorderName()
	{
	}

	FActorRecordedProperty(const FName& InName, const bool bInEnabled, const FText& InRecorderName)
	{
		PropertyName = InName;
		bEnabled = bInEnabled;
		RecorderName = InRecorderName;
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Property")
	FName PropertyName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Property")
	bool bEnabled;

	UPROPERTY(VisibleAnywhere, Category = "Property")
	FText RecorderName;
};

/**
* This represents a list of all possible properties and components on an actor
* which can be recorded by the Actor Recorder and whether or not the user wishes
* to record them. If you wish to expose a property to be recorded it needs to be marked
* as "Interp" (C++) or "Expose to Cinematics" in Blueprints.
*/
UCLASS(MinimalAPI, BlueprintType)
class UActorRecorderPropertyMap : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Property")
	TSoftObjectPtr<UObject> RecordedObject;

	/* Represents properties exposed to Cinematics that can possibly be recorded. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Property")
	TArray<FActorRecordedProperty> Properties;

	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, meta=(ShowInnerProperties, EditFixedOrder), Category = "Property")
	TArray<TObjectPtr<UActorRecorderPropertyMap>> Children;

public:
	struct Cache
	{
		Cache& operator+=(const Cache& InOther)
		{
			Properties += InOther.Properties;
			Components += InOther.Components;
			return *this;
		}
		/** The number of properties (both on actor + components) that we are recording. This includes child maps.*/
		int32 Properties = 0;
		/** The number of components that belong to the target actor that we are recording. This include child maps.*/
		int32 Components = 0;
	};

	/** Return the number of properties and components participating in recording.*/
	Cache CachedPropertyComponentCount() const
	{
		return RecordingInfo;
	}

	/** Visit all properties recursively and cache the record state. */
	UE_API void UpdateCachedValues();
private:
	UE_API int32 NumberOfPropertiesRecordedOnThis() const;
	UE_API int32 NumberOfComponentsRecordedOnThis() const;

	/** This is called by the children to let the parent know that the number of recorded properties has changed*/
	UE_API void ChildChanged();

	TWeakObjectPtr<UActorRecorderPropertyMap> Parent;

	Cache RecordingInfo;
};

/**
 * Encapsulates audio device properties which are utilized by UI facing classes such as FAudioInputDeviceProperty.
 */
USTRUCT(BlueprintType)
struct FAudioInputDeviceInfoProperty
{
	GENERATED_BODY()

	FAudioInputDeviceInfoProperty() = default;

	FAudioInputDeviceInfoProperty(const FString& InDeviceName, const FString& InDeviceId, int32 InChannelCount, int32 InSampleRate, bool bInIsDefaultDevice)
		: DeviceName(InDeviceName)
		, DeviceId(InDeviceId)
		, InputChannels(InChannelCount)
		, PreferredSampleRate(InSampleRate)
		, bIsDefaultDevice(bInIsDefaultDevice)
	{
	}

	/** User friendly name of the audio device */
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Property")
	FString DeviceName;

	/** The unique id used to identify the device */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Property")
	FString DeviceId;

	/** The number input channels this device supports */
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category = "Property")
	int32 InputChannels = 0;

	/** The preferred sample rate for this audio device */
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category = "Property")
	int32 PreferredSampleRate = 0;

	/** Boolean indicating if this device is the currently the system selected input device */
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category = "Property")
	bool bIsDefaultDevice = false;
};

/**
 * Encapsulates the array of audio input devices which is populated by UTakeRecorderMicrophoneAudioManager and
 * utilized by the audio input device list in FAudioInputDevicePropertyCustomization.
 */
USTRUCT(BlueprintType)
struct FAudioInputDeviceProperty
{
	GENERATED_BODY()

	/** The maximum number input channels currently supported */
	constexpr static int32 MaxInputChannelCount = 8;

	/** Boolean indicating if the system selects audio device should be used or to use the selected device from the details panel */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Property")
	bool bUseSystemDefaultAudioDevice = true;

	/* Holds device information for each of the audio devices available on this system. */
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category = "Property")
	TArray<FAudioInputDeviceInfoProperty> DeviceInfoArray;

	/** The unique id of the currently selected audio device */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Property", meta = (DisplayName = "Audio Input Device"))
	FString DeviceId;
	
	/** The desired buffer size used for audio callbacks during record */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Property", meta = (ClampMin = "256", UIMin = "256", ClampMax = "8192", UIMax = "8192"))
	int32 AudioInputBufferSize = 1024;	
};

/**
 * This class is used by Microphone sources to track the currently selected channel. It aslo
 * contains a local cache of the max channel count for the currently selected audio device.
 */
USTRUCT(BlueprintType)
struct FAudioInputDeviceChannelProperty
{
	GENERATED_BODY()

public:

	/** The currently selected channel from the details panel for this source */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Property")
	int32 AudioInputDeviceChannel = 0;
};

DECLARE_MULTICAST_DELEGATE(FOnAudioInputDeviceChanged);

/**
 * Provides access to the UI code for registering OnAudioInputDeviceChanged delegates.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, BlueprintType)
class UTakeRecorderAudioInputSettings : public UObject
{
	GENERATED_BODY()
	
public:

	/** Subclasses can override this to enumerate audio devices */
	virtual void EnumerateAudioDevices(bool InForceRefresh = false) {}

	/** Returns the channel count for the currently selected audio device */
	virtual int32 GetDeviceChannelCount() { return 0; }

	/** Accessor for the OnAudioInputDeviceChanged delegate list */
	FOnAudioInputDeviceChanged& GetOnAudioInputDeviceChanged() { return OnAudioInputDeviceChanged;  }

private:

	/** Multicast notification when the user selects a new audio device */
	FOnAudioInputDeviceChanged OnAudioInputDeviceChanged;
};

#undef UE_API
