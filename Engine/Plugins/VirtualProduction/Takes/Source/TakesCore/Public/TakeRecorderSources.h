// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Containers/ArrayView.h"
#include "Templates/SubclassOf.h"
#include "Tickable.h"
#include "Misc/FrameRate.h"
#include "Misc/QualifiedFrameTime.h"
#include "Serializers/MovieSceneManifestSerialization.h"
#include "TakeRecorderSpawnableBehavior.h"

#include "TakeRecorderSources.generated.h"

#define UE_API TAKESCORE_API

class UTakeRecorderSource;
class UMovieSceneSubSection;

DECLARE_LOG_CATEGORY_EXTERN(SubSequenceSerialization, Verbose, All);

struct FTakeRecorderSourcesSettings
{
	bool bStartAtCurrentTimecode;
	bool bRecordSourcesIntoSubSequences;
	bool bRecordToPossessable;
	ETakeRecorderSpawnableOverwriteBehavior SpawnableOverwriteBehavior;
	bool bSaveRecordedAssets;
	bool bAutoLock;
	bool bRemoveRedundantTracks;
	FTransform TransformOrigin;
	FString SubSequenceDirectory;
};

class UWorld;
class ULevelSequence;
class FViewport;
class FCanvas;

namespace UE::TakeRecorderSources
{
	class FTakeRecorderSourcesModule;	
}
	

/**
 * A list of sources to record for any given take. 
 * Is stored and looked up against a level sequence using the ITakeRecorderSourcesManager.
 * Note that within an editor environment, ITakeRecorderSourcesManager will store the sources object
 * as meta-data on ULevelSequence through ULevelSequence::FindMetaData<UTakeRecorderSources>.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UTakeRecorderSources : public UObject
{
public:
	GENERATED_BODY()

	UE_API UTakeRecorderSources(const FObjectInitializer& ObjInit);

	/**
	 * Add a new source to this source list of the templated type
	 *
	 * @return An instance of the templated type
	 */
	template<typename SourceType>
	SourceType* AddSource()
	{
		return static_cast<SourceType*>(AddSource(SourceType::StaticClass()));
	}


	/**
	 * Add a new source to this source list of the templated type
	 *
	 * @param InSourceType    The class type of the source to add
	 * @return An instance of the specified source type
	 */
	UFUNCTION(BlueprintCallable, Category = "Take Recorder", meta = (DeterminesOutputType = "InSourceType"))
	UE_API UTakeRecorderSource* AddSource(TSubclassOf<UTakeRecorderSource> InSourceType);

	/**
	 * Remove the specified source from this list
	 *
	 * @param InSource        The source to remove
	 */
	UFUNCTION(BlueprintCallable, Category = "Take Recorder")
	UE_API void RemoveSource(UTakeRecorderSource* InSource);


	/**
	 * Access all the sources stored in this list
	 */
	TArrayView<UTakeRecorderSource* const> GetSources() const
	{
		return Sources;
	}

	/**
	* Retrieves a copy of the list of sources that are being recorded. This is intended for Blueprint usages which cannot
	* use TArrayView.
	* DO NOT MODIFY THIS ARRAY, modifications will be lost.
	*/
	UFUNCTION(BlueprintPure, DisplayName = "Get Sources (Copy)", Category = "Take Recorder")
	TArray<UTakeRecorderSource*> GetSourcesCopy() const
	{
		return TArray<UTakeRecorderSource*>(Sources);
	}
	/**
	 * Retrieve the serial number that is incremented when a source is added or removed from this list.
	 * @note: This field is not serialized, and not copied along with UObject duplication.
	 */
	uint32 GetSourcesSerialNumber() const
	{
		return SourcesSerialNumber;
	}

	/** Sources settings from the user and project parameters */
	FTakeRecorderSourcesSettings GetSettings() const { return Settings; }
	void SetSettings(FTakeRecorderSourcesSettings& InSettings) { Settings = InSettings; }

	/** Calls the recording initialization flows on each of the specified sources. */
	UFUNCTION(BlueprintCallable, Category = "Take Recorder")
	UE_API void StartRecordingSource(TArray<UTakeRecorderSource*> InSources, const FQualifiedFrameTime& CurrentFrameTime);
	
public:

	/**
	 * Bind a callback for when this source list changes
	 *
	 * @param Handler         The delegate to call when the list changes
	 * @return A handle to this specific binding that should be passed to UnbindSourcesChanged
	 */
	UE_API FDelegateHandle BindSourcesChanged(const FSimpleDelegate& Handler);

	/**
	 * Unbind a previously bound handler for when this source list changes
	 *
	 * @param Handle          The handle returned from BindSourcesChanged for the handler to remove
	 */
	UE_API void UnbindSourcesChanged(FDelegateHandle Handle);
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSourceAdded, UTakeRecorderSource* Source);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSourceRemoved, UTakeRecorderSource* Source);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSourceModified, UTakeRecorderSource* Source);
	
	/** Multicast delegate for when any source is added. */
	static UE_API FOnSourceAdded& OnSourceAdded();

	/** Multicast delegate for when any source is removed. */
	static UE_API FOnSourceRemoved& OnSourceRemoved();
	
	/** Multicast delegate for when any source is modified. */
	static UE_API FOnSourceModified& OnSourceModified();
	
private:
	friend class UE::TakeRecorderSources::FTakeRecorderSourcesModule;
	
	DECLARE_DELEGATE_RetVal_TwoParams(UTakeRecorderSource*, FOnActorBindingAdded, AActor*, ULevelSequence*);
	/** Delegate for when a source is being added for an actor binding. This is for our Sources Module, so it can properly add an actor source. */
	static UE_API FOnActorBindingAdded& OnAddSourceForActorBinding();
	
public:
	/**
	 * Notifies sources that a new actor binding has been added.
	 * 
	 * @param InActor The actor the binding was added for.
	 * @param InLevelSequence The level sequence of the binding.
	 */
	UE_API ULevelSequence* AddSourceForActorBinding(AActor* InActor, ULevelSequence* InLevelSequence);
	
	/** Checks if a given actor is possessable given source configuration. */
	UE_API bool IsActorPossessable(const AActor* InActor) const;
	
public:

	/*
	 * Cache assets needed for sequencer.
	 */
	UE_API void SetCachedAssets(class ULevelSequence* InSequence, FManifestSerializer* InManifestSerializer);

	/*
	 * Pre recording pass
	*
	*/
	UE_API void PreRecording(class ULevelSequence* InSequence, const FQualifiedFrameTime& InCurrentFrameTime, FManifestSerializer* InManifestSerializer);

	/*
	 * Start recording pass
	 *
	 */
	UE_API void StartRecording(class ULevelSequence* InSequence, const FQualifiedFrameTime& InCurrentFrameTime, FManifestSerializer* InManifestSerializer);

	/*
	 * Moves time forward by given DeltaTime
	 *
	 * @return Current Frame Number
	 */
	UE_API FFrameTime AdvanceTime(const FQualifiedFrameTime& CurrentFrameTime, float DeltaTime);

	/*
	* Tick recording pass
	* @return Current Frame Number we are recording at.
	*/
	UE_API FFrameTime TickRecording(class ULevelSequence* InSequence, const FQualifiedFrameTime& CurrentFrameTime, float DeltaTime);

	/*
	* Stop recording pass
	*
	*/
	UE_API void StopRecording(class ULevelSequence* InSequence, const bool bCancelled);

public:
	/*
	*  Static functions used by other parts of the take system
	*/

	/** Creates a sub-sequence asset for the specified sub sequence name based on the given root sequence. */
	static UE_API ULevelSequence* CreateSubSequenceForSource(ULevelSequence* InRootSequence, const FString& SubSequenceTrackName,
		const FString& SubSequenceAssetName, const FTakeRecorderSourcesSettings& InSettings = FTakeRecorderSourcesSettings());

	/**
	 * Array of pairs - key time and the corresponding timecode
	 */
	static UE_API TArray<TPair<FQualifiedFrameTime, FQualifiedFrameTime> > RecordedTimes;

	FQualifiedFrameTime GetCachedFrameTime() const { return CachedFrameTime; }

private:
	static UE_API ULevelSequence* FindExistingSubSequence(const ULevelSequence* InRootSequence, const FString& InSubSequenceAssetName);
	
	/**
	 * Renders stat text that displays the current list of active take recorder sources.  The take recorder can now operate without a panel and this
	 * gives users a way to view the current sources list outside of the panel.
	 */
	int32 RenderStatTakeSources(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation);
		
	/** Called at the end of each frame in both the Editor and in Game to update all Sources. */
	virtual void Tick(float DeltaTime) {}

#if WITH_EDITOR
	/** Called whenever a property is changed on this class */
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	/** A list of handlers to invoke when the sources list changes */
	DECLARE_MULTICAST_DELEGATE(FOnSourcesChanged);
	FOnSourcesChanged OnSourcesChangedEvent;

	/** Calls PreRecording on sources recursively allowing them to create other sources which properly get PreRecording called on them as well. */
	void StartPreRecordingRecursive(const TArray<UTakeRecorderSource*>& InSources, ULevelSequence* InRootSequence, TArray<UTakeRecorderSource*>& NewSourcesOut,
		FManifestSerializer* InManifestSerializer);
	
	/** Calls PreRecording on sources recursively allowing them to create other sources which properly get PreRecording called on them as well. */
	UE_API void PreRecordingRecursive(TArray<UTakeRecorderSource*> InSources, ULevelSequence* InRootSequence, TArray<UTakeRecorderSource*>& NewSourcesOut,
		FManifestSerializer* InManifestSerializer, TSet<FString>& UsedTrackNames);

	/** Finds the folder that the given Source should be created in, creating it if necessary. */
	UE_API class UMovieSceneFolder* AddFolderForSource(const UTakeRecorderSource* InSource, class UMovieScene* InMovieScene);

	/** Remove object bindings that don't have any tracks and are not bindings for attach/path tracks */
	UE_API void RemoveRedundantTracks();

	UE_API void StartRecordingPreRecordedSources(const FQualifiedFrameTime& CurrentFrameTime);

	UE_API void PreRecordSources(TArray<UTakeRecorderSource *> InSources);

	UE_API void StartRecordingTheseSources(const TArray<UTakeRecorderSource *>& InSources, const FQualifiedFrameTime& CurrentFrameTime);

	UE_API void SetSectionStartTimecode(UMovieSceneSubSection* SubSection, const FTimecode& Timecode, const FQualifiedFrameTime& CurrentFrameTime);

private:

	/** The array of all sources contained within this list */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UTakeRecorderSource>> Sources;

	/** Maps each source to the level sequence that was created for that source, or to the root source if a subsequence was not created. */
	UPROPERTY(Transient)
	TMap<TObjectPtr<UTakeRecorderSource>, TObjectPtr<ULevelSequence>> SourceSubSequenceMap;

	/** List of sub-sections that we're recording into. Needed to ensure they're all the right size at the end without re-adjusting every sub-section in a sequence. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<class UMovieSceneSubSection>> ActiveSubSections;
	
	/** Bindings added for an actor mapped to whether it is possessable or not. Reset on PreRecord. */
	TMap<TWeakObjectPtr<AActor>, bool> AddedActorPossessableBindings;

	/** Are we currently in a recording pass and should be ticking our Sources? */
	bool bIsRecording;

	/** What Tick Resolution is the target level sequence we're recording into? Used to convert seconds into FrameNumbers. */
	FFrameRate TargetLevelSequenceTickResolution;

	/** What Display Rate is the target level sequence we're recording into? Used to convert seconds into FrameNumbers. */
	FFrameRate TargetLevelSequenceDisplayRate;

	/** Non-serialized serial number that is used for updating UI when the source list changes */
	uint32 SourcesSerialNumber;

	/** Sources settings */
	FTakeRecorderSourcesSettings Settings;

	/** Manifest Serializer that we are recording into. */
	FManifestSerializer* CachedManifestSerializer;

	/** Level Sequence that we are recording into. Cached so that new sources added mid-recording get placed in the right sequence. */
	ULevelSequence* CachedLevelSequence;

	/** Array of Allocated Serializers created for each sub sequence.  Deleted at the end of the recording so memory is freed. */
	TArray<TSharedPtr<FManifestSerializer>> CreatedManifestSerializers;

	/** All sources after PreRecord */
	TArray<UTakeRecorderSource *> PreRecordedSources;

	/** The last frame time during tick recording */
	FQualifiedFrameTime CachedFrameTime;
};

#undef UE_API
