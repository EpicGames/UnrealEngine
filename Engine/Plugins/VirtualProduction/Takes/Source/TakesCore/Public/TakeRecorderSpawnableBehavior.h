// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "TakeRecorderSpawnableBehavior.generated.h"

UENUM(BlueprintType)
enum class ETakeRecorderSpawnableOverwriteBehavior : uint8
{
	/** Use the project default to determine if spawnables are overwritten */
	ProjectDefault,
	/** Overwrite one existing spawnable similar to behavior in UE 5.7 and below */
	OverwriteLegacy UMETA(DisplayName = "Overwrite (Legacy)"),
	/** Creates a new spawnable */
	CreateNew
};