// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneFloatChannel.h"
#include "Containers/Array.h"
#include "Misc/QualifiedFrameTime.h"
#include "Templates/Tuple.h"

class ULevelSequence;
class UMovieScene;
class UMovieSceneTakeTrack;

namespace UE::TakesCore
{
/** Key: Recorded frame (to which to apply the timecode). Value: The timecode the recorded frame has. */
using FArrayOfRecordedTimePairs = TArray<TPair<FQualifiedFrameTime, FQualifiedFrameTime>>;

/** Creates a takes track to store timecode data on a take recorder source. */
TAKESCORE_API void ProcessRecordedTimes(
	ULevelSequence* InSequence, UMovieSceneTakeTrack* TakeTrack,
	const TOptional<TRange<FFrameNumber>>& FrameRange, const FArrayOfRecordedTimePairs& RecordedTimes
	);
	
enum class ETimecodeExtractionFlags : uint8
{
	None,

	Hours = 1 << 0,
	Minutes = 1 << 1,
	Seconds = 1 << 2,
	Frames = 1 << 3,
	Subframes = 1 << 4,
	
	Times = 1 << 5,

	All = Hours | Minutes | Seconds | Frames | Subframes | Times
};
ENUM_CLASS_FLAGS(ETimecodeExtractionFlags);

// All arrays, except those excluded by the ETimecodeExtractionFlags flags, are equal length.
struct FTimecodeExtractionData
{
	TArray<int32> Hours;
	TArray<int32> Minutes;
	TArray<int32> Seconds;
	TArray<int32> Frames;
	TArray<FMovieSceneFloatValue> SubFrames;
	TArray<FFrameNumber> Times;
	FFrameRate TCRate;
};
	
/** Splits recorded InRecordedTimes up into arrays, one for each component. */
[[nodiscard]] TAKESCORE_API FTimecodeExtractionData ExtractIntoTimecodeArrays(
	UMovieScene* InMovieScene, const TRange<FFrameNumber>& InRange, const FArrayOfRecordedTimePairs& InRecordedTimes,
	ETimecodeExtractionFlags InFlags = ETimecodeExtractionFlags::All
	);
}