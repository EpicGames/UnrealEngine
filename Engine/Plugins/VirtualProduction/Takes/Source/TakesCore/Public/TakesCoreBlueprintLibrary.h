// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "AssetRegistry/AssetData.h"
#include "TakesCoreBlueprintLibrary.generated.h"

#define UE_API TAKESCORE_API


UCLASS(MinimalAPI)
class UTakesCoreBlueprintLibrary : public UBlueprintFunctionLibrary
{
public:

	GENERATED_BODY()

	/**
	 * Compute the next unused sequential take number for the specified slate
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder")
	static UE_API int32 ComputeNextTakeNumber(const FString& Slate);


	/**
	 * Find all the existing takes that were recorded with the specified slate
	 *
	 * @param Slate        The slate to filter by
	 * @param TakeNumber   The take number to filter by. <=0 denotes all takes
	 */
	UFUNCTION(BlueprintCallable, Category="Take Recorder")
	static UE_API TArray<FAssetData> FindTakes(const FString& Slate, int32 TakeNumber = 0);

public:

	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnTakeRecorderSlateChanged, const FString&, Slate);
	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnTakeRecorderTakeNumberChanged, int32, TakeNumber);

	/** Called when the slate is changed. */
	UFUNCTION(BlueprintCallable, Category = "Take Recorder")
	static UE_API void SetOnTakeRecorderSlateChanged(FOnTakeRecorderSlateChanged OnTakeRecorderSlateChanged);

	/** Called when the take number is changed. */
	UFUNCTION(BlueprintCallable, Category = "Take Recorder")
	static UE_API void SetOnTakeRecorderTakeNumberChanged(FOnTakeRecorderTakeNumberChanged OnTakeRecorderTakeNumberChanged);

	static UE_API void OnTakeRecorderSlateChanged(const FString& InSlate);
	static UE_API void OnTakeRecorderTakeNumberChanged(int32 InTakeNumber);

};

#undef UE_API
