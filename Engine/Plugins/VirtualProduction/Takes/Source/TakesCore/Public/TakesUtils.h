// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "CoreMinimal.h"
#include "Misc/PackageName.h"
#include "MovieScene.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UObject/UObjectGlobals.h"
#include "TakesCoreLog.h"
#include "Misc/Paths.h"

#if WITH_EDITOR
#include "ISequencer.h"
#endif // WITH_EDITOR

class UMovieSceneTrack;
class UTakeRecorderSource;
class UWorld;
class ULevelSequence;

namespace UE::MovieScene
{
struct FSharedPlaybackState;
} // namespace UE::MovieScene

namespace TakesUtils
{
	/**
	 * Converts all invalid object name characters to _.
	 *
	 * Taken from ObjectTools and moved a small subset here to remove the need to call into ObjectTools and allow
	 * the same functionality in a non-Editor environment.
	 */
	TAKESCORE_API FString SanitizeInvalidChars(const FString& InText, const TCHAR* InvalidChars);

	/**
	 * Converts all invalid object name characters to _.
	 *
	 * Taken from ObjectTools and moved a small subset here to remove the need to call into ObjectTools and allow
	 * the same functionality in a non-Editor environment.
	 */
	TAKESCORE_API FString SanitizeObjectName(const FString& InObjectName);

	/**
	 * Replaces all invalid package name characters with _.
	 *
	 * Taken from PackageTools and moved a small subset here to remove the need to call into PackageTools and allow
	 * the same functionality in a non-Editor environment.
	 */
	TAKESCORE_API FString SanitizePackageName(const FString& InPackageName);

	/*
	 * Get the first PIE world (or first PIE client world if there is more than one)
	 */

	TAKESCORE_API UWorld* GetFirstPIEWorld();

	TAKESCORE_API void ClampPlaybackRangeToEncompassAllSections(UMovieScene* InMovieScene, bool bUpperBoundOnly);

	TAKESCORE_API void ResetViewAndWorkRange(UMovieScene* InMovieScene);

	TAKESCORE_API void CleanupExternalReferences(UObject* InObject);
	
	/** 
	 * Save an object to disk. 
	 * @param InObject The object to save.
	 * @param bAlwaysKeepExternalReferences If we should always keep external references.
	 * When false, we clear them. When false and the InObject is a Level Sequence, we perform fine-grained cleanup.
	 */
	TAKESCORE_API void SaveAsset(UObject* InObject, bool bAlwaysKeepExternalReferences = true);

	TAKESCORE_API void CreateCameraCutTrack(ULevelSequence* LevelSequence, const FGuid& RecordedCameraGuid, const FMovieSceneSequenceID& SequenceID, const TRange<FFrameNumber>& InRange);

	/*
	* Creates a new Package with the given Package Name (ie: /Game/Test/Foo) of the specified AssetType. If a package already exists at that name the package name will
	* have a number appended and iterated on until an unused package name is found. InPackageName will be modified in this case and will return the package name that the
	* asset was actually created at.
	*
	* You should consider calling MarkPackageDirty() on the returned asset if you further modify it, and you shouldstill notify the FAssetRegistryModule that the asset was 
	* created after this by calling FAssetRegistryModule::AssetCreated
	* 
	* @param InPackageName	- The desired package name (path and asset name) for the new asset. May be mutated by this function if that package name is already taken.
	* @param OutAsset		- The resulting asset if we were able to successfully create an asset.
	* @param OutError		- Human readable error string to describe what went wrong (if anything). Can be nullptr if you don't care about the error message.
	* @param OptionalBase	- Optional asset reference to duplicate the new asset from. If nullptr a brand new asset will be created.
	* @return True if the asset was created successfully, false if there was an error.
	*/
	template<typename AssetType>
	static bool CreateNewAssetPackage(FString &InPackageName, AssetType*& OutAsset, FText* OutError, AssetType* OptionalBase = nullptr, UClass* OptionalClass = nullptr)
	{
		if (!FPackageName::IsValidLongPackageName(InPackageName))
		{
			if (OutError)
			{
				*OutError = FText::Format(NSLOCTEXT("TakeRecorderUtils", "InvalidPathError", "{0} is not a valid asset path."), FText::FromString(InPackageName));
			}
			return false;
		}

		int32 UniqueIndex = 2;
		int32 BasePackageLength = InPackageName.Len();

		// Generate a unique level sequence name for this take if there are already assets of the same name
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		TArray<FAssetData> OutAssetData;

		AssetRegistry.GetAssetsByPackageName(*InPackageName, OutAssetData);
		while (OutAssetData.Num() > 0)
		{
			int32 TrimCount = InPackageName.Len() - BasePackageLength;
			if (TrimCount > 0)
			{
				InPackageName.RemoveAt(BasePackageLength, TrimCount, EAllowShrinking::No);
			}

			InPackageName += FString::Printf(TEXT("_%04d"), UniqueIndex++);

			OutAssetData.Empty();
			AssetRegistry.GetAssetsByPackageName(*InPackageName, OutAssetData);
		}

		// Create the asset to record into
		const FString NewAssetName = FPackageName::GetLongPackageAssetName(InPackageName);
		UPackage*     NewPackage = CreatePackage( *InPackageName);

		if (OptionalBase)
		{
			// Duplicate the level sequence into the asset package
			OutAsset = Cast<AssetType>(StaticDuplicateObject(OptionalBase, NewPackage, *NewAssetName, RF_NoFlags));
			OutAsset->SetFlags(RF_Public | RF_Standalone | RF_Transactional);
		}
		else if (OptionalClass)
		{
			OutAsset = NewObject<AssetType>(NewPackage, OptionalClass, *NewAssetName, RF_Public | RF_Standalone | RF_Transactional);
		}
		else
		{
			// Create a new level sequence from scratch
			OutAsset = NewObject<AssetType>(NewPackage, *NewAssetName, RF_Public | RF_Standalone | RF_Transactional);
		}

		return true;
	}

	template<typename AssetType>
	static bool CreateNewAssetPackage(FString& InPackageName, TObjectPtr<AssetType>& OutAsset, FText* OutError, AssetType* OptionalBase = nullptr, UClass* OptionalClass = nullptr)
	{
		AssetType* RawOutAsset = nullptr;
		if (CreateNewAssetPackage(InPackageName, RawOutAsset, OutError, OptionalBase, OptionalClass))
		{
			OutAsset = RawOutAsset;
			return true;
		}
		return false;
	}

	/**
	 * Utility function that creates an asset with the specified asset path and name.
	 * If the asset cannot be created (as one already exists), we try to postfix the asset
	 * name until we can successfully create the asset.
	 */
	template<typename AssetType>
	static AssetType* MakeNewAsset(const FString& BaseAssetPath, const FString& BaseAssetName)
	{
		auto MakeAssetObjectPath = [](const FString& BaseAssetPath, const FString& AssetName) -> FSoftObjectPath
			{
				return FSoftObjectPath((BaseAssetPath / AssetName) + TEXT(".") + AssetName);
			};

		FString SanitizedBaseAssetPath = SanitizePackageName(BaseAssetPath);
		FString SanitizedBaseAssetName = SanitizePackageName(BaseAssetName);

		FString SourceAssetName = FPaths::MakeValidFileName(SanitizedBaseAssetName);
		FSoftObjectPath AssetObjectPath = MakeAssetObjectPath(SanitizedBaseAssetPath, SourceAssetName);

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(AssetObjectPath);

		// if object with same name exists, try a different name until we don't find one
		int32 ExtensionIndex = 0;
		FString AssetName = SourceAssetName;
		while (AssetData.IsValid() && AssetData.GetClass() == AssetType::StaticClass())
		{
			AssetName = FString::Printf(TEXT("%s_%d"), *SourceAssetName, ExtensionIndex);
			AssetObjectPath = MakeAssetObjectPath(SanitizedBaseAssetPath, AssetName);
			AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(AssetObjectPath);

			ExtensionIndex++;
		}

		// Create the new asset in the package we just made
		const FString AssetPath = (SanitizedBaseAssetPath / AssetName);

		FString FileName;
		if (FPackageName::TryConvertLongPackageNameToFilename(AssetPath, FileName))
		{
			UObject* Package = CreatePackage( *AssetPath);
			return NewObject<AssetType>(Package, *AssetName, RF_Public | RF_Standalone);
		}

		UE_LOGF(LogTakesCore, Error, "Couldn't create file for package %ls", *AssetPath);

		return nullptr;
	}

	TAKESCORE_API UWorld* DiscoverSourceWorld();

#if WITH_EDITOR
	TAKESCORE_API TSharedPtr<ISequencer> OpenSequencer(ULevelSequence* LevelSequence, FText* OutError);
	
	/* The time at which to record. Taken from the Sequencer global time, otherwise based on timecode */
	TAKESCORE_API FQualifiedFrameTime GetRecordTime(TSharedPtr<ISequencer> Sequencer, ULevelSequence* SequenceAsset,
		const FTimecode& TimecodeAtStart, bool bStartAtCurrentTimecode, float TimeDilation = 1.f);

	/** Find the shared playback state from the current Sequencer. This can return nullptr. */
	TAKESCORE_API TSharedPtr<const UE::MovieScene::FSharedPlaybackState> FindSequencerSharedPlaybackState(ULevelSequence* LevelSequence);
#endif // WITH_EDITOR


	/* The time at which to record, based on timecode */
	TAKESCORE_API FQualifiedFrameTime GetRecordTime(ULevelSequence* SequenceAsset,
		const FTimecode& TimecodeAtStart, bool bStartAtCurrentTimecode, float TimeDilation = 1.f);
	
	/** Retrieve an actor guid for moviescene. */
	TAKESCORE_API FGuid ResolveActorFromSequence(const AActor* InActor, const ULevelSequence* CurrentSequence);

	/** Retrieve an actor guid for the given sequence, providing a shared playback state. This uses the shared playback state as the lookup context for the binding. */
	TAKESCORE_API FGuid ResolveActorFromSequence(const AActor* InActor, const ULevelSequence* CurrentSequence, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState);
	
	/** Checks if a source is considered possessable. */
	TAKESCORE_API bool IsSourcePossessable(const UTakeRecorderSource* InSource);

	/**
	 * Returns true if the track has any keys across all of its sections and channels.
	 */
	TAKESCORE_API bool DoesTrackHaveAnyKeys(UMovieSceneTrack& Track);
} // namespace TakesUtils
