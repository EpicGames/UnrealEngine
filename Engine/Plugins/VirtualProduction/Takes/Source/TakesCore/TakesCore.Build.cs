// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TakesCore : ModuleRules
{
	public TakesCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"NamingTokens", 
				"TakeMovieScene",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"LevelSequence",
				"SlateCore",
				"MovieScene",
				"MovieSceneTracks",
				"Engine",
				"SerializedRecorderInterface"
			}
		);
		
		PublicIncludePathModuleNames.AddRange(
			new string[]
			{
				"TakeRecorderNamingTokens",
				"TakeTrackRecorders"
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"EditorFramework",
					"UnrealEd",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"MovieSceneTools",
					"LevelSequenceEditor",
				}
			);
		}
	}
}
