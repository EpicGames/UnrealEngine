{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Take Recorder",
	"Description": "A suite of tools and interfaces designed for recording, reviewing and playing back takes in a virtual production environment.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedPrograms": ["LiveLinkHub"],
	"Plugins":
	[
		{
			"Name": "AudioCapture",
			"Enabled": true
		},
		{
			"Name": "LevelSequenceEditor",
			"Enabled": true
		},
		{
			"Name": "NamingTokens",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "TakesCore",
			"Type": "Runtime",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"LoadingPhase": "PostDefault"
		},
		{
			"Name": "TakeTrackRecorders",
			"Type": "Runtime",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"LoadingPhase": "PostDefault"
		},
		{
			"Name": "TakeRecorderSources",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"Type": "Runtime",
			"LoadingPhase": "PostDefault"
		},
		{
			"Name": "TakeRecorder",
			"Type": "Runtime",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"LoadingPhase": "PostDefault"
		},
		{
			"Name": "TakeRecorderEditor",
			"Type": "UncookedOnly",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"LoadingPhase": "PostDefault"
		},
		{
			"Name": "TakeMovieScene",
			"Type": "Runtime",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"LoadingPhase": "Default"
		},
		{
			"Name": "TakeSequencer",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CacheTrackRecorder",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"Type": "UncookedOnly",
			"LoadingPhase": "PostDefault"
		},
		{
			"Name": "TakeRecorderNamingTokens",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"Type": "Runtime",
			"LoadingPhase": "PostDefault"
		}
	]
}